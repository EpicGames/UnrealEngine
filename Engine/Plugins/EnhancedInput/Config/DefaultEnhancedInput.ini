[CoreRedirects]
+FunctionRedirects=(OldName="/Script/InputMappingContext.UnmapAction", NewName="/Script/InputMappingContext.UnmapAllKeysFromAction")

; 5.2
+FunctionRedirects=(OldName="/Script/EnhancedInput.EnhancedInputSubsystemInterface.AddPlayerMappedKey", NewName="/Script/EnhancedInput.EnhancedInputSubsystemInterface.K2_AddPlayerMappedKeyInSlot")
+FunctionRedirects=(OldName="/Script/EnhancedInput.EnhancedInputSubsystemInterface.RemovePlayerMappedKey", NewName="/Script/EnhancedInput.EnhancedInputSubsystemInterface.K2_RemovePlayerMappedKeyInSlot")
+FunctionRedirects=(OldName="/Script/EnhancedInput.EnhancedInputSubsystemInterface.GetPlayerMappedKey", NewName="/Script/EnhancedInput.EnhancedInputSubsystemInterface.K2_GetPlayerMappedKeyInSlot")
; 5.6
+PropertyRedirects=(OldName="EnhancedInput.PlayerKeyMapping.AssociatedInputAction",NewName="EnhancedInput.PlayerKeyMapping.AssociatedInputActionSoft")
+FunctionRedirects=(OldName="/Script/EnhancedInput.EnhancedInputUserSettings.GetCurrentKeyProfile", NewName="/Script/EnhancedInput.EnhancedInputUserSettings.GetActiveKeyProfile")
+FunctionRedirects=(OldName="/Script/EnhancedInput.EnhancedInputUserSettings.GetKeyProfileWithIdentifier", NewName="/Script/EnhancedInput.EnhancedInputUserSettings.GetKeyProfileWithId")
+FunctionRedirects=(OldName="/Script/EnhancedInput.EnhancedInputUserSettings.SetKeyProfile", NewName="/Script/EnhancedInput.EnhancedInputUserSettings.SetActiveKeyProfile")
+FunctionRedirects=(OldName="/Script/EnhancedInput.EnhancedInputUserSettings.GetCurrentKeyProfileIdentifier", NewName="/Script/EnhancedInput.EnhancedInputUserSettings.GetActiveKeyProfileId")
+FunctionRedirects=(OldName="/Script/EnhancedInput.EnhancedPlayerMappableKeyProfile.GetProfileIdentifer", NewName="/Script/EnhancedInput.EnhancedPlayerMappableKeyProfile.GetProfileIdString")
+PropertyRedirects=(OldName="EnhancedInput.MapPlayerKeyArgs.ProfileId",NewName="EnhancedInput.MapPlayerKeyArgs.ProfileIdString")
+PropertyRedirects=(OldName="EnhancedInput.PlayerMappableKeyProfileCreationArgs.ProfileIdentifier",NewName="EnhancedInput.PlayerMappableKeyProfileCreationArgs.ProfileStringIdentifier")

;5.7
+PropertyRedirects=(OldName="EnhancedInput.InputMappingContext.Mappings",NewName="EnhancedInput.InputMappingContext.DefaultKeyMappings.Mappings")

;[/Script/Engine.Engine]
;+ActiveClassRedirects=(OldName="/Script/EnhancedInput.InputMappingContext:Mappings", NewName="/Script/EnhancedInput.InputMappingContext:DefaultKeyMappings.Mappings")
