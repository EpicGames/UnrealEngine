[/Script/EngineSettings.ConsoleSettings]
+ManualAutoCompleteList=(Command="ShowDebug EnhancedInput",Desc="Displays debug information about the current state of any Enhanced Input Mapping Contexts")
+ManualAutoCompleteList=(Command="ShowDebug WorldSubsystemInput",Desc="Displays debug information about the current state of any Enhanced Input Mapping Contexts applied to the Enhanced Input world subsystem.")
+ManualAutoCompleteList=(Command="ShowDebug InputSettings",Desc="Displays debug information any user input settings, such as player mappable keys.")
