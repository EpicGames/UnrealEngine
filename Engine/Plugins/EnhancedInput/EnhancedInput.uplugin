{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Enhanced Input",
	"Description": "Input handling that allows for contextual and dynamic mappings.",
	"Category": "Input",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/enhanced-input-in-unreal-engine/",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "EnhancedInput",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "InputBlueprintNodes",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "InputEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "DataValidation",
			"Enabled": true
		}
	]
}