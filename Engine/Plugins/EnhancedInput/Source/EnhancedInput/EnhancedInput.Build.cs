// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class EnhancedInput : ModuleRules
	{
        public EnhancedInput(ReadOnlyTargetRules Target) : base(Target)
        {
			// Uncomment this line to make for easier debugging
	        //OptimizeCode = CodeOptimization.Never;
	        
            PrivateDependencyModuleNames.AddRange(
				new string[] {
                    "ApplicationCore",
					"Core",
					"CoreUObject",
                    "Engine",
					"InputCore",
					"Slate",
                    "SlateCore",
                    "DeveloperSettings",
                    "GameplayTags",
                }
            );
        }
    }
}
