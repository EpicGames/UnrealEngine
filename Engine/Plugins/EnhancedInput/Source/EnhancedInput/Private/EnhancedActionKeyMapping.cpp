// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedActionKeyMapping.h"

#include "HAL/IConsoleManager.h"
#include "Internationalization/Text.h"
#include "Misc/DataValidation.h"
#include "PlayerMappableKeySettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedActionKeyMapping)

#define LOCTEXT_NAMESPACE "ActionKeyMapping"

#if WITH_EDITOR
namespace UE::EnhancedInput
{
	static TAutoConsoleVariable<bool> CVarCheckForEmptyKeyMappingsDuringValidation
	(
		TEXT("EnhancedInput.Mappings.bCheckForEmptyKeyMappingsDuringValidation"),
		false,
		TEXT("When true, Enhanced Input key mappings will throw an error during validation if they are mapped to an empty key."),
		ECVF_Default
	);
}
#endif	// WITH_EDITOR

UPlayerMappableKeySettings* FEnhancedActionKeyMapping::GetPlayerMappableKeySettings() const
{
	switch (SettingBehavior)
	{
		case EPlayerMappableKeySettingBehaviors::InheritSettingsFromAction:	
			return Action != nullptr ? Action->GetPlayerMappableKeySettings() : nullptr;
		case EPlayerMappableKeySettingBehaviors::OverrideSettings:			
			return PlayerMappableKeySettings;
		case EPlayerMappableKeySettingBehaviors::IgnoreSettings:				
			return nullptr;
	}
	return nullptr;
}

FEnhancedActionKeyMapping::FEnhancedActionKeyMapping(const UInputAction* InAction /*= nullptr*/, const FKey InKey /*= EKeys::Invalid*/)
	: Action(InAction)
	, Key(InKey)
	, bShouldBeIgnored(false)
	, bHasAlwaysTickTrigger(false)
{}

FName FEnhancedActionKeyMapping::GetMappingName() const
{
	if (IsPlayerMappable())
	{
		if (UPlayerMappableKeySettings* MappableKeySettings = GetPlayerMappableKeySettings())
		{
			return MappableKeySettings->MakeMappingName(this);
		}
	}
	return NAME_None;
}

const FText& FEnhancedActionKeyMapping::GetDisplayName() const
{
	if (UPlayerMappableKeySettings* MappableKeySettings = GetPlayerMappableKeySettings())
	{
		return MappableKeySettings->DisplayName;
	}
	return FText::GetEmpty();
}

const FText& FEnhancedActionKeyMapping::GetDisplayCategory() const
{
	if (UPlayerMappableKeySettings* MappableKeySettings = GetPlayerMappableKeySettings())
	{
		return MappableKeySettings->DisplayCategory;
	}
	return FText::GetEmpty();
}

bool FEnhancedActionKeyMapping::IsPlayerMappable() const
{
	return GetPlayerMappableKeySettings() != nullptr;
}

#if WITH_EDITOR
EDataValidationResult FEnhancedActionKeyMapping::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = EDataValidationResult::Valid;

	//Validate Action Reference.
	if (Action == nullptr)
	{
		Result = EDataValidationResult::Invalid;
		Context.AddError(LOCTEXT("NullInputAction", "A mapping cannot have an empty input action!"));
	}

	// Validate Settings.
	if (PlayerMappableKeySettings != nullptr)
	{
		Result = CombineDataValidationResults(Result, PlayerMappableKeySettings->IsDataValid(Context));
	}

	// Check for an invalid key mapping if enabled. Having an invalid key mapping is not technically disallowed by Enhanced Input, but 
	// can oftentimes having an invalid key mapping could be something that you want to track down because of some custom system you have
	// made for your game. This setting makes it really easy to track down any empty mappings! 
	if (UE::EnhancedInput::CVarCheckForEmptyKeyMappingsDuringValidation.GetValueOnAnyThread() && Key == EKeys::Invalid)
	{
		Result = EDataValidationResult::Invalid;
		
		FFormatNamedArguments Args;
		Args.Add(TEXT("ActionName"), FText::FromString(GetNameSafe(Action)));

		const FText ErrorMsg =
			FText::Format(
				LOCTEXT("MappedToInvalidKey", "Mapping to Input Action {ActionName}. \nEnhanced Input Key Mappings cannot be mapped to EKeys::Invalid when 'EnhancedInput.Mappings.bCheckForEmptyKeyMappingsDuringValidation' is true!"),
				Args);
		
		Context.AddError(ErrorMsg);
	}

	// Validate the triggers.
	bool bContextContainsComboTrigger = false;
	bool bContextContainsNonComboTrigger = false;
	for (const UInputTrigger* Trigger : Triggers)
	{
		if (Trigger != nullptr)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// check if it the trigger is a combo or not
			Trigger->IsA(UInputTriggerCombo::StaticClass()) ? bContextContainsComboTrigger = true : bContextContainsNonComboTrigger = true;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			
			Result = CombineDataValidationResults(Result, Trigger->IsDataValid(Context));
		}
		else
		{
			Result = EDataValidationResult::Invalid;
			FFormatNamedArguments Args;
			Args.Add("InputActionName", FText::FromString(GetNameSafe(Action)));
			Context.AddError(FText::Format(LOCTEXT("NullInputTriggerWithAction", "There cannot be a null Input Trigger on a key mapping for Input Action '{InputActionName}'"), Args));
		}
	}

	if (Action)
	{
		bool bInputActionContainsComboTrigger = false;
	    bool bInputActionContainsNonComboTrigger = false;
		// we also need to check the input action triggers for combo triggers
		for (const UInputTrigger* Trigger : Action->Triggers)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (Trigger != nullptr)
			{
				Trigger->IsA(UInputTriggerCombo::StaticClass()) ? bInputActionContainsComboTrigger = true : bInputActionContainsNonComboTrigger = true;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		FFormatNamedArguments Args;
		Args.Add("InputActionName", FText::FromName(Action->GetFName()));
		Args.Add("KeyBeingMapped", Key.GetDisplayName());
		FText DefaultComboNonComboWarning = LOCTEXT("DefaultComboNonComboWarningText", "The mapping of {InputActionName} to {KeyBeingMapped} has a Combo Trigger ({ComboTriggerLocation}) with additional non-combo triggers ({NonComboTriggerLocation}). Mixing Combo Triggers with other types of Triggers is not supported. Consider putting the Combo Trigger(s) on a seperate mapping or making a seperate Input Action for them.");
		if (bContextContainsComboTrigger)
		{
			Args.Add("ComboTriggerLocation", LOCTEXT("ComboInContextText", "From the Mapping Context"));
			if (bContextContainsNonComboTrigger)
			{
				Result = EDataValidationResult::Invalid;
				Args.Add("NonComboTriggerLocation", LOCTEXT("NonComboInContextText", "From the Mapping Context"));
				Context.AddError(FText::Format(DefaultComboNonComboWarning, Args));
			}
			// Input Action contains non-combo trigger(s) 
			if (bInputActionContainsNonComboTrigger)
			{
				Args.Add("NonComboTriggerLocation", LOCTEXT("NonComboInInputActionText", "From the Input Action"));
				Result = EDataValidationResult::Invalid;
				Context.AddError(FText::Format(DefaultComboNonComboWarning, Args));
			}
		}
		// Input Action contains combo trigger(s)
		if (bInputActionContainsComboTrigger)
		{
			Args.Add("ComboTriggerLocation", LOCTEXT("ComboInInputActionText", "From the Input Action"));
			// Context contains non-combo trigger(s)
			if (bContextContainsNonComboTrigger)
			{
				Result = EDataValidationResult::Invalid;
				Args.Add("NonComboTriggerLocation", LOCTEXT("NonComboInContextText", "From the Mapping Context"));
				Context.AddError(FText::Format(DefaultComboNonComboWarning, Args));
			}
			// Input Action contains non-combo trigger(s) 
			if (bInputActionContainsNonComboTrigger)
			{
				Args.Add("NonComboTriggerLocation", LOCTEXT("NonComboInInputActionText", "From the Input Action"));
				Result = EDataValidationResult::Invalid;
				Context.AddError(FText::Format(DefaultComboNonComboWarning, Args));
			}
		}
	}
	
	// Validate the modifiers.
	for (const UInputModifier* Modifier : Modifiers)
	{
		if (Modifier != nullptr)
		{
			Result = CombineDataValidationResults(Result, Modifier->IsDataValid(Context));
		}
		else
		{
			Result = EDataValidationResult::Invalid;
			FFormatNamedArguments Args;
			Args.Add("InputActionName", FText::FromString(GetNameSafe(Action)));
			Context.AddError(FText::Format(LOCTEXT("NullInputModifierWithAction", "There cannot be a null Input Modifier on a key mapping for Input Action '{InputActionName}'"), Args));
		}
	}
	return Result;
}

#endif // WITH_EDITOR

bool FEnhancedActionKeyMapping::operator==(const FEnhancedActionKeyMapping& Other) const
{
	return (Action == Other.Action &&
			Key == Other.Key &&
			Triggers == Other.Triggers &&
			Modifiers == Other.Modifiers);
}

#undef LOCTEXT_NAMESPACE
