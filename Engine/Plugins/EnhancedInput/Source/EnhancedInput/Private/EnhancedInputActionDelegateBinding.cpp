// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputActionDelegateBinding.h"
#include "EnhancedInputComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedInputActionDelegateBinding)

UEnhancedInputActionDelegateBinding::UEnhancedInputActionDelegateBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UEnhancedInputActionDelegateBinding::BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const
{
	UEnhancedInputComponent* Component = Cast<UEnhancedInputComponent>(InputComponent);
	if (!Component)
	{
		return;
	}

	for(const FBlueprintEnhancedInputActionBinding& Binding : InputActionDelegateBindings)
	{
		Component->BindAction(Binding.InputAction.Get(), Binding.TriggerEvent, ObjectToBindTo, Binding.FunctionNameToBind);
	}
}

UEnhancedInputActionValueBinding::UEnhancedInputActionValueBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UEnhancedInputActionValueBinding::BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const
{
	UEnhancedInputComponent* Component = Cast<UEnhancedInputComponent>(InputComponent);
	if (!Component)
	{
		return;
	}

	for (const FBlueprintEnhancedInputActionBinding& Binding : InputActionValueBindings)
	{
		Component->BindActionValue(Binding.InputAction);
	}
}
