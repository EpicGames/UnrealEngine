// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedInputComponent)

/* UEnhancedInputComponent interface
 *****************************************************************************/

UEnhancedInputComponent::UEnhancedInputComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Priority = -1;
	bBlockInput = false;
}

bool UEnhancedInputComponent::ShouldCheckForLegacyKeyConsumption() const
{
	return bShouldCheckForLegacyKeyConsumption;
}

void UEnhancedInputComponent::SetShouldFireDelegatesInEditor(const bool bInNewValue)
{
	bShouldFireDelegatesInEditor = bInNewValue;

	// Propegate this info to any bound delegates
	if (HasBindings())
	{
		for (auto& Binding : GetActionEventBindings())
		{
			Binding->SetShouldFireWithEditorScriptGuard(bShouldFireDelegatesInEditor);
		}
	}
}

bool UEnhancedInputComponent::HasBindings( ) const
{
	return GetActionEventBindings().Num() > 0 || GetActionValueBindings().Num() > 0 || GetDebugKeyBindings().Num() > 0 || Super::HasBindings();
}

void UEnhancedInputComponent::ClearActionBindings()
{
	Super::ClearActionBindings();

	ClearActionEventBindings();
}

void UEnhancedInputComponent::ClearBindingsForObject(UObject* InOwner)
{
	Super::ClearBindingsForObject(InOwner);

	for (int32 Index = EnhancedActionEventBindings.Num() - 1; Index >= 0; --Index)
	{
		if (EnhancedActionEventBindings[Index]->IsBoundToObject(InOwner))
		{
			EnhancedActionEventBindings.RemoveAtSwap(Index, EAllowShrinking::No);
		}
	}
}

template<typename T>
bool RemoveBindingByIndex(TArray<T>& Bindings, const int32 BindingIndex)
{
	if (BindingIndex >= 0 && BindingIndex < Bindings.Num())
	{
		Bindings.RemoveAt(BindingIndex, EAllowShrinking::No);
		return true;
	}
	return false;
}

bool UEnhancedInputComponent::RemoveActionEventBinding(const int32 BindingIndex)
{
	return RemoveBindingByIndex(EnhancedActionEventBindings, BindingIndex);
}

bool UEnhancedInputComponent::RemoveActionValueBinding(const int32 BindingIndex)
{
	return RemoveBindingByIndex(EnhancedActionValueBindings, BindingIndex);
}

bool UEnhancedInputComponent::RemoveDebugKeyBinding(const int32 BindingIndex)
{
	return RemoveBindingByIndex(DebugKeyBindings, BindingIndex);
}

bool UEnhancedInputComponent::RemoveBindingByHandle(const uint32 Handle)
{
	auto TryRemove = [](auto& Bindings, auto Predicate)
	{
		int32 EventFoundIndex = Bindings.IndexOfByPredicate(Predicate);
		return EventFoundIndex != INDEX_NONE && RemoveBindingByIndex(Bindings, EventFoundIndex);
	};

	// TODO: Searching three separate arrays is slow. Store a map of handles -> type? (and index?)
	return	TryRemove(EnhancedActionEventBindings, [Handle](const TUniquePtr<FEnhancedInputActionEventBinding>& Binding) { return Binding->GetHandle() == Handle; }) ||
			TryRemove(EnhancedActionValueBindings, [Handle](const FEnhancedInputActionValueBinding& Binding) { return Binding.GetHandle() == Handle; }) ||
			TryRemove(DebugKeyBindings, [Handle](const TUniquePtr<FInputDebugKeyBinding>& Binding) { return Binding->GetHandle() == Handle; });
}

bool UEnhancedInputComponent::RemoveBinding(const FInputBindingHandle& BindingToRemove)
{
	return RemoveBindingByHandle(BindingToRemove.GetHandle());
}

FInputActionValue UEnhancedInputComponent::GetBoundActionValue(const UInputAction* Action) const
{
	for (const FEnhancedInputActionValueBinding& Binding : GetActionValueBindings())
	{
		if (Binding.GetAction() == Action)
		{
			return Binding.GetValue();
		}
	}
	return FInputActionValue(Action ? Action->ValueType : EInputActionValueType::Axis3D, FVector::ZeroVector);
}

// Must be in C++ to avoid duplicate statics across execution units
static uint32 GInputBindingHandle = 1;

FInputBindingHandle::FInputBindingHandle()
{
	// Handles are shared between all binding types
	Handle = GInputBindingHandle++;
}
