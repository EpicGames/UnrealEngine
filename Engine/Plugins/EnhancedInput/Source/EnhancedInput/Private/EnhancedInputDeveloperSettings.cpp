// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputDeveloperSettings.h"
#include "EnhancedInputPlatformSettings.h"
#include "EnhancedPlayerInput.h"
#include "HAL/IConsoleManager.h"
#include "UserSettings/EnhancedInputUserSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedInputDeveloperSettings)

UEnhancedInputDeveloperSettings::UEnhancedInputDeveloperSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, UserSettingsClass(UEnhancedInputUserSettings::StaticClass())
	, DefaultPlayerMappableKeyProfileClass(UEnhancedPlayerMappableKeyProfile::StaticClass())
	, DefaultWorldInputClass(UEnhancedPlayerInput::StaticClass())
	, bSendTriggeredEventsWhenInputIsFlushed(true)
	, bEnableUserSettings(true)
	, bEnableDefaultMappingContexts(true)
	, bShouldOnlyTriggerLastActionInChord(true)
	, bEnableInputModeFiltering(true)
	, bEnableWorldSubsystem(false)
{
	PlatformSettings.Initialize(UEnhancedInputPlatformSettings::StaticClass());
}