// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputLibrary.h"

#include "Engine/LocalPlayer.h"
#include "EnhancedActionKeyMapping.h"
#include "EnhancedInputComponent.h"
#include "EnhancedInputDeveloperSettings.h"
#include "EnhancedInputModule.h"
#include "EnhancedInputSubsystems.h"
#include "GameFramework/Actor.h"
#include "GameFramework/PlayerController.h"
#include "HAL/IConsoleManager.h"
#include "InputTriggers.h"
#include "PlayerMappableKeySettings.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedInputLibrary)

namespace UE::Input
{
	static void RebuildAllControlMappings(const TArray<FString>& Args)
	{
		UEnhancedInputLibrary::ForEachSubsystem([Args](IEnhancedInputSubsystemInterface* Subsystem)
		{
			Subsystem->RequestRebuildControlMappings();
		});
	}

	static FAutoConsoleCommand ConsoleCommandDumpProfileToLog(
		TEXT("EnhancedInput.RebuildControlMappings"),
		TEXT("Calls RequestRebuildControlMappings on every available Enhanced Input Subsystem"),
		FConsoleCommandWithArgsDelegate::CreateStatic(UE::Input::RebuildAllControlMappings));
}


void UEnhancedInputLibrary::ForEachSubsystem(TFunctionRef<void(IEnhancedInputSubsystemInterface*)> SubsystemPredicate)
{
	// World subsystem
	if (GetDefault<UEnhancedInputDeveloperSettings>()->bEnableWorldSubsystem)
	{
		for (TObjectIterator<UEnhancedInputWorldSubsystem> It; It; ++It)
		{
			SubsystemPredicate(Cast<IEnhancedInputSubsystemInterface>(*It));
		}
	}

	// Players
	for (TObjectIterator<UEnhancedInputLocalPlayerSubsystem> It; It; ++It)
	{
		SubsystemPredicate(Cast<IEnhancedInputSubsystemInterface>(*It));
	}
}

void UEnhancedInputLibrary::RequestRebuildControlMappingsUsingContext(const UInputMappingContext* Context, bool bForceImmediately)
{
	ForEachSubsystem([Context, bForceImmediately](IEnhancedInputSubsystemInterface* Subsystem) 
		{
			check(Subsystem);
			if (Subsystem && Subsystem->HasMappingContext(Context))
			{
				FModifyContextOptions Options {};
				Options.bForceImmediately = bForceImmediately;
				Subsystem->RequestRebuildControlMappings(Options);
			}
		});
}

FInputActionValue UEnhancedInputLibrary::GetBoundActionValue(AActor* Actor, const UInputAction* Action)
{
	if (Actor && Action)
	{
		UEnhancedInputComponent* EIC = Cast<UEnhancedInputComponent>(Actor->InputComponent);
		return EIC ? EIC->GetBoundActionValue(Action) : FInputActionValue(Action->ValueType, FVector::ZeroVector);
	}
	else
	{
		if (!Actor)
		{
			UE_LOGF(LogEnhancedInput, Error, "UEnhancedInputLibrary::GetBoundActionValue was called with an invalid Actor!");
		}
		
		if (!Action)
		{
			UE_LOGF(LogEnhancedInput, Error, "UEnhancedInputLibrary::GetBoundActionValue was called with an invalid Action!");
		}		
		ensureMsgf(false, TEXT("Invalid GetBoundActionValue call. Check logs for details!"));
	}
	return FInputActionValue();
}


void UEnhancedInputLibrary::BreakInputActionValue(FInputActionValue InActionValue, double& X, double& Y, double& Z, EInputActionValueType& Type)
{
	FVector AsAxis3D = InActionValue.Get<FInputActionValue::Axis3D>();
	X = AsAxis3D.X;
	Y = AsAxis3D.Y;
	Z = AsAxis3D.Z;
	Type = InActionValue.GetValueType();
}

UPlayerMappableKeySettings* UEnhancedInputLibrary::GetPlayerMappableKeySettings(const FEnhancedActionKeyMapping& ActionKeyMapping)
{
	return ActionKeyMapping.GetPlayerMappableKeySettings();
}

FName UEnhancedInputLibrary::GetMappingName(const FEnhancedActionKeyMapping& ActionKeyMapping)
{
	return ActionKeyMapping.GetMappingName();
}

bool UEnhancedInputLibrary::IsActionKeyMappingPlayerMappable(const FEnhancedActionKeyMapping& ActionKeyMapping)
{
	return ActionKeyMapping.IsPlayerMappable();
}

FInputActionValue UEnhancedInputLibrary::MakeInputActionValueOfType(double X, double Y, double Z, const EInputActionValueType ValueType)
{
	return FInputActionValue(ValueType, FVector(X, Y, Z));
}

// FInputActionValue type conversions

bool UEnhancedInputLibrary::Conv_InputActionValueToBool(FInputActionValue InValue)
{
	return InValue.Get<bool>();
}

double UEnhancedInputLibrary::Conv_InputActionValueToAxis1D(FInputActionValue InValue)
{
	return static_cast<double>(InValue.Get<FInputActionValue::Axis1D>());
}

FVector2D UEnhancedInputLibrary::Conv_InputActionValueToAxis2D(FInputActionValue InValue)
{
	return InValue.Get<FInputActionValue::Axis2D>();
}

FVector UEnhancedInputLibrary::Conv_InputActionValueToAxis3D(FInputActionValue InValue)
{
	return InValue.Get<FInputActionValue::Axis3D>();
}

FString UEnhancedInputLibrary::Conv_InputActionValueToString(FInputActionValue ActionValue)
{
	return ActionValue.ToString();
}

FString UEnhancedInputLibrary::Conv_TriggerEventValueToString(const ETriggerEvent TriggerEvent)
{
	return UE::Input::LexToString(TriggerEvent);
}

void UEnhancedInputLibrary::FlushPlayerInput(APlayerController* PlayerController)
{
	if (PlayerController)
	{
		PlayerController->FlushPressedKeys();
	}
}