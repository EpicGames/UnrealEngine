// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputModule.h"

#include "DrawDebugHelpers.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/LocalPlayer.h"
#include "EnhancedInputLibrary.h"
#include "EnhancedInputSubsystems.h"
#include "GameFramework/HUD.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/Pawn.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "EnhancedInputDeveloperSettings.h"
#include "UserSettings/EnhancedInputUserSettings.h"

#define LOCTEXT_NAMESPACE "EnhancedInput"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FKey FEnhancedInputKeys::ComboKey("ComboKey");
PRAGMA_ENABLE_DEPRECATION_WARNINGS

DEFINE_LOG_CATEGORY(LogEnhancedInput);

class FEnhancedInputModule : public IEnhancedInputModule, public FTickableGameObject
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface interface

	// FTickableGameObject interface
	virtual void Tick(float DeltaTime) override;

	virtual ETickableTickType GetTickableTickType() const override
	{
		// Input can only possibly exist on the client, so never attempt to tick it on the server.
		if (IsRunningDedicatedServer())
		{
			return ETickableTickType::Never;
		}
		
		return ETickableTickType::Always;
	}

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FEnhancedInputModule, STATGROUP_Tickables);
	}

	virtual bool IsTickableWhenPaused() const override
	{
		return true;
	}
	// End FTickableGameObject interface

	virtual UEnhancedInputLibrary* GetLibrary() override
	{
		ensureAlwaysMsgf(Library, TEXT("Trying to access the enhanced input library after the plugin has been unloaded!"));
		return Library;
	}

	FInputActionValue ParseValue(const TArray<FString>& Args, FInputActionValue Default)
	{
		FInputActionValue Value(Default);
		if (Args.Num() > 1)
		{
			// Build value string from the remaining args
			FString ValueStr;
			for (int32 i = 1; i < Args.Num(); ++i)
			{
				ValueStr += Args[i];
				// There must be a space beteen the values of axis types for InitFromString to work correctly
				// There is no need add a space to the end
				if(i < Args.Num() - 1)
				{
					ValueStr += " ";
				}
			}

			FVector2D Value2D;
			FVector Value3D;

			if (Value3D.InitFromString(ValueStr))
			{
				Value = Value3D;					// Axis 3D type
			}
			else if (Value2D.InitFromString(ValueStr))
			{
				Value = Value2D;					// Axis 2D type
			}
			else if (ValueStr.IsNumeric())
			{
				Value = FCString::Atof(*ValueStr);	// Axis 1D type
			}
			else
			{
				Value = ValueStr.ToBool();	// Bool type
			}
		}
		return Value;
	}

	void EnableForcedAction(const TArray<FString>& Args, UWorld* World)
	{
		APlayerController* PC = World->GetFirstPlayerController();	// TODO: UGameplayStatics::GetPlayerControllerFromId(OptionalPlayerIdArg)
		if (!PC)
		{
			return;
		}

		if (!Args.Num())
		{
			PC->ClientMessage(TEXT("Expected arguments:\\n   <action> - Name of action to force each tick\\n   <value> - Value to apply. Valid examples are: true, 0.4, \"X=1.0, Y=-2.0\", \"X=0.1, Y=0.2, Z=0.3\""));
			return;
		}

		if (!Cast<UEnhancedPlayerInput>(PC->PlayerInput))
		{
			PC->ClientMessage(TEXT("Player controller is not using the Enhanced Input system."));
			return;
		}

		UInputAction* Action = FindFirstObject<UInputAction>(*Args[0], EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("EnableForcedAction"));
		if (!Action)
		{
			PC->ClientMessage(TEXT("Failed to find action '%s'."), *Args[0]);
			return;
		}

		if(UEnhancedInputLocalPlayerSubsystem* Subsystem = ULocalPlayer::GetSubsystem<UEnhancedInputLocalPlayerSubsystem>(PC->GetLocalPlayer()))
		{
			Subsystem->ApplyForcedInput(Action, ParseValue(Args, FInputActionValue(true)));	// Default to triggered boolean
		}
	}

	void DisableForcedAction(const TArray<FString>& Args, UWorld* World)
	{
		APlayerController* PC = World->GetFirstPlayerController();
		if (!PC)
		{
			return;
		}

		if (!Args.Num())
		{
			PC->ClientMessage(TEXT("Expected arguments:\\n   <action> - Name of action to stop forcing"));
			return;
		}

		UInputAction* Action = FindFirstObject<UInputAction>(*Args[0], EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("DisableForcedAction"));
		if (!Action)
		{
			PC->ClientMessage(TEXT("Failed to find action '%s'."), *Args[0]);
			return;
		}

		if(UEnhancedInputLocalPlayerSubsystem* Subsystem = ULocalPlayer::GetSubsystem<UEnhancedInputLocalPlayerSubsystem>(PC->GetLocalPlayer()))
		{
			Subsystem->RemoveForcedInput(Action);
		}
	}

	void EnableForcedKey(const TArray<FString>& Args, UWorld* World)
	{
		APlayerController* PC = World->GetFirstPlayerController();
		if (!PC)
		{
			return;
		}

		if (!Args.Num())
		{
			PC->ClientMessage(TEXT("Expected arguments:\\n   <key> - Name of key to force each tick\\n   <value> - Value to apply. Valid examples are: true, 0.4"));
			return;
		}

		FKey FoundKey(FName(*Args[0], FNAME_Find));
		if (!FoundKey.IsValid())
		{
			PC->ClientMessage(FString::Printf(TEXT("Failed to locate a key named '%s'"), *Args[0]));
			return;
		}
		
		if(UEnhancedInputLocalPlayerSubsystem* Subsystem = ULocalPlayer::GetSubsystem<UEnhancedInputLocalPlayerSubsystem>(PC->GetLocalPlayer()))
		{
			Subsystem->ApplyForcedInput(FoundKey, ParseValue(Args, FInputActionValue(1.f))); // Default to "pressed" standard key (Axis1D)
		}
	}

	void DisableForcedKey(const TArray<FString>& Args, UWorld* World)
	{
		APlayerController* PC = World->GetFirstPlayerController();
		if (!PC)
		{
			return;
		}

		if (!Args.Num())
		{
			PC->ClientMessage(TEXT("Expected arguments:\\n   <key> - Name of key to stop forcing"));
			return;
		}

		FKey FoundKey(FName(*Args[0], FNAME_Find));
		if (!FoundKey.IsValid())
		{
			PC->ClientMessage(FString::Printf(TEXT("Failed to locate a key named '%s'"), *Args[0]));
			return;
		}


		if (UEnhancedInputLocalPlayerSubsystem* Subsystem = ULocalPlayer::GetSubsystem<UEnhancedInputLocalPlayerSubsystem>(PC->GetLocalPlayer()))
		{
			Subsystem->RemoveForcedInput(FoundKey);
		}
	}

	static void OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

private:
	UEnhancedInputLibrary* Library = nullptr;

	TArray<IConsoleCommand*> ConsoleCommands;

	/** The last frame number we were ticked.  We don't want to tick multiple times per frame */
	uint64 LastFrameNumberWeTicked = (uint64)-1;
};

IMPLEMENT_MODULE(FEnhancedInputModule, EnhancedInput)

void FEnhancedInputModule::StartupModule()
{
	Library = NewObject<UEnhancedInputLibrary>(GetTransientPackage(), UEnhancedInputLibrary::StaticClass(), NAME_None);
	Library->AddToRoot();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FName NAME_EnhancedInput(TEXT("EnhancedInput"));
	EKeys::AddMenuCategoryDisplayInfo(NAME_EnhancedInput, LOCTEXT("EnhancedInputSubCateogry", "Enhanced Input"), TEXT("GraphEditor.KeyEvent_16x"));
	EKeys::AddKey(FKeyDetails(FEnhancedInputKeys::ComboKey, LOCTEXT("ComboKey", "Combo Key (Deprecated)"), FKeyDetails::NotActionBindableKey, NAME_EnhancedInput));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	if (!IsRunningDedicatedServer())
	{
#if ENABLE_DRAW_DEBUG
		AHUD::OnShowDebugInfo.AddStatic(&FEnhancedInputModule::OnShowDebugInfo);
#endif
	    // Register console commands
	    ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		    TEXT("Input.+action"),
		    TEXT("Provide the named action with a constant input value each frame"),
		    FConsoleCommandWithWorldAndArgsDelegate::CreateRaw(this, &FEnhancedInputModule::EnableForcedAction),
		    ECVF_Cheat
	    ));

		ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("Input.-action"),
			TEXT("Stop forcing the named action value each frame"),
			FConsoleCommandWithWorldAndArgsDelegate::CreateRaw(this, &FEnhancedInputModule::DisableForcedAction),
			ECVF_Cheat
		));

		ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("Input.+key"),
			TEXT("Provide the named key with a constant input value each frame"),
			FConsoleCommandWithWorldAndArgsDelegate::CreateRaw(this, &FEnhancedInputModule::EnableForcedKey),
			ECVF_Cheat
		));

		ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("Input.-key"),
			TEXT("Stop forcing the named key each frame"),
			FConsoleCommandWithWorldAndArgsDelegate::CreateRaw(this, &FEnhancedInputModule::DisableForcedKey),
			ECVF_Cheat
		));
	}
}

void FEnhancedInputModule::ShutdownModule()
{
	// Unregister console commands
	for (IConsoleCommand* Command : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Command);
	}

	if (!GExitPurge) // If GExitPurge Object is already gone
	{
		Library->RemoveFromRoot();
	}
	Library = nullptr;
}

void FEnhancedInputModule::Tick(float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEnhancedInputModule::Tick);

	// This may tick multiple times per frame? See MIDIDevice module.
	if (LastFrameNumberWeTicked == GFrameCounter)
	{
		return;
	}
	LastFrameNumberWeTicked = GFrameCounter;

	/* 
	* Tick the World subsystems 
	* The Enhanced Input local player subsystem will have their Player Input's ticked by their owning 
	* Player Controller in APlayerController::TickPlayerInput, but because the world subsystem has no
	* owning controller we need to tick it here.
	*/
	if (GetDefault<UEnhancedInputDeveloperSettings>()->bEnableWorldSubsystem)
	{
		for (TObjectIterator<UEnhancedInputWorldSubsystem> It; It; ++It)
		{
			(*It)->TickPlayerInput(DeltaTime);
		}
	}	

	UEnhancedInputLibrary::ForEachSubsystem([DeltaTime](IEnhancedInputSubsystemInterface* Subsystem)
		{
			Subsystem->RebuildControlMappings();
			Subsystem->TickForcedInput(DeltaTime);
			Subsystem->HandleControlMappingRebuildDelegate();
		});
}


void FEnhancedInputModule::OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	static const FName NAME_EnhancedInput("EnhancedInput");
	static const FName NAME_WorldSubsystemInput("WorldSubsystemInput");
	static const FName NAME_InputSettings("InputSettings");

	if (Canvas)
	{
		if (HUD->ShouldDisplayDebug(NAME_EnhancedInput))
		{
			FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
			DisplayDebugManager.SetFont(GEngine->GetSmallFont());
			DisplayDebugManager.SetDrawColor(FColor::Yellow);
			DisplayDebugManager.DrawString(TEXT("ENHANCED INPUT"));

			bool bFoundValidDebugTarget = false;
			// Checking if the debug target actor is a valid local player, if so then show their debug info
			// If target actor is not a local player then don't display anything
			if (APawn* TargetPawn = Cast<APawn>(HUD->GetCurrentDebugTargetActor()))
			{
				if (APlayerController* PlayerController = Cast<APlayerController>(TargetPawn->GetController()))
				{
					if (const ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(PlayerController->Player))
					{
						if (UEnhancedInputLocalPlayerSubsystem* EnhancedInputSubsystem = ULocalPlayer::GetSubsystem<UEnhancedInputLocalPlayerSubsystem>(LocalPlayer))
						{
							EnhancedInputSubsystem->ShowDebugInfo(Canvas);
							bFoundValidDebugTarget = true;
						}
					}
				}
			}

			if(!bFoundValidDebugTarget)
			{ 
				DisplayDebugManager.SetDrawColor(FColor::Red);
				DisplayDebugManager.DrawString(TEXT("Debug Target is not valid for Enhanced Input Debug Display"));
			}
		}

		if (HUD->ShouldDisplayDebug(NAME_WorldSubsystemInput))
		{
			for (TObjectIterator<UEnhancedInputWorldSubsystem> It; It; ++It)
			{
				It->ShowDebugInfo(Canvas);
			}
		}

#if ENABLE_DRAW_DEBUG
		// Show the debug info for input user settings
		if (HUD->ShouldDisplayDebug(NAME_InputSettings))
		{
			const APawn* TargetPawn = Cast<APawn>(HUD->GetCurrentDebugTargetActor());
			const APlayerController* TargetPlayerController = TargetPawn ? TargetPawn->GetController<APlayerController>() : Cast<APlayerController>(HUD->GetCurrentDebugTargetActor());
			const ULocalPlayer* TargetLocalPlayer = TargetPlayerController ? TargetPlayerController->GetLocalPlayer() : nullptr;
			const UEnhancedInputLocalPlayerSubsystem* TargetEnhancedInputSubsystem = TargetPlayerController ? TargetLocalPlayer->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>() : nullptr;

			//Fallback onto first found input system.
			if (!TargetEnhancedInputSubsystem)
			{
				TObjectIterator<UEnhancedInputLocalPlayerSubsystem> FirstFoundInputSubsystem;
				if (FirstFoundInputSubsystem)
				{
					TargetEnhancedInputSubsystem = *FirstFoundInputSubsystem;
				}
			}
			
			if (TargetEnhancedInputSubsystem)
			{
				if (UEnhancedInputUserSettings* Settings = TargetEnhancedInputSubsystem->GetUserSettings())
				{
					Settings->ShowDebugInfo(Canvas);
				}
				else
				{
					FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
					DisplayDebugManager.SetFont(GEngine->GetLargeFont());
					DisplayDebugManager.SetDrawColor(FColor::Red);
					DisplayDebugManager.DrawString(TEXT("No Input User Settings found! Did you enable it in project settings?"));
				}
			}
		}
#endif	// ENABLE_DRAW_DEBUG

	}
	else
	{
		// Free up visualizer textures
		IEnhancedInputSubsystemInterface::PurgeDebugVisualizations();
	}
}



#undef LOCTEXT_NAMESPACE
