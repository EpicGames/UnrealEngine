// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputPlatformSettings.h"
#include "InputMappingContext.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedInputPlatformSettings)

#define LOCTEXT_NAMESPACE "EnhancedInputPlatformSettings"

//////////////////////////////////////////////////////////////////////////
// UEnhancedInputPlatformData

#if WITH_EDITOR
EDataValidationResult UEnhancedInputPlatformData::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	for (const TPair<TObjectPtr<const UInputMappingContext>, TObjectPtr<const UInputMappingContext>>& Pair : MappingContextRedirects)
	{
		if (!Pair.Key)
		{
			Result = CombineDataValidationResults(Result, EDataValidationResult::Invalid);
			
			FFormatNamedArguments Args;
			Args.Add(TEXT("AssetPath"), FText::FromString(GetPathName()));
			const FText NullKeyError = FText::Format(LOCTEXT("NullKeyError", "'{AssetPath}' does not have a valid key in the MappingContextRedirects!"), Args);
			Context.AddError(NullKeyError);
		}
		
		if (!Pair.Value)
		{
			Result = CombineDataValidationResults(Result, EDataValidationResult::Invalid);
			
			FFormatNamedArguments Args;
			Args.Add(TEXT("AssetPath"), FText::FromString(GetPathName()));
			const FText NullValueError = FText::Format(LOCTEXT("NullValueError", "'{AssetPath}' does not have a valid value in the MappingContextRedirects!"), Args);
			Context.AddError(NullValueError);
		}
	}

	return Result;
}
#endif	// WITH_EDITOR

const UInputMappingContext* UEnhancedInputPlatformData::GetContextRedirect(UInputMappingContext* InContext) const
{
	if (const TObjectPtr<const UInputMappingContext>* RedirectToIMC = MappingContextRedirects.Find(InContext))
	{
		return RedirectToIMC->Get();
	}
	return InContext;
}

//////////////////////////////////////////////////////////////////////////
// UEnhancedInputPlatformSettings

void UEnhancedInputPlatformSettings::GetAllMappingContextRedirects(OUT TMap<TObjectPtr<const UInputMappingContext>, TObjectPtr<const UInputMappingContext>>& OutRedirects)
{
	ForEachInputData([&OutRedirects](const UEnhancedInputPlatformData& Data)
	{
		OutRedirects.Append(Data.GetMappingContextRedirects());
	});
}

void UEnhancedInputPlatformSettings::ForEachInputData(TFunctionRef<void(const UEnhancedInputPlatformData&)> Predicate)
{
	for (TSoftClassPtr<UEnhancedInputPlatformData> InputDataPtr : InputData)
	{
		if (TSubclassOf<UEnhancedInputPlatformData> InputDataClass = InputDataPtr.LoadSynchronous())
		{
			if (const UEnhancedInputPlatformData* Data = InputDataClass.GetDefaultObject())
			{
				Predicate(*Data);
			}
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UEnhancedInputPlatformSettings::LoadInputDataClasses()
{
	if (InputData.Num() != InputDataClasses.Num())
	{
		for (TSoftClassPtr<UEnhancedInputPlatformData> InputDataPtr : InputData)
		{
			if (TSubclassOf<UEnhancedInputPlatformData> InputDataClass = InputDataPtr.LoadSynchronous())
			{
				InputDataClasses.Add(InputDataClass);
			}
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
