// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputSubsystemInterface.h"

#include "EnhancedInputModule.h"
#include "EnhancedInputPlatformSettings.h"
#include "UserSettings/EnhancedInputUserSettings.h"
#include "EnhancedInputDeveloperSettings.h"
#include "GameFramework/PlayerController.h"
#include "HAL/IConsoleManager.h"
#include "InputMappingContext.h"
#include "InputModifiers.h"
#include "InputTriggers.h"
#include "InputMappingQuery.h"
#include "PlayerMappableInputConfig.h"
#include "PlayerMappableKeySettings.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedInputSubsystemInterface)

/* Shared input subsystem functionality.
 * See EnhancedInputSubsystemInterfaceDebug.cpp for debug specific functionality.
 */

static constexpr int32 GGlobalAxisConfigMode_Default = 0;
static constexpr int32 GGlobalAxisConfigMode_All = 1;
static constexpr int32 GGlobalAxisConfigMode_None = 2;

static int32 GGlobalAxisConfigMode = 0;
static FAutoConsoleVariableRef GCVarGlobalAxisConfigMode(
	TEXT("input.GlobalAxisConfigMode"),
	GGlobalAxisConfigMode,
	TEXT("Whether or not to apply Global Axis Config settings. 0 = Default (Mouse Only), 1 = All, 2 = None")
);

static bool bRespectIMCPriorityForTriggers = true;
static FAutoConsoleVariableRef CVarRespectIMCPriorityForTriggers(
	TEXT("input.bRespectIMCPriortyForTriggers"),
	bRespectIMCPriorityForTriggers,
	TEXT("If true, then when an IMC with the same triggers but a higher priority is added, the triggers on the higher priority IMC will be used.\n")
		 TEXT("This CVar will be removed in a future release.")
);

static bool bTrackKeysForAllActionMappings = false;
static FAutoConsoleVariableRef GCVarTrackKeysForAllActionMappings(
	TEXT("EnhancedInput.TrackKeysForAllActionMappings"),
	bTrackKeysForAllActionMappings,
	TEXT("If true, keep track of ALL keys mapped to each action, regardless of whether that mapping has been configured to consume legacy keys. This is needed for Enhanced Input bindings to respect legacy key consumption.")
);

template<typename T>
void DeepCopyPtrArray(const TArray<T*>& From, TArray<T*>& To)
{
	To.Empty(From.Num());
	for (T* ToDuplicate : From)
	{
		if (ToDuplicate)
		{
			To.Add(DuplicateObject<T>(ToDuplicate, nullptr));
		}
	}
}

template<typename T>
void DeepCopyPtrArray(const TArray<T*>& From, TArray<TObjectPtr<T>>& To)
{
	To.Empty(From.Num());
	for (T* ToDuplicate : From)
	{
		if (ToDuplicate)
		{
			To.Add(DuplicateObject<T>(ToDuplicate, nullptr));
		}
	}
}

void IEnhancedInputSubsystemInterface::InitalizeUserSettings()
{
	// Not every implementer of the EI subsystem wants user settings, so leave it up to them to determine if they want it or not
}

UEnhancedInputUserSettings* IEnhancedInputSubsystemInterface::GetUserSettings() const
{
	// Not every implementer of the EI subsystem wants user settings, so leave it up to them to determine if they want it or not
	return nullptr;
}

bool IEnhancedInputSubsystemInterface::TrackingKeysForAllActionMappings()
{
	return bTrackKeysForAllActionMappings;
}

void IEnhancedInputSubsystemInterface::BindUserSettingDelegates()
{
	UEnhancedInputUserSettings* Settings = GetUserSettings();
	if (!Settings)
	{
		UE_LOGF(LogEnhancedInput, Error, "Unable to get the user settings object!");
		return;
	}

	// There is no need to bind to any delegates if the setting is turned off. We shouldn't even get here,
	// but do this in case someone implements this interface
	if (!GetDefault<UEnhancedInputDeveloperSettings>()->bEnableUserSettings)
	{
		UE_LOGF(LogEnhancedInput, Error, "Attempting to bind to user settings delegates but they are disabled in UEnhancedInputDeveloperSettings!");
		return;
	}

	Settings->OnSettingsChanged.AddUniqueDynamic(this, &IEnhancedInputSubsystemInterface::OnUserSettingsChanged);
	Settings->OnKeyProfileChanged.AddUniqueDynamic(this, &IEnhancedInputSubsystemInterface::OnUserKeyProfileChanged);
}

void IEnhancedInputSubsystemInterface::OnUserSettingsChanged(UEnhancedInputUserSettings* Settings)
{
	// We want to rebuild our control mappings whenever a setting has changed
	RequestRebuildControlMappings();
}

void IEnhancedInputSubsystemInterface::OnUserKeyProfileChanged(const UEnhancedPlayerMappableKeyProfile* InNewProfile)
{
	// We want to rebuild our control mappings whenever a setting has changed
	RequestRebuildControlMappings();
}

void IEnhancedInputSubsystemInterface::InjectInputForAction(const UInputAction* Action, FInputActionValue RawValue, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers)
{
	if(UEnhancedPlayerInput* PlayerInput = GetPlayerInput())
	{
		PlayerInput->InjectInputForAction(Action, RawValue, Modifiers, Triggers);
	}
}

void IEnhancedInputSubsystemInterface::InjectInputVectorForAction(const UInputAction* Action, FVector Value, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers)
{
	FInputActionValue RawValue((Action != nullptr) ? Action->ValueType : EInputActionValueType::Boolean, Value);
	InjectInputForAction(Action, RawValue, Modifiers, Triggers);
}

void IEnhancedInputSubsystemInterface::InjectInputForPlayerMapping(const FName MappingName, FInputActionValue RawValue, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers)
{
	InjectInputVectorForPlayerMapping(MappingName, RawValue.Get<FVector>(), Modifiers, Triggers);
}

void IEnhancedInputSubsystemInterface::StartContinuousInputInjectionForAction(const UInputAction* Action, FInputActionValue RawValue, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers)
{
	FInjectedInput& Injection = GetContinuouslyInjectedInputs().FindOrAdd(Action);

	Injection.RawValue = RawValue;
	DeepCopyPtrArray<UInputModifier>(Modifiers, Injection.Modifiers);
	DeepCopyPtrArray<UInputTrigger>(Triggers, Injection.Triggers);
}

void IEnhancedInputSubsystemInterface::StartContinuousInputInjectionForPlayerMapping(const FName MappingName, FInputActionValue RawValue, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers)
{
	if (const UEnhancedInputUserSettings* UserSettings = GetUserSettings())
	{
		if (const UInputAction* Action = UserSettings->FindInputActionForMapping(MappingName))
		{
			StartContinuousInputInjectionForAction(Action, RawValue, Modifiers, Triggers);
		}
		else
		{
			UE_LOGF(LogEnhancedInput, Warning, "Could not find a Input Action for mapping name '%ls'", *MappingName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogEnhancedInput, Warning, "Could not find a valid UEnhancedInputUserSettings object, is it enabled in the project settings?");
	}
}

void IEnhancedInputSubsystemInterface::UpdateValueOfContinuousInputInjectionForAction(const UInputAction* Action, FInputActionValue RawValue)
{
	FInjectedInput& Injection = GetContinuouslyInjectedInputs().FindOrAdd(Action);
	Injection.RawValue = RawValue;

	// Do NOT update the triggers/modifiers here to preserve their state
}

void IEnhancedInputSubsystemInterface::UpdateValueOfContinuousInputInjectionForPlayerMapping(const FName MappingName, FInputActionValue RawValue)
{
	if (const UEnhancedInputUserSettings* UserSettings = GetUserSettings())
	{
		if (const UInputAction* Action = UserSettings->FindInputActionForMapping(MappingName))
		{
			UpdateValueOfContinuousInputInjectionForAction(Action, RawValue);
		}
		else
		{
			UE_LOGF(LogEnhancedInput, Warning, "Could not find a Input Action for mapping name '%ls'", *MappingName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogEnhancedInput, Warning, "Could not find a valid UEnhancedInputUserSettings object, is it enabled in the project settings?");
	}
}

void IEnhancedInputSubsystemInterface::StopContinuousInputInjectionForAction(const UInputAction* Action)
{
	GetContinuouslyInjectedInputs().Remove(Action);
}

void IEnhancedInputSubsystemInterface::StopContinuousInputInjectionForPlayerMapping(const FName MappingName)
{
	if (const UEnhancedInputUserSettings* UserSettings = GetUserSettings())
	{
		if (const UInputAction* Action = UserSettings->FindInputActionForMapping(MappingName))
		{
			StopContinuousInputInjectionForAction(Action);
		}
		else
		{
			UE_LOGF(LogEnhancedInput, Warning, "Could not find a Input Action for mapping name '%ls'", *MappingName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogEnhancedInput, Warning, "Could not find a valid UEnhancedInputUserSettings object, is it enabled in the project settings?");
	}
}

bool IEnhancedInputSubsystemInterface::HasContinuousInputInjectionForAction(const UInputAction* Action) const
{
	return const_cast<IEnhancedInputSubsystemInterface*>(this)->GetContinuouslyInjectedInputs().Contains(Action);
}

void IEnhancedInputSubsystemInterface::InjectInputVectorForPlayerMapping(const FName MappingName, FVector Value, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers)
{
	if (const UEnhancedInputUserSettings* UserSettings = GetUserSettings())
	{
		if (const UInputAction* Action = UserSettings->FindInputActionForMapping(MappingName))
		{
			FInputActionValue RawValue(Action->ValueType, Value);
			InjectInputForAction(Action, RawValue, Modifiers, Triggers);
		}
		else
		{
			UE_LOGF(LogEnhancedInput, Warning, "Could not find a Input Action for mapping name '%ls'", *MappingName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogEnhancedInput, Warning, "Could not find a valid UEnhancedInputUserSettings object, is it enabled in the project settings?");
	}
}

void IEnhancedInputSubsystemInterface::ClearAllMappings()
{
	if (UEnhancedPlayerInput* PlayerInput = GetPlayerInput())
	{
		PlayerInput->AppliedInputContextData.Empty();
		RequestRebuildControlMappings();
	}
}

void IEnhancedInputSubsystemInterface::AddMappingContext(const UInputMappingContext* MappingContext, int32 Priority, const FModifyContextOptions& Options)
{
	// Layer mappings on top of existing mappings
	if (MappingContext)
	{
		if (UEnhancedPlayerInput* PlayerInput = GetPlayerInput())
		{
			const EMappingContextRegistrationTrackingMode TrackingMode = MappingContext->GetRegistrationTrackingMode();
			if (TrackingMode == EMappingContextRegistrationTrackingMode::Untracked)
			{
				PlayerInput->AppliedInputContextData.Add(MappingContext, { Priority });
				RequestRebuildControlMappings(Options);
			}
			else if (TrackingMode == EMappingContextRegistrationTrackingMode::CountRegistrations)
			{
				if (FAppliedInputContextData* IMCDataPtr = PlayerInput->AppliedInputContextData.Find(MappingContext))
				{
					IMCDataPtr->RegistrationCount++;
				}
				else
				{
					constexpr int32 InitialRegistrationCount = 1;
					PlayerInput->AppliedInputContextData.Add(MappingContext, { Priority, InitialRegistrationCount });
					RequestRebuildControlMappings(Options);
				}
			}
			else
			{
				checkNoEntry(); // Unhandled tracking mode
			}
		}

		if (Options.bNotifyUserSettings)
		{
			if (UEnhancedInputUserSettings* Settings = GetUserSettings())
			{
				Settings->RegisterInputMappingContext(MappingContext);
			}
		}
	}
	else
	{
		UE_LOGF(LogEnhancedInput, Warning, "Called AddMappingContext with a null Mapping Context! No changes have been applied.");
	}
}

void IEnhancedInputSubsystemInterface::RemoveMappingContext(const UInputMappingContext* MappingContext, const FModifyContextOptions& Options)
{
	if (MappingContext)
	{
		bool bDidRemoveMappingContext = false;
		if (UEnhancedPlayerInput* PlayerInput = GetPlayerInput())
		{
			const EMappingContextRegistrationTrackingMode TrackingMode = MappingContext->GetRegistrationTrackingMode();
			if (TrackingMode == EMappingContextRegistrationTrackingMode::Untracked)
			{
				PlayerInput->AppliedInputContextData.Remove(MappingContext);
				RequestRebuildControlMappings(Options);
				bDidRemoveMappingContext = true;
			}
			else if (TrackingMode == EMappingContextRegistrationTrackingMode::CountRegistrations)
			{
				if (FAppliedInputContextData* IMCDataPtr = PlayerInput->AppliedInputContextData.Find(MappingContext))
				{
					const int32 RegistrationCount = --(IMCDataPtr->RegistrationCount);
					ensureMsgf(RegistrationCount >= 0, TEXT("Input Mapping Context [%s] has a negative registration count without being removed"), *MappingContext->GetName());
					if (RegistrationCount <= 0)
					{
						PlayerInput->AppliedInputContextData.Remove(MappingContext);
						RequestRebuildControlMappings(Options);
						bDidRemoveMappingContext = true;
					}
				}
			}
			else
			{
				checkNoEntry(); // Unhandled tracking mode
			}
		}

		if (bDidRemoveMappingContext && Options.bNotifyUserSettings)
		{
			if (UEnhancedInputUserSettings* Settings = GetUserSettings())
			{
				Settings->UnregisterInputMappingContext(MappingContext);
			}
		}
	}
}

FGameplayTagContainer IEnhancedInputSubsystemInterface::GetInputMode() const
{
	UE_CLOGF(!GetDefault<UEnhancedInputDeveloperSettings>()->bEnableInputModeFiltering,		
		LogEnhancedInput, Warning, "[%s] bEnableInputModeFiltering is false in the Enhanced Input developer settings. Nothing will happen.",
		__func__);
	
	if (const UEnhancedPlayerInput* Input = GetPlayerInput())
	{
		return Input->GetCurrentInputMode();
	}

	UE_LOGF(LogEnhancedInput, Error, "[%hs] Null player input, cannot get the current input mode.", __func__);
	
	return FGameplayTagContainer {};
}

void IEnhancedInputSubsystemInterface::SetInputMode(const FGameplayTagContainer& NewMode, const FModifyContextOptions& Options)
{
	UE_CLOGF(!GetDefault<UEnhancedInputDeveloperSettings>()->bEnableInputModeFiltering,
		LogEnhancedInput, Warning, "[%s] bEnableInputModeFiltering is false in the Enhanced Input developer settings. Nothing will happen.",
		__func__);
	
	UEnhancedPlayerInput* Input = GetPlayerInput();
	if (!Input)
	{
		UE_LOGF(LogEnhancedInput, Error, "[%hs] Null player input, unable to set the input mode to '%ls'", __func__, *NewMode.ToString());
		return;
	}
	
	Input->SetCurrentInputMode(NewMode);

	RequestRebuildControlMappings(Options);
}

void IEnhancedInputSubsystemInterface::AppendTagsToInputMode(const FGameplayTagContainer& TagsToAdd, const FModifyContextOptions& Options)
{
	UE_CLOGF(!GetDefault<UEnhancedInputDeveloperSettings>()->bEnableInputModeFiltering,
		LogEnhancedInput, Warning, "[%s] bEnableInputModeFiltering is false in the Enhanced Input developer settings. Nothing will happen.",
		__func__);
	
	UEnhancedPlayerInput* Input = GetPlayerInput();
	if (!Input)
	{
		UE_LOGF(LogEnhancedInput, Error, "[%hs] Null player input, unable to append tags '%ls' to the input mode", __func__, *TagsToAdd.ToString());
		return;
	}
	
	Input->GetCurrentInputMode().AppendTags(TagsToAdd);
	
	RequestRebuildControlMappings(Options);
}

void IEnhancedInputSubsystemInterface::AddTagToInputMode(const FGameplayTag& TagToAdd, const FModifyContextOptions& Options)
{
	AppendTagsToInputMode(FGameplayTagContainer { TagToAdd }, Options);
}

void IEnhancedInputSubsystemInterface::RemoveTagsFromInputMode(const FGameplayTagContainer& TagsToRemove, const FModifyContextOptions& Options)
{
	UE_CLOGF(!GetDefault<UEnhancedInputDeveloperSettings>()->bEnableInputModeFiltering,
		LogEnhancedInput, Warning, "[%s] bEnableInputModeFiltering is false in the Enhanced Input developer settings. Nothing will happen.",
		__func__);
	
	UEnhancedPlayerInput* Input = GetPlayerInput();
	if (!Input)
	{
		UE_LOGF(LogEnhancedInput, Error, "[%hs] Null player input, unable to remove tags '%ls' from the input mode", __func__, *TagsToRemove.ToString());
		return;
	}

	Input->GetCurrentInputMode().RemoveTags(TagsToRemove);
	
	RequestRebuildControlMappings(Options);
}

void IEnhancedInputSubsystemInterface::RemoveTagFromInputMode(const FGameplayTag& TagToRemove, const FModifyContextOptions& Options)
{
	RemoveTagsFromInputMode(FGameplayTagContainer { TagToRemove }, Options);
}

void IEnhancedInputSubsystemInterface::RequestRebuildControlMappings(const FModifyContextOptions& Options, EInputMappingRebuildType MappingRebuildType)
{
	bMappingRebuildPending = true;
	bIgnoreAllPressedKeysUntilReleaseOnRebuild &= Options.bIgnoreAllPressedKeysUntilRelease;
	MappingRebuildPending = MappingRebuildType;
	
	if (Options.bForceImmediately)
	{
		RebuildControlMappings();
	}
}

EMappingQueryResult IEnhancedInputSubsystemInterface::QueryMapKeyInActiveContextSet(const UInputMappingContext* InputContext, const UInputAction* Action, FKey Key, TArray<FMappingQueryIssue>& OutIssues, EMappingQueryIssue BlockingIssues/* = DefaultMappingIssues::StandardFatal*/)
{
	UEnhancedPlayerInput* PlayerInput = GetPlayerInput();
	if (!PlayerInput)
	{
		return EMappingQueryResult::Error_EnhancedInputNotEnabled;
	}

	// TODO: Inefficient, but somewhat forgivable as the mapping context count is likely to be single figure.
	TMap<TObjectPtr<const UInputMappingContext>, FAppliedInputContextData> OrderedInputContexts = PlayerInput->AppliedInputContextData;
	OrderedInputContexts.ValueSort([](const FAppliedInputContextData& A, const FAppliedInputContextData& B) { return A.Priority > B.Priority; });

	TArray<UInputMappingContext*> Applied;
	Applied.Reserve(OrderedInputContexts.Num());
	for (const TPair<TObjectPtr<const UInputMappingContext>, FAppliedInputContextData>& ContextPair : OrderedInputContexts)
	{
		Applied.Add(const_cast<UInputMappingContext*>(ToRawPtr(ContextPair.Key)));
	}

	return QueryMapKeyInContextSet(Applied, InputContext, Action, Key, OutIssues, BlockingIssues);
}

EMappingQueryResult IEnhancedInputSubsystemInterface::QueryMapKeyInContextSet(const TArray<UInputMappingContext*>& PrioritizedActiveContexts, const UInputMappingContext* InputContext, const UInputAction* Action, FKey Key, TArray<FMappingQueryIssue>& OutIssues, EMappingQueryIssue BlockingIssues/* = DefaultMappingIssues::StandardFatal*/)
{
	if (!Action)
	{
		return EMappingQueryResult::Error_InvalidAction;
	}

	OutIssues.Reset();

	// Report on keys being bound that don't support the action's value type.
	EInputActionValueType KeyValueType = FInputActionValue(Key).GetValueType();
	if (Action->ValueType != KeyValueType)
	{
		// We exclude bool -> Axis1D promotions, as these are commonly used for paired mappings (e.g. W + S/Negate bound to a MoveForward action), and are fairly intuitive anyway.
		if (Action->ValueType != EInputActionValueType::Axis1D || KeyValueType != EInputActionValueType::Boolean)
		{
			OutIssues.Add(KeyValueType < Action->ValueType ? EMappingQueryIssue::ForcesTypePromotion : EMappingQueryIssue::ForcesTypeDemotion);
		}
	}

	enum class EStage : uint8
	{
		Pre,
		Main,
		Post,
	};
	EStage Stage = EStage::Pre;

	EMappingQueryResult Result = EMappingQueryResult::MappingAvailable;

	UEnhancedInputUserSettings* CurrentUserSettings = GetUserSettings();
	UEnhancedPlayerMappableKeyProfile* PlayerKeyProfile = CurrentUserSettings ? CurrentUserSettings->GetActiveKeyProfile() : nullptr;
	const FString& CurrentProfileId = PlayerKeyProfile ? PlayerKeyProfile->GetProfileIdString() : TEXT("");

	// These will be ordered by priority
	for (const UInputMappingContext* BlockingContext : PrioritizedActiveContexts)
	{
		if (!BlockingContext)
		{
			continue;
		}

		// Update stage
		if (Stage == EStage::Main)
		{
			Stage = EStage::Post;
		}
		else if (BlockingContext == InputContext)
		{
			Stage = EStage::Main;
		}

		for (const FEnhancedActionKeyMapping& Mapping : BlockingContext->GetMappingsForProfile(CurrentProfileId))
		{
			if (Mapping.Key == Key && Mapping.Action)
			{
				FMappingQueryIssue Issue;
				// Block mappings that would have an unintended effect with an existing mapping
				// TODO: This needs to apply chording input consumption rules
				if (Stage == EStage::Pre && Mapping.Action->bConsumeInput)
				{
					Issue.Issue = EMappingQueryIssue::HiddenByExistingMapping;
				}
				else if (Stage == EStage::Post && Action->bConsumeInput)
				{
					Issue.Issue = EMappingQueryIssue::HidesExistingMapping;
				}
				else if (Stage == EStage::Main)
				{
					Issue.Issue = EMappingQueryIssue::CollisionWithMappingInSameContext;
				}

				// Block mapping over any action that refuses it.
				if (Mapping.Action->bReserveAllMappings)
				{
					Issue.Issue = EMappingQueryIssue::ReservedByAction;
				}

				if (Issue.Issue != EMappingQueryIssue::NoIssue)
				{
					Issue.BlockingContext = BlockingContext;
					Issue.BlockingAction = Mapping.Action;
					OutIssues.Add(Issue);

					if ((Issue.Issue & BlockingIssues) != EMappingQueryIssue::NoIssue)
					{
						Result = EMappingQueryResult::NotMappable;
					}
				}
			}
		}
	}

	// Context must be part of the tested collection. If we didn't find it raise an error.
	if (Stage < EStage::Main)
	{
		return EMappingQueryResult::Error_InputContextNotInActiveContexts;
	}

	return Result;

}

bool IEnhancedInputSubsystemInterface::HasTriggerWith(TFunctionRef<bool(const UInputTrigger*)> TestFn, const TArray<UInputTrigger*>& Triggers)
{
	for (const UInputTrigger* Trigger : Triggers)
	{
		if (TestFn(Trigger))
		{
			return true;
		}
	}
	return false;
};

void IEnhancedInputSubsystemInterface::InjectChordBlockers(const TArray<int32>& ChordedMappings)
{
	UEnhancedPlayerInput* PlayerInput = GetPlayerInput();
	if (!PlayerInput)
	{
		return;
	}

	// Inject chord blockers into all lower priority action mappings with a shared key
	for (int32 MappingIndex : ChordedMappings)
	{
		FEnhancedActionKeyMapping& ChordMapping = PlayerInput->EnhancedActionMappings[MappingIndex];
		for (int32 i = MappingIndex + 1; i < PlayerInput->EnhancedActionMappings.Num(); ++i)
		{
			FEnhancedActionKeyMapping& Mapping = PlayerInput->EnhancedActionMappings[i];
			if (Mapping.Action && Mapping.Key == ChordMapping.Key)
			{
				// If we have no explicit triggers we can't inject an implicit as it may cause us to fire when we shouldn't.
				auto AnyExplicit = [](const UInputTrigger* Trigger) { return Trigger->GetTriggerType() == ETriggerType::Explicit; };
				if (!HasTriggerWith(AnyExplicit, Mapping.Triggers) && !HasTriggerWith(AnyExplicit, Mapping.Action->Triggers))
				{
					// Insert a down trigger to ensure we have valid rules for triggering when the chord blocker is active.
					Mapping.Triggers.Add(NewObject<UInputTriggerDown>());
					Mapping.Triggers.Last()->ActuationThreshold = SMALL_NUMBER;	// TODO: "No trigger" actuates on any non-zero value but Down has a threshold so this is a hack to reproduce no trigger behavior!
				}

				UInputTriggerChordBlocker* ChordBlocker = NewObject<UInputTriggerChordBlocker>(PlayerInput);
				ChordBlocker->ChordAction = ChordMapping.Action;
				// TODO: If the chording action is bound at a lower priority than the blocked action its trigger state will be evaluated too late, which may produce unintended effects on the first tick.
				Mapping.Triggers.Add(ChordBlocker);
			}
		}
	}
}

void IEnhancedInputSubsystemInterface::ApplyAxisPropertyModifiers(UEnhancedPlayerInput* PlayerInput, FEnhancedActionKeyMapping& Mapping) const
{
	// Axis properties are treated as per-key default modifier layouts.

	// TODO: Make this optional? Opt in or out? Per modifier type?
	//if (!EnhancedInputSettings.bApplyAxisPropertiesAsModifiers)
	//{
	//	return;
	//}

	if (GGlobalAxisConfigMode_None == GGlobalAxisConfigMode)
	{
		return;
	}

	// TODO: This function is causing issues with gamepads, applying a hidden 0.25 deadzone modifier by default. Apply it to mouse inputs only until a better system is in place.
	if (GGlobalAxisConfigMode_All != GGlobalAxisConfigMode &&
		!Mapping.Key.IsMouseButton())
	{
		return;
	}

	// Apply applicable axis property modifiers from the old input settings automatically.
	// TODO: This needs to live at the EnhancedInputSettings level.
	// TODO: Adopt this approach for all modifiers? Most of these are better done at the action level for most use cases.
	FInputAxisProperties AxisProperties;
	if (PlayerInput->GetAxisProperties(Mapping.Key, AxisProperties))
	{
		TArray<TObjectPtr<UInputModifier>> Modifiers;

		// If a modifier already exists it should override axis properties.
		auto HasExistingModifier = [&Mapping](UClass* OfType)
		{
			auto TypeMatcher = [&OfType](UInputModifier* Modifier) { return Modifier != nullptr && Modifier->IsA(OfType); };
			return Mapping.Modifiers.ContainsByPredicate(TypeMatcher) || Mapping.Action->Modifiers.ContainsByPredicate(TypeMatcher);
		};

		// Maintain old input system modification order.

		if (AxisProperties.DeadZone > 0.f &&
			!HasExistingModifier(UInputModifierDeadZone::StaticClass()))
		{
			UInputModifierDeadZone* DeadZone = NewObject<UInputModifierDeadZone>();
			DeadZone->LowerThreshold = AxisProperties.DeadZone;
			DeadZone->Type = EDeadZoneType::Axial;
			Modifiers.Add(DeadZone);
		}

		if (AxisProperties.Exponent != 1.f &&
			!HasExistingModifier(UInputModifierResponseCurveExponential::StaticClass()))
		{
			UInputModifierResponseCurveExponential* Exponent = NewObject<UInputModifierResponseCurveExponential>();
			Exponent->CurveExponent = FVector::OneVector * AxisProperties.Exponent;
			Modifiers.Add(Exponent);
		}

		// Sensitivity stacks with user defined.
		// TODO: Unexpected behavior but makes sense for most use cases. E.g. Mouse sensitivity, which is scaled by 0.07 in BaseInput.ini, would be broken by adding a Look action sensitivity.
		if (AxisProperties.Sensitivity != 1.f /* &&
			!HasExistingModifier(UInputModifierScalar::StaticClass())*/)
		{
			UInputModifierScalar* Sensitivity = NewObject<UInputModifierScalar>();
			Sensitivity->Scalar = FVector::OneVector * AxisProperties.Sensitivity;
			Modifiers.Add(Sensitivity);
		}

		if (AxisProperties.bInvert &&
			!HasExistingModifier(UInputModifierNegate::StaticClass()))
		{
			Modifiers.Add(NewObject<UInputModifierNegate>());
		}

		// Add to front of modifier list (these modifiers should be executed before any user defined modifiers)
		Swap(Mapping.Modifiers, Modifiers);
		Mapping.Modifiers.Append(Modifiers);
	}
}

bool IEnhancedInputSubsystemInterface::HasMappingContext(const UInputMappingContext* MappingContext) const
{
	int32 DummyPri = INDEX_NONE;
	return HasMappingContext(MappingContext, DummyPri);
}

bool IEnhancedInputSubsystemInterface::HasMappingContext(const UInputMappingContext* MappingContext, int32& OutFoundPriority) const
{
	bool bResult = false;
	OutFoundPriority = INDEX_NONE;
	
	if (const UEnhancedPlayerInput* const Input = GetPlayerInput())
	{
		if (const FAppliedInputContextData* const FoundInputContextData = Input->AppliedInputContextData.Find(MappingContext))
		{
			OutFoundPriority = FoundInputContextData->Priority;
			bResult = true;
		}
	}

	return bResult;
}

TArray<FKey> IEnhancedInputSubsystemInterface::QueryKeysMappedToAction(const UInputAction* Action) const
{
	TArray<FKey> MappedKeys;

	if (Action)
	{
		if (const UEnhancedPlayerInput* const PlayerInput = GetPlayerInput())
		{
			for (const FEnhancedActionKeyMapping& Mapping : PlayerInput->EnhancedActionMappings)
			{
				if (Mapping.Action == Action)
				{
					MappedKeys.AddUnique(Mapping.Key.GetVirtualKey());
				}
			}
		}
	}

	return MappedKeys;
}

TArray<FEnhancedActionKeyMapping> IEnhancedInputSubsystemInterface::GetAllPlayerMappableActionKeyMappings() const
{
	TArray<FEnhancedActionKeyMapping> PlayerMappableMappings;
	
	if (const UEnhancedPlayerInput* const PlayerInput = GetPlayerInput())
    {
    	for (const FEnhancedActionKeyMapping& Mapping : PlayerInput->EnhancedActionMappings)
		{
			if (Mapping.IsPlayerMappable())
			{
				PlayerMappableMappings.AddUnique(Mapping);
			}
		}
    }
	
	return PlayerMappableMappings;
}

void IEnhancedInputSubsystemInterface::ValidateTrackedMappingContextsAreUnregistered() const
{
	if (UEnhancedPlayerInput* PlayerInput = GetPlayerInput())
	{
		for (const TPair<TObjectPtr<const UInputMappingContext>, FAppliedInputContextData>& Pair : PlayerInput->GetAppliedInputContextData())
		{
			if (IsValid(Pair.Key) && Pair.Key->GetRegistrationTrackingMode() != EMappingContextRegistrationTrackingMode::Untracked)
			{
				UE_LOGF(LogEnhancedInput, Warning, "Input Mapping Context [%ls] has tracking mode [%ls] but is still applied and might be leaking, unregister it before deinitialization.",
					*Pair.Key->GetName(), *UEnum::GetValueAsString(Pair.Key->GetRegistrationTrackingMode()));
			}
		}
	}
}

// TODO: This should be a delegate (along with InjectChordBlockers), moving chording out of the underlying subsystem and enabling implementation of custom mapping handlers.
/**
 * Reorder the given UnordedMappings such that chording mappings > chorded mappings > everything else.
 * This is used to ensure mappings within a single context are evaluated in the correct order to support chording.
 * Populate the DependentChordActions array with any chorded triggers so that we can detect which ones should be triggered
 * later. 
 */
TArray<FEnhancedActionKeyMapping> IEnhancedInputSubsystemInterface::ReorderMappings(const TArray<FEnhancedActionKeyMapping>& UnorderedMappings, TArray<UEnhancedPlayerInput::FDependentChordTracker>& OUT DependentChordActions)
{
	TSet<const UInputAction*> ChordingActions;

	struct FTriggerEvaluationResults
	{
		bool bFoundChordTrigger = false;
		bool bFoundAlwaysTickTrigger = false;
	};
	
	// Gather all chording actions within a mapping's triggers.
	auto GatherChordingActions = [&ChordingActions, &DependentChordActions](const FEnhancedActionKeyMapping& Mapping) -> FTriggerEvaluationResults
	{
		FTriggerEvaluationResults Res = {};
		auto EvaluateTriggers = [&Mapping, &ChordingActions, &DependentChordActions, &Res](const TArray<UInputTrigger*>& Triggers)-> FTriggerEvaluationResults
		{
			for (const UInputTrigger* Trigger : Triggers)
			{
				if (!Trigger)
				{
					UE_LOGF(LogEnhancedInput, Error, "Null input trigger detected in mapping to input action '%ls'", *GetNameSafe(Mapping.Action));
					continue;
				}
				
				if (const UInputTriggerChordAction* ChordTrigger = Cast<const UInputTriggerChordAction>(Trigger))
				{
					ChordingActions.Add(ChordTrigger->ChordAction);

					// Keep track of the action itself, and the action it is dependant on
					DependentChordActions.Emplace(UEnhancedPlayerInput::FDependentChordTracker { Mapping.Action, ChordTrigger->ChordAction });
					
					Res.bFoundChordTrigger = true;
				}

				// Keep track of if this trigger is marked as being "always tick".
				// This is not a great thing to do but some custom triggers may require always being ticked, so allow it as an option
				Res.bFoundAlwaysTickTrigger |= Trigger->bShouldAlwaysTick;
			}
			return Res;
		};
		
		const FTriggerEvaluationResults MappingResults = EvaluateTriggers(Mapping.Triggers);

		ensureMsgf(Mapping.Action, TEXT("A key mapping has no associated action!"));
		const FTriggerEvaluationResults ActionResults = EvaluateTriggers(Mapping.Action->Triggers);

		// returned the combined results of each individual keymapping and it's associated input action.
		return FTriggerEvaluationResults
		{
			.bFoundChordTrigger			= (MappingResults.bFoundChordTrigger || ActionResults.bFoundChordTrigger),
			.bFoundAlwaysTickTrigger	= (MappingResults.bFoundAlwaysTickTrigger || ActionResults.bFoundAlwaysTickTrigger)
		};
	};

	// Split chorded mappings (second priority) from all others whilst building a list of chording actions to use for further prioritization.
	TArray<FEnhancedActionKeyMapping> ChordedMappings;
	TArray<FEnhancedActionKeyMapping> OtherMappings;
	OtherMappings.Reserve(UnorderedMappings.Num());		// Mappings will most likely be Other
	int32 NumEmptyMappings = 0;
	for (const FEnhancedActionKeyMapping& Mapping : UnorderedMappings)
	{
		if (Mapping.Action)
		{
			// Evaluate the triggers on each key mapping to check for chords and also "always tick" input triggers.
			const FTriggerEvaluationResults TriggerEvalResults = GatherChordingActions(Mapping);

			// Determine which array this mapping should be in based on if it has a chord or not
			TArray<FEnhancedActionKeyMapping>& MappingArray = TriggerEvalResults.bFoundChordTrigger ? ChordedMappings : OtherMappings;

			// flag this new mapping as being always tick as necessary
			FEnhancedActionKeyMapping& NewlyAddedMapping = MappingArray.Add_GetRef(Mapping);
			NewlyAddedMapping.bHasAlwaysTickTrigger = TriggerEvalResults.bFoundAlwaysTickTrigger;
		}
		else
		{
			++NumEmptyMappings;
			UE_LOGF(LogEnhancedInput, Warning, "A Key Mapping with a blank action has been added! Ignoring the key mapping to '%ls'", *Mapping.Key.ToString());
		}
	}

	TArray<FEnhancedActionKeyMapping> OrderedMappings;
	OrderedMappings.Reserve(UnorderedMappings.Num());

	// Move chording mappings to the front as they need to be evaluated before chord and blocker triggers
	// TODO: Further ordering of chording mappings may be required should one of them be chorded against another
	auto ExtractChords = [&OrderedMappings, &ChordingActions](TArray<FEnhancedActionKeyMapping>& Mappings)
	{
		for (int32 i = 0; i < Mappings.Num();)
		{
			if (ChordingActions.Contains(Mappings[i].Action))
			{
				OrderedMappings.Add(Mappings[i]);
				Mappings.RemoveAtSwap(i);	// TODO: Do we care about reordering underlying mappings?
			}
			else
			{
				++i;
			}
		}
	};
	ExtractChords(ChordedMappings);
	ExtractChords(OtherMappings);

	OrderedMappings.Append(MoveTemp(ChordedMappings));
	OrderedMappings.Append(MoveTemp(OtherMappings));
	checkf(OrderedMappings.Num() == UnorderedMappings.Num() - NumEmptyMappings, TEXT("Number of mappings unexpectedly changed during reorder."));

	return OrderedMappings;
}

void IEnhancedInputSubsystemInterface::RebuildControlMappings()
{
	if(MappingRebuildPending == EInputMappingRebuildType::None)
	{
		return;
	}

	UEnhancedPlayerInput* PlayerInput = GetPlayerInput();
	if (!PlayerInput)
	{
		// TODO: Prefer to reset MappingRebuildPending here?
		return;
	}
	
	const FGameplayTagContainer& CurrentInputMode = PlayerInput->GetCurrentInputMode();
	const bool bInputModeFilteringEnabled = GetDefault<UEnhancedInputDeveloperSettings>()->bEnableInputModeFiltering;
	
	// Clear existing mappings, but retain the mapping array for later processing
	TArray<FEnhancedActionKeyMapping> OldMappings(MoveTemp(PlayerInput->EnhancedActionMappings));
	PlayerInput->ClearAllMappings();
	PlayerInput->KeyConsumptionData.Reset();
	AppliedContextRedirects.Reset();

	// Order contexts by priority
	TMap<TObjectPtr<const UInputMappingContext>, FAppliedInputContextData> OrderedInputContexts = PlayerInput->AppliedInputContextData;
	
	// Replace any mapping contexts that may have a redirect on this platform
	if (UEnhancedInputPlatformSettings* PlatformSettings = UEnhancedInputPlatformSettings::Get())
	{
		TMap<TObjectPtr<const UInputMappingContext>, TObjectPtr<const UInputMappingContext>> ContextRedirects;
		PlatformSettings->GetAllMappingContextRedirects(ContextRedirects);
		for (const TPair<TObjectPtr<const UInputMappingContext>, TObjectPtr<const UInputMappingContext>>& Pair : ContextRedirects)
		{
			if (!Pair.Key || !Pair.Value)
			{
				UE_LOGF(LogEnhancedInput, Error, "An invalid Mapping Context Redirect specified in '%ls'", PlatformSettings->GetConfigOverridePlatform());
				continue;
			}
			
			// Replace the existing IMC with the one that it should be redirected to on the PlayerInput 
			if (const FAppliedInputContextData* ExistingIMCData = OrderedInputContexts.Find(Pair.Key))
			{
				OrderedInputContexts.Remove(Pair.Key);
				OrderedInputContexts.Add(Pair.Value, *ExistingIMCData);
				AppliedContextRedirects.Add(Pair);

				// Optional logging that may be helpful for debugging purposes
				if (PlatformSettings->ShouldLogMappingContextRedirects())
				{
					UE_LOGF(
						LogEnhancedInput,
						Log,
						"'%ls' Redirecting Mapping Context '%ls' -> '%ls'",
						PlatformSettings->GetConfigOverridePlatform(), *Pair.Key->GetName(), *Pair.Value->GetName()
					);
				}
			}
		}
	}
	
	// Order contexts by priority
	OrderedInputContexts.ValueSort([](const FAppliedInputContextData& A, const FAppliedInputContextData& B) { return A.Priority > B.Priority; });

	TSet<FKey> AppliedKeys;

	TArray<int32> ChordedMappings;

	// Reset the tracking of dependant chord actions on the player input
	PlayerInput->DependentChordActions.Reset();

	UEnhancedInputUserSettings* CurrentUserSettings = GetUserSettings();
	UEnhancedPlayerMappableKeyProfile* PlayerKeyProfile = CurrentUserSettings ? CurrentUserSettings->GetActiveKeyProfile() : nullptr;

	// An array of keys that are mapped to a given Action.
	// This is populated by any player mapped keys if they exist, or the default mapping from
	// an input mapping context.
	TArray<FKey> MappedKeysToActionName;
	
	for (const TPair<TObjectPtr<const UInputMappingContext>, FAppliedInputContextData>& ContextPair : OrderedInputContexts)
	{
		// Don't apply context specific keys immediately, allowing multiple mappings to the same key within the same context if required.
		TArray<FKey> ContextAppliedKeys;
		const UInputMappingContext* MappingContext = ContextPair.Key;

		if (!ensure(IsValid(MappingContext)))
		{
			UE_LOGF(LogEnhancedInput, Error, "[%hs] Encountered an invalid mapping context while rebuilding control mappings. Some inputs may be ignored!", __func__);
			continue;
		}
		
		// Check if this mapping context can be applied in our current input mode on the player
		// If it can't, then we will not process its mappings..
		if (bInputModeFilteringEnabled &&
			MappingContext->ShouldFilterMappingByInputMode() &&
			!MappingContext->GetInputModeQuery().Matches(CurrentInputMode))
		{
			UE_LOGF(LogEnhancedInput, Verbose, "[%hs] Not applying mappings from IMC '%ls' because it does not meet the requirements of the current input mode '%ls'",
				__func__, *GetNameSafe(MappingContext), *CurrentInputMode.ToString());
				
			continue;
		}
		
		TArray<FEnhancedActionKeyMapping> OrderedMappings = ReorderMappings(MappingContext->GetMappingsForProfile(PlayerKeyProfile ? PlayerKeyProfile->GetProfileIdString() : TEXT("")), PlayerInput->DependentChordActions);

		for (FEnhancedActionKeyMapping& Mapping : OrderedMappings)
		{
			// Clear out mappings from the previous iteration
			MappedKeysToActionName.Reset();
			
			const UPlayerMappableKeySettings* KeySettings = Mapping.GetPlayerMappableKeySettings();

			// If this mapping has specified a specific key profile, and the current profile isn't it, then don't add this key mapping
			if (KeySettings && PlayerKeyProfile && !KeySettings->SupportedKeyProfileIds.IsEmpty() && !KeySettings->SupportedKeyProfileIds.Contains(PlayerKeyProfile->GetProfileIdString()))
			{
				continue;			
			}
			
			// See if there are any player mapped keys to this action
			if (PlayerKeyProfile && GetDefault<UEnhancedInputDeveloperSettings>()->bEnableUserSettings)
			{
				PlayerKeyProfile->GetPlayerMappedKeysForRebuildControlMappings(Mapping, MappedKeysToActionName);
			}

			// True if there were any player mapped keys to this mapping and we are using those instead.
			const bool bIsPlayerMapping = !MappedKeysToActionName.IsEmpty();

			// If there aren't, then just use the default mapping for this action
			if (!bIsPlayerMapping)
			{
				MappedKeysToActionName.Add(Mapping.Key);
			}
			
			for (const FKey& PlayerMappedKey : MappedKeysToActionName)
			{
				Mapping.Key = PlayerMappedKey;

				if (bTrackKeysForAllActionMappings)
				{
					// Track all key mappings for each action so that we can check if they've already been consumed before executing their action.
					FKeyConsumptionOptions& Opts = PlayerInput->KeyConsumptionData.FindOrAdd(Mapping.Action);
					Opts.KeysToConsume.AddUnique(Mapping.Key);
					if (Mapping.Action->bConsumesActionAndAxisMappings)
					{
						Opts.EventsToCauseConsumption |= static_cast<ETriggerEvent>(Mapping.Action->TriggerEventsThatConsumeLegacyKeys);
					}
				}
				else if (Mapping.Action->bConsumesActionAndAxisMappings)
				{
					// If this Input Action is flagged to consume input, then mark it's key state as being consumed every tick.
					// This has the effect where the base UPlayerInput class will not fire any legacy bindings
					FKeyConsumptionOptions& Opts = PlayerInput->KeyConsumptionData.FindOrAdd(Mapping.Action);
					Opts.KeysToConsume.AddUnique(Mapping.Key);
					Opts.EventsToCauseConsumption |= static_cast<ETriggerEvent>(Mapping.Action->TriggerEventsThatConsumeLegacyKeys);
				}
				
				if (Mapping.Action && !AppliedKeys.Contains(Mapping.Key))
				{
					// TODO: Wasteful query as we've already established chord state within ReorderMappings. Store TOptional bConsumeInput per mapping, allowing override? Query override via delegate?
					auto IsChord = [](const UInputTrigger* Trigger)
					{
						return Cast<const UInputTriggerChordAction>(Trigger) != nullptr;
					};
					bool bHasActionChords = HasTriggerWith(IsChord, Mapping.Action->Triggers);
					bool bHasChords = bHasActionChords || HasTriggerWith(IsChord, Mapping.Triggers);

					// Chorded actions can't consume input or they would hide the action they are chording.
					if (!bHasChords && Mapping.Action->bConsumeInput)
					{
						ContextAppliedKeys.Add(Mapping.Key);
					}

					int32 NewMappingIndex = PlayerInput->AddMapping(Mapping);
					FEnhancedActionKeyMapping& NewMapping = PlayerInput->EnhancedActionMappings[NewMappingIndex];

					// Re-instance modifiers
					DeepCopyPtrArray<UInputModifier>(Mapping.Modifiers, MutableView(NewMapping.Modifiers));

					ApplyAxisPropertyModifiers(PlayerInput, NewMapping);

					// Re-instance triggers
					DeepCopyPtrArray<UInputTrigger>(Mapping.Triggers, MutableView(NewMapping.Triggers));

					if (bHasChords)
					{
						// TODO: Re-prioritize chorded mappings (within same context only?) by number of chorded actions, so Ctrl + Alt + [key] > Ctrl + [key] > [key].
						// TODO: Above example shouldn't block [key] if only Alt is down, as there is no direct Alt + [key] mapping.y
						ChordedMappings.Add(NewMappingIndex);

						// Action level chording triggers need to be evaluated at the mapping level to ensure they block early enough.
						// TODO: Continuing to evaluate these at the action level is redundant.
						if (bHasActionChords)
						{
							for (const UInputTrigger* Trigger : Mapping.Action->Triggers)
							{
								if (IsChord(Trigger))
								{
									NewMapping.Triggers.Add(DuplicateObject(Trigger, nullptr));
								}
							}
						}
					}
				}
			}
		}

		AppliedKeys.Append(MoveTemp(ContextAppliedKeys));
	}

	InjectChordBlockers(ChordedMappings);

	PlayerInput->ForceRebuildingKeyMaps(false);

	// Clean out invalidated actions
	if (MappingRebuildPending == EInputMappingRebuildType::RebuildWithFlush)
	{
		PlayerInput->ActionInstanceData.Empty();
	}
	else
	{
		
		// Remove action instance data for actions that are not referenced in the new action mappings
		TSet<const UInputAction*> RemovedActions;
		for (TPair<TObjectPtr<const UInputAction>, FInputActionInstance>& ActionInstance : PlayerInput->ActionInstanceData)
		{
			RemovedActions.Add(ActionInstance.Key.Get());
		}

		for (FEnhancedActionKeyMapping& Mapping : PlayerInput->EnhancedActionMappings)
		{
			RemovedActions.Remove(Mapping.Action);

			// Retain old mapping trigger/modifier state for identical key -> action mappings.
			TArray<FEnhancedActionKeyMapping>::SizeType Idx = OldMappings.IndexOfByPredicate(
				[&Mapping](const FEnhancedActionKeyMapping& Other)
				{
					// Use Equals() to ignore Triggers' values. We want to keep their values from before remapping to
					// prevent resets. Otherwise, triggers like UInputTriggerPressed re-trigger when their value is
					// reset to 0; and time counting triggers, like UInputTriggerHold, restart their time.
					// But don't ignore Modifier and Trigger types and their order in the comparison. If we did, we'd
					// replace new mappings for old ones with different Trigger and Modifier settings.
					return Mapping.Equals(Other);
				});
			
			if (Idx != INDEX_NONE)
			{
				// REBUILT MAPPING: This mapping existed before the rebuild with the same
				// Action, Key, and trigger/modifier types.

				// Fix for UE-270589: Use triggers from the new (potentially higher-priority) IMC,
				// but transfer runtime state from the old triggers to prevent false transitions.
				// Equals() ensures triggers match in class and order, so index pairing is safe.
				if (bRespectIMCPriorityForTriggers)
				{
					// Transfer runtime state (LastValue, HeldDuration, etc.) from old triggers
					// to new ones via OnTriggerReinstanced before replacement. This prevents
					// Pressed triggers from re-firing (LastValue preserved as "already actuated"),
					// preserves timed trigger progress, and avoids needing bShouldBeIgnored which
					// would break continuous triggers like Down. (UE-372464)
					const int32 NumTriggers = FMath::Min(Mapping.Triggers.Num(), OldMappings[Idx].Triggers.Num());
					for (int32 TriggerIdx = 0; TriggerIdx < NumTriggers; ++TriggerIdx)
					{
						if (Mapping.Triggers[TriggerIdx] && OldMappings[Idx].Triggers[TriggerIdx])
						{
							Mapping.Triggers[TriggerIdx]->OnTriggerReinstanced(OldMappings[Idx].Triggers[TriggerIdx]);
						}
					}

					// Transfer runtime state from old modifiers to new ones (e.g. smoothing accumulators).
					const int32 NumModifiers = FMath::Min(Mapping.Modifiers.Num(), OldMappings[Idx].Modifiers.Num());
					for (int32 ModifierIdx = 0; ModifierIdx < NumModifiers; ++ModifierIdx)
					{
						if (Mapping.Modifiers[ModifierIdx] && OldMappings[Idx].Modifiers[ModifierIdx])
						{
							Mapping.Modifiers[ModifierIdx]->OnModifierReinstanced(OldMappings[Idx].Modifiers[ModifierIdx]);
						}
					}

					OldMappings[Idx].Triggers = MoveTemp(Mapping.Triggers);
					OldMappings[Idx].Modifiers = MoveTemp(Mapping.Modifiers);
				}

				Mapping = MoveTemp(OldMappings[Idx]);
				OldMappings.RemoveAtSwap(Idx);
			}
			else
			{
				// This is a NEW key mapping that did not exist on the player before. No matching old mapping exists.
				// If the key is already held, mark it to be ignored until release to prevent
				// false trigger events on the first evaluation. This only applies to new
				// mappings. Rebuilt mappings preserve trigger state in the scope above
				if (bIgnoreAllPressedKeysUntilReleaseOnRebuild && Mapping.Action->ValueType == EInputActionValueType::Boolean)
				{
					const FKeyState* KeyState = PlayerInput->GetKeyState(Mapping.Key);
					if (KeyState && KeyState->bDown)
					{
						Mapping.bShouldBeIgnored = true;
					}
				}
			}
		}

		// Actions that are no longer mapped to a key may have been "In progress" by the player
		// Notify the player input object so that it can reconcile this state and call the "Canceled" event
		// on the next evaluation of the input.
		PlayerInput->NotifyInputActionsUnmapped(RemovedActions);
	}

	// Perform a modifier calculation pass on the default data to initialize values correctly.
	// We do this at the end to ensure ActionInstanceData is accessible without requiring a tick for new/flushed actions.
	for (FEnhancedActionKeyMapping& Mapping : PlayerInput->EnhancedActionMappings)
	{
		PlayerInput->InitializeMappingActionModifiers(Mapping);
	}
	
	MappingRebuildPending = EInputMappingRebuildType::None;
	bIgnoreAllPressedKeysUntilReleaseOnRebuild = true;
	bControlMappingsRebuiltThisTick = true;
}

template<typename T>
void InjectKey(T* InjectVia, FKey Key, const FInputActionValue& Value, float DeltaTime, const FPlatformUserId OwningUser, const EInputEvent Event = IE_Pressed)
{
	const FInputDeviceId DeviceToSimulate = IPlatformInputDeviceMapper::Get().GetPrimaryInputDeviceForUser(OwningUser);
	
	auto SimulateKeyPress = [InjectVia, DeltaTime, DeviceToSimulate, Event](const FKey& KeyToSim, const float Value)
	{
		FInputKeyEventArgs Args = FInputKeyEventArgs::CreateSimulated(
			KeyToSim,
			Event,
			Value,
			KeyToSim.IsAnalog() ? 1 : 0,
			DeviceToSimulate);

		Args.DeltaTime = DeltaTime;
		
		InjectVia->InputKey(Args);
	};

	if (const EKeys::FPairedKeyDetails* PairDetails = EKeys::GetPairedKeyDetails(Key))
	{
		// For paired axis keys, send a key press for each 
		const FVector ValueVector = Value.Get<FVector>();
		SimulateKeyPress(PairDetails->XKeyDetails->GetKey(), static_cast<float>(ValueVector.X));
		SimulateKeyPress(PairDetails->YKeyDetails->GetKey(), static_cast<float>(ValueVector.Y));
	}
	else
	{
		// TODO: IE_Repeat support. Ideally ticking at whatever rate the application platform is sending repeat key messages.
		SimulateKeyPress(Key, Value.Get<float>());
	}
}

void IEnhancedInputSubsystemInterface::TickForcedInput(float DeltaTime)
{
	UEnhancedPlayerInput* PlayerInput = GetPlayerInput();
	if (!PlayerInput)
	{
		return;
	}

	// Any continuous input injection needs to be added each frame until its stopped
	TMap<TObjectPtr<const UInputAction>, FInjectedInput>& ContinuouslyInjectedInputs = GetContinuouslyInjectedInputs();
	for (TPair<TObjectPtr<const UInputAction>, FInjectedInput>& ContinuousInjection : ContinuouslyInjectedInputs)
	{
		TObjectPtr<const UInputAction>& Action = ContinuousInjection.Key;
		if (const UInputAction* InputAction = Action.Get())
		{
			PlayerInput->InjectInputForAction(InputAction, ContinuousInjection.Value.RawValue, ContinuousInjection.Value.Modifiers, ContinuousInjection.Value.Triggers);
		}
	}

	// Forced action triggering
	for (TPair<TWeakObjectPtr<const UInputAction>, FInputActionValue>& ForcedActionPair : ForcedActions)
	{
		TWeakObjectPtr<const UInputAction>& Action = ForcedActionPair.Key;
		if (const UInputAction* InputAction = Action.Get())
		{
			PlayerInput->InjectInputForAction(InputAction, ForcedActionPair.Value);	// TODO: Support modifiers and triggers?
		}
	}

	// Forced key presses
	for (TPair<FKey, FInjectedKeyData>& ForcedKeyPair : ForcedKeys)
	{
		// Prefer sending the key pressed event via a player controller if one is available.
		if (APlayerController* Controller = Cast<APlayerController>(PlayerInput->GetOuter()))
		{
			InjectKey(Controller, ForcedKeyPair.Key, ForcedKeyPair.Value.InputValue, DeltaTime, Controller->GetPlatformUserId());
		}
		else
		{
			InjectKey(PlayerInput, ForcedKeyPair.Key, ForcedKeyPair.Value.InputValue, DeltaTime, PLATFORMUSERID_NONE);
		}

		// Keep track of the fact that we have injected this input value so we can check
		// it if we remove input on the same frame
		ForcedKeyPair.Value.LastInjectedValue = ForcedKeyPair.Value.InputValue;
	}
}

void IEnhancedInputSubsystemInterface::HandleControlMappingRebuildDelegate()
{
	if (bControlMappingsRebuiltThisTick)
	{
		ControlMappingsRebuiltThisFrame();
		
		bControlMappingsRebuiltThisTick = false;
	}
}

void IEnhancedInputSubsystemInterface::ApplyForcedInput(const UInputAction* Action, FInputActionValue Value)
{
	check(Action);
	ForcedActions.Emplace(Action, Value);		// TODO: Support modifiers and triggers?
}

void IEnhancedInputSubsystemInterface::ApplyForcedInput(FKey Key, FInputActionValue Value)
{
	check(Key.IsValid());
	
	FInjectedKeyData& Data = ForcedKeys.FindOrAdd(Key);
	Data.InputValue = Value;
}

void IEnhancedInputSubsystemInterface::RemoveForcedInput(const UInputAction* Action)
{
	ForcedActions.Remove(Action);
}

void IEnhancedInputSubsystemInterface::RemoveForcedInput(FKey Key)
{
	check(Key.IsValid());

	const FInjectedKeyData* InjectedKeyData = ForcedKeys.Find(Key);
	if (!InjectedKeyData)
	{
		// Nothing to do if the value was not being injected
		return;
	}
	
	// Otherwise, we need to inject a release event tos player input
	if (UEnhancedPlayerInput* PlayerInput = GetPlayerInput())
	{
		// Set the input device id to the platform user's default input device
		const FPlatformUserId UserId = PlayerInput->GetOwningLocalPlayer()->GetPlatformUserId();
		const float DeltaTime = PlayerInput->GetWorld()->GetDeltaSeconds();

		// We want to inject the opposite of whatever we were previously injecting for this key
		// in order to get it back to providing a fake value of zero. For example, if we were injecting (.5,.5)
		// we want to use a delta of -.5,-.5 to get us back to a zero value. We only want to do this
		// for analog keys.
		//
		// Any digital key we always want a value of zero to ensure it is treated as a release event.
		const FInputActionValue ValueToInject = Key.IsAnalog() ? -InjectedKeyData->LastInjectedValue.Get<FVector>() : FVector::ZeroVector;
	
		// Prefer sending the key released event via a player controller if one is available.
		if (APlayerController* Controller = Cast<APlayerController>(PlayerInput->GetOuter()))
		{
			InjectKey(Controller, Key, ValueToInject, DeltaTime, UserId, IE_Released);
		}
		else
		{
			InjectKey(PlayerInput, Key, ValueToInject, DeltaTime, UserId, IE_Released);
		}
	
		// Flush the player's pressed keys to ensure that the removed event is read
		// and the PlayerInput re-evaluates the RawEventAccumulator as needed.
		PlayerInput->FlushPressedKeys();
	}

	// No longer inject this key on tick
	ForcedKeys.Remove(Key);
}
