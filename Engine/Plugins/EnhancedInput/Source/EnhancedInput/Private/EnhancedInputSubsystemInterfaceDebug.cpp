// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputSubsystemInterface.h"

#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "EnhancedInputLibrary.h"
#include "EnhancedInputModule.h"
#include "GameFramework/PlayerController.h"
#include "InputMappingContext.h"
#include "ImageUtils.h"
#include "UObject/Package.h"

/* Shared input subsystem debug functionality.
 * See EnhancedInputSubsystemInterface.cpp for main functionality.
 */

namespace UE::EnhancedInput
{
	static int32 EIDebug_ShowKeyboardKeys = 1;
	static FAutoConsoleVariableRef CVarEIDebug_ShowKeyboardKeys(TEXT("EnhancedInput.Debug.ShowKeyboardKeys"),
		EIDebug_ShowKeyboardKeys,
		TEXT("Should Enhanced Input Debug show keyboard keys in the debug info?"));
	
	static int32 EIDebug_ShowGamepadKeys = 1;
	static FAutoConsoleVariableRef CVarEIDebug_ShowGamepadKeys(TEXT("EnhancedInput.Debug.ShowGamepadKeys"),
		EIDebug_ShowGamepadKeys,
		TEXT("Should Enhanced Input Debug show Gamepad keys in the debug info?"));
		
	static int32 EIDebug_ShowTouchKeys = 1;
	static FAutoConsoleVariableRef CVarEIDebug_ShowTouchKeys(TEXT("EnhancedInput.Debug.ShowTouchKeys"),
		EIDebug_ShowTouchKeys,
		TEXT("Should Enhanced Input Debug show Touch keys in the debug info?"));
		
	static FString EIDebug_InputActionFilter = TEXT("");
	static FAutoConsoleVariableRef CVarEIDebug_InputActionFilter(TEXT("EnhancedInput.Debug.InputActionFilter"),
		EIDebug_InputActionFilter,
		TEXT("This is the list of filtered input actions to display. Multiple filter entries should be separated by a comma."));
		
	static FString EIDebug_ContextMappingFilter = TEXT("");
	static FAutoConsoleVariableRef CVarEIDebug_ContextMappingFilter(TEXT("EnhancedInput.Debug.ContextMappingFilter"),
		EIDebug_ContextMappingFilter,
		TEXT("This is the list of filtered mapping contexts to display. Multiple filter entries should be separated by a comma."));
}

// Manual GC protection is required for visualization textures as they live within an interface class.
struct FVisualizationTexture
{
	// Move only
	FVisualizationTexture(UTexture2D* InTexture) : Texture(InTexture)
	{
		if (Texture)
		{
			Texture->AddToRoot();
		}
	}
	FVisualizationTexture(FVisualizationTexture&& Other)
	{
		Texture = Other.Texture;
		Other.Texture = nullptr;
	}
	FVisualizationTexture& operator=(FVisualizationTexture&& Other)
	{
		Texture = Other.Texture;
		Other.Texture = nullptr;
		return *this;
	}
	~FVisualizationTexture()
	{
		if (!GExitPurge && Texture)
		{
			Texture->RemoveFromRoot();
		}
	}
	UTexture2D* Texture = nullptr;
};
static TMap<uint64, FVisualizationTexture> CachedModifierVisualizations;

void IEnhancedInputSubsystemInterface::ShowMappingContextDebugInfo(UCanvas* Canvas, const UEnhancedPlayerInput* PlayerInput)
{
	if (!Canvas)
	{
		return;
	}

	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;

	if (!PlayerInput)
	{
		DisplayDebugManager.SetDrawColor(FColor::Red);
		DisplayDebugManager.DrawString(TEXT("Invalid Player Input!"));
		return;
	}

	DisplayDebugManager.SetDrawColor(FColor::Cyan);
	DisplayDebugManager.DrawString(FString::Printf(TEXT("Input Mode: %s"), *PlayerInput->GetCurrentInputMode().ToString()));

	if (PlayerInput->EnhancedActionMappings.Num() + PlayerInput->LastInjectedActions.Num() == 0)
	{
		DisplayDebugManager.SetDrawColor(FColor::Orange);
		DisplayDebugManager.DrawString(TEXT("No enhanced player input action mappings have been applied to this input."));
	}
	else
	{
		// Map of already applied keys and the context that applied them
		TMap<FName, FString> AppliedKeys;

		auto ColorFromTriggerState = [](ETriggerState TriggerState)
		{
			switch (TriggerState)
			{
			case ETriggerState::Ongoing:	return FColor::Yellow;
			case ETriggerState::Triggered:	return FColor::Green;
			default:
			case ETriggerState::None:		return FColor::White;
			}
		};

		auto ColorFromTriggerEvent = [](ETriggerEvent TriggerEvent)
		{
			switch (TriggerEvent)
			{
			case ETriggerEvent::Started:	return FColor::Orange;
			case ETriggerEvent::Ongoing:	return FColor::Yellow;
			case ETriggerEvent::Canceled:	return FColor::Red;
			case ETriggerEvent::Triggered:	return FColor::Green;
			default:
			case ETriggerEvent::None:		return FColor::White;
			}
		};

		TMap<TObjectPtr<const UInputMappingContext>, FAppliedInputContextData> OrderedInputContexts = PlayerInput->AppliedInputContextData;
		OrderedInputContexts.ValueSort([](const FAppliedInputContextData& A, const FAppliedInputContextData& B) { return A.Priority > B.Priority; });
		
		// Work through all input contexts, displaying active mappings, overridden mappings, etc.
		for (const TPair<TObjectPtr<const UInputMappingContext>, FAppliedInputContextData>& ContextPair : OrderedInputContexts)
		{
			const UInputMappingContext* AppliedContext = ContextPair.Key.Get();

			// If you for some reason choose to not use IMC assets and instead dynamically spawn them at runtime, they could be
			// garbage collected. This is NOT a recommended approach to using Enhanced Input, but we will put this error here
			// just in case.
			if (!AppliedContext)
			{
				DisplayDebugManager.SetDrawColor(FColor::Red);
				DisplayDebugManager.DrawString(TEXT("Null Input Mapping Context!"));
				continue;
			}

			//Update the array of filtered entries
			static FString LastAppliedContextMappingFilter = TEXT("");
			static TArray<FString> AppliedContextMappingFilters;
			if (!LastAppliedContextMappingFilter.Equals(UE::EnhancedInput::EIDebug_ContextMappingFilter, ESearchCase::IgnoreCase))
			{
				AppliedContextMappingFilters.Reset();
				LastAppliedContextMappingFilter = UE::EnhancedInput::EIDebug_ContextMappingFilter;
				LastAppliedContextMappingFilter.ParseIntoArray(AppliedContextMappingFilters, TEXT(","));
				for (FString& Entry : AppliedContextMappingFilters)
				{
					Entry.TrimStartAndEndInline();
				}
			}

			//apply the filter
			bool bMatchedContextMappingFilter = AppliedContextMappingFilters.IsEmpty();
			for (const FString& FilterEntry : AppliedContextMappingFilters)
			{
				if (AppliedContext->GetName().Contains(FilterEntry, ESearchCase::IgnoreCase))
				{
					bMatchedContextMappingFilter = true;
					break;
				}
			}
			if (!bMatchedContextMappingFilter)
			{
				continue;
			}			

			DisplayDebugManager.SetDrawColor(FColor::Yellow);

			const EMappingContextRegistrationTrackingMode TrackingMode = AppliedContext->GetRegistrationTrackingMode();
			FString TrackingModeDebugString;
			switch (TrackingMode)
			{
			case EMappingContextRegistrationTrackingMode::Untracked:
				// Nothing to show for untracked
				break;
			case EMappingContextRegistrationTrackingMode::CountRegistrations:
				TrackingModeDebugString = FString::Printf(TEXT(" | Registration Count: %d"), ContextPair.Value.RegistrationCount);
				break;
			default:
				checkNoEntry(); // Unhandled tracking mode
				break;
			}
			
			// If this context was redirected via platform settings, then add some debug info about it here
			if (const TObjectPtr<const UInputMappingContext>* RedirectedContextPtr = AppliedContextRedirects.Find(AppliedContext))
			{
				const UInputMappingContext* RedirectedContext = *RedirectedContextPtr;
				DisplayDebugManager.DrawString(FString::Printf(TEXT("  Redirected Context: %s -> %s | Priority: %d | Tracking Mode: %s%s"),
					*AppliedContext->GetFName().ToString(),
					*RedirectedContext->GetFName().ToString(),
					ContextPair.Value.Priority,
					*UEnum::GetValueAsString(TrackingMode),
					*TrackingModeDebugString));

				// Change the current context that is being used to display the correct mappings
				AppliedContext = RedirectedContext;
			}
			else
			{
				DisplayDebugManager.DrawString(FString::Printf(TEXT("  Context: %s | Priority: %d | Tracking Mode: %s%s"),
					*AppliedContext->GetFName().ToString(),
					ContextPair.Value.Priority,
					*UEnum::GetValueAsString(TrackingMode),
					*TrackingModeDebugString));
			}

			// Build a table of mappings per action
			TArray<const UInputAction*> OrderedActions;
			TMap<const UInputAction*, TArray<FEnhancedActionKeyMapping>> ActionMappings;
			for (const FEnhancedActionKeyMapping& Mapping : AppliedContext->GetMappings())
			{
				if (Mapping.Action)
				{
					TArray<FEnhancedActionKeyMapping>& Mappings = ActionMappings.FindOrAdd(Mapping.Action);
					Mappings.Add(Mapping);
					OrderedActions.AddUnique(Mapping.Action);
				}
			}

			// Some mappings may have been added to the player via the Player Mappable Key system, and thus would not be in any of the 
			// IMC assets. We can determine that here :) 
			const TArray<FEnhancedActionKeyMapping>& EnhancedActionMappings = PlayerInput->GetEnhancedActionMappings();
			for (const FEnhancedActionKeyMapping& EnhancedActionMapping : EnhancedActionMappings)
			{
				// Player Mappable keys would have to have an Input Action asset already, so we can filter our search down to that
				if (TArray<FEnhancedActionKeyMapping>* Mappings = ActionMappings.Find(EnhancedActionMapping.Action))
				{
					const bool bMappingExistsAlready = Mappings->ContainsByPredicate([&EnhancedActionMapping](const FEnhancedActionKeyMapping& ExistingMapping)
						{ 
							return ExistingMapping.Key == EnhancedActionMapping.Key && ExistingMapping.Action == EnhancedActionMapping.Action;
						});

					if (!bMappingExistsAlready)
					{
						Mappings->Add(EnhancedActionMapping);
					}					
				}
			}

			Algo::SortBy(OrderedActions, &UInputAction::GetFName, FNameLexicalLess());

			for (const UInputAction* Action : OrderedActions)
			{
				//Update the array of filtered entries
				static FString LastAppliedInputActionFilter = TEXT("");
				static TArray<FString> AppliedInputActionFilters;
				if (!LastAppliedInputActionFilter.Equals(UE::EnhancedInput::EIDebug_InputActionFilter, ESearchCase::IgnoreCase))
				{
					AppliedInputActionFilters.Reset();
					LastAppliedInputActionFilter = UE::EnhancedInput::EIDebug_InputActionFilter;
					LastAppliedInputActionFilter.ParseIntoArray(AppliedInputActionFilters, TEXT(","));
					for (FString& Entry : AppliedInputActionFilters)
					{
						Entry.TrimStartAndEndInline();
					}
				}

				//apply the filter
				bool bMatchedInputActionFilter = AppliedInputActionFilters.IsEmpty();
				for (const FString& FilterEntry : AppliedInputActionFilters)
				{
					if (Action->GetName().Contains(FilterEntry, ESearchCase::IgnoreCase))
					{
						bMatchedInputActionFilter = true;
						break;
					}
				}
				if (!bMatchedInputActionFilter)
				{
					continue;
				}
				
				struct FDisplayLine
				{
					FDisplayLine(FColor InColor, FString InLine) : Color(InColor), Line(InLine) {}
					FColor Color;
					FString Line;
				};
				TArray<FDisplayLine> MappingDisplayLine;

				ETriggerState ActionTriggerState = ETriggerState::None;

				auto GetTriggerState = [](const TArray<UInputTrigger*> Triggers) {
					FString TriggerOutput;
					for (UInputTrigger* Trigger : Triggers)
					{
						if (Trigger)
						{
							FString State = Trigger->GetDebugState();
							if (State.Len())
							{
								TriggerOutput += (TriggerOutput.IsEmpty() ? FString("") : FString(", ")) + State;
							}
						}
					}
					return !TriggerOutput.IsEmpty() ? "  Triggers: " + TriggerOutput : FString();
				};

				// TODO: Order these by key display name?
				for (FEnhancedActionKeyMapping& Mapping : ActionMappings[Action])
				{
					const bool bIsGamepadKey = Mapping.Key.IsGamepadKey();
					const bool bIsTouchKey = Mapping.Key.IsTouch();
					const bool bIsKeyboardKey = !bIsGamepadKey && !bIsTouchKey;

					//skip based on key type and cvars
					if (bIsGamepadKey && !UE::EnhancedInput::EIDebug_ShowGamepadKeys)
					{
						continue;
					}
					else if (bIsTouchKey && !UE::EnhancedInput::EIDebug_ShowTouchKeys)
					{
						continue;
					}
					else if (bIsKeyboardKey && !UE::EnhancedInput::EIDebug_ShowKeyboardKeys)
					{
						continue;
					}
					
					FString* KeyOwner = AppliedKeys.Find(Mapping.Key.GetFName());
					FColor DrawColor = FColor::White;
					const FColor DrawColorInvalid = FColor(64, 64, 64);

					FString Output = "      ";

					const FKeyState* KeyState = PlayerInput->GetKeyState(Mapping.Key);
					FInputActionValue RawValue(FInputActionValue::GetValueTypeFromKey(Mapping.Key), KeyState ? KeyState->RawValue : FVector::ZeroVector);

					// Show raw key value if non-zero
					Output += Mapping.Key.GetDisplayName().ToString() + (!KeyOwner && RawValue.GetMagnitudeSq() ? " - ( " + RawValue.ToString() + " ) - " : " - ");

					auto AnyChords = [](const UInputTrigger* Trigger) { return Cast<const UInputTriggerChordAction>(Trigger) != nullptr; };
					bool bHasChords = IEnhancedInputSubsystemInterface::HasTriggerWith(AnyChords, Mapping.Triggers) || IEnhancedInputSubsystemInterface::HasTriggerWith(AnyChords, Mapping.Action->Triggers);
					if (KeyOwner)
					{
						// Another mapping already owns this key
						Output += "  : OVERRIDDEN BY " + *KeyOwner;
						DrawColor = FColor(64, 64, 64);
					}
					else if (!bHasChords && Mapping.Action->bConsumeInput)
					{
						// This mapping owns this key!
						AppliedKeys.Emplace(Mapping.Key.GetFName(), AppliedContext->GetFName().ToString() + ":" + Action->GetFName().ToString());
					}
					else if (bHasChords && KeyOwner)
					{
						// Another mapping is chording this key
						Output += "  : Chorded BY " + *KeyOwner;
						DrawColor = FColor(64, 64, 64);	// TODO: Change color if chord is active
					}
					else if (!Mapping.Action->bConsumeInput)
					{
						// Draw attention to the fact we aren't consuming input as it adds processing cost.
						Output += " (Not consuming input)";
					}
					Output += "  ";

					auto Idx = PlayerInput->EnhancedActionMappings.Find(Mapping);

					if (Idx == INDEX_NONE)
					{
						DrawColor = FColor(64, 64, 64);
					}
					else
					{
						// TODO: Show state breakdown for all (Active?) triggers?
						const FEnhancedActionKeyMapping& InstancedMapping = PlayerInput->EnhancedActionMappings[Idx];
						Output += GetTriggerState(InstancedMapping.Triggers);
					}

					MappingDisplayLine.Add(FDisplayLine(DrawColor, Output));
				}

				const FInputActionInstance* ActionData = PlayerInput->FindActionInstanceData(Action);

				DisplayDebugManager.SetDrawColor(ColorFromTriggerEvent(ActionData ? ActionData->GetTriggerEvent() : ETriggerEvent::None));
				FString ActionStr(TEXT("    Action: "));
				ActionStr += Action->GetFName().ToString();
				if (ActionData/* && ActionData->GetTriggerEvent() != ETriggerEvent::None*/)
				{
					ActionStr += FString::Printf(TEXT(" - %s - %.3fs (%s)"), *UEnum::GetValueAsString(TEXT("EnhancedInput.ETriggerEvent"), ActionData->GetTriggerEvent()), ActionData->GetTriggerEvent() == ETriggerEvent::Triggered ? ActionData->GetTriggeredTime() : ActionData->GetElapsedTime(), *ActionData->GetValue().ToString());
					ActionStr += GetTriggerState(ActionData->GetTriggers());
				}
				DisplayDebugManager.DrawString(ActionStr);

				// The drawing of these debug textures only works in the editor and causes some serious perf hitches 
				// when you try to call it in a cooked game build.
#if WITH_EDITOR
				ShowDebugActionModifiers(Canvas, Action);
#endif	// WITH_EDITOR

				for (const FDisplayLine& DisplayLine : MappingDisplayLine)
				{
					DisplayDebugManager.SetDrawColor(DisplayLine.Color);
					DisplayDebugManager.DrawString(DisplayLine.Line);
				}
			}
		}
	}
}

void IEnhancedInputSubsystemInterface::ShowDebugInfo(UCanvas* Canvas)
{
	if (!Canvas)
	{
		return;
	}

	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;

	// TODO: Localize some/all debug output?
	UEnhancedPlayerInput* PlayerInput = GetPlayerInput();
	if (!PlayerInput)
	{
		DisplayDebugManager.SetDrawColor(FColor::Orange);
		DisplayDebugManager.DrawString(TEXT("This player does not support Enhanced Input. To enable it update Project Settings -> Input -> Default Classes to the Enhanced versions."));
		return;
	}

	if (APlayerController* PC = Cast<APlayerController>(PlayerInput->GetOuter()))
	{
		DisplayDebugManager.SetDrawColor(FColor::White);
		DisplayDebugManager.DrawString(FString::Printf(TEXT("Player: %s"), *PC->GetFName().ToString()));

		// TODO: Display input stack? Remove input stack?
		//TArray<UInputComponent*> InputStack;
		//PC->BuildInputStack(InputStack);
		//FString InputStackStr;
		//for(UInputComponent* IC : InputStack)
		//{
		//	AActor* Owner = InputStack[i]->GetOwner();
		//	InputStackStr += Owner ? Owner->GetFName().ToString() + "." : "" + IC->GetFName().ToString() + " > ";
		//}
		//DisplayDebugManager.SetDrawColor(FColor::White);
		//DisplayDebugManager.DrawString(FString::Printf(TEXT("Input stack: %s"), *InputStackStr.LeftChop(3)));
	}

	ShowMappingContextDebugInfo(Canvas, PlayerInput);
}


void IEnhancedInputSubsystemInterface::GetAllRelevantInputModifiersForDebug(const UEnhancedPlayerInput* PlayerInput, const FInputActionInstance* InstanceData, OUT TArray<UInputModifier*>& OutModifiers)
{
	OutModifiers.Reset();

	// Start with the modifiers from the instance data, these will be from the Input Action asset
	OutModifiers.Append(InstanceData->GetModifiers());	
	
	// We also need to check the mappings that have been added via the Input Mapping Context
	for (const FEnhancedActionKeyMapping& Mapping : PlayerInput->GetEnhancedActionMappings())
	{
		if (Mapping.Action == InstanceData->GetSourceAction())
		{
			for (const TObjectPtr<UInputModifier>& Modifier : Mapping.Modifiers)
			{
				OutModifiers.AddUnique(Modifier.Get());
			}
		}
	}
}

void IEnhancedInputSubsystemInterface::ShowDebugActionModifiers(UCanvas* Canvas, const UInputAction* Action)
{
	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
	const UEnhancedPlayerInput* PlayerInput = GetPlayerInput();
	const FInputActionInstance* ActionData = PlayerInput ? PlayerInput->FindActionInstanceData(Action) : nullptr;
	if (!ActionData)
	{
		return;
	}

	// Visualize modifiers
	// TODO: Display per mapping, including mapping modifiers
	// TODO: Display colored dots to show current sampling locations of valid mappings?
	// TODO: Invalidate and recalculate textures if sample hash changes? (Rebuild sample data every frame)

	static TArray<UInputModifier*> Modifiers;

	IEnhancedInputSubsystemInterface::GetAllRelevantInputModifiersForDebug(PlayerInput, ActionData, OUT Modifiers);
	
	if (Modifiers.Num() > 0)
	{
		const bool bIs1D = ActionData->GetValue().GetValueType() == EInputActionValueType::Axis1D || ActionData->GetValue().GetValueType() == EInputActionValueType::Boolean;

		const FVector2D VisSize(64, bIs1D ? 1 : 64);
		const FVector2D VisPad(16.f, 4.f);	// TODO: Use '+' char width?
		const FVector2D DrawSize(64, bIs1D ? 16 : 64); // Squash modifier representations vertically if they're 1D

		int32 FinalYPos = DisplayDebugManager.GetYPos() + DrawSize.Y + VisPad.Y * 2.f + DisplayDebugManager.GetMaxCharHeight();
		FVector2D VisScreenPos(VisPad.X + DisplayDebugManager.GetXPos(), VisPad.Y + DisplayDebugManager.GetYPos());

		auto DrawStringAt = [&DisplayDebugManager](const FString& String, float x, float y) {
			DisplayDebugManager.SetYPos(y);
			DisplayDebugManager.DrawString(String, x - DisplayDebugManager.GetXPos());
		};

		auto DrawVisualization = [&Canvas, &VisScreenPos, &VisPad, &ActionData, &DrawSize, &DisplayDebugManager, DrawStringAt](UInputModifier* Modifier, UTexture2D* Visualization, const FString& PreSeperator)
		{
			Canvas->K2_DrawTexture(Visualization, VisScreenPos, DrawSize, FVector2D::ZeroVector);
			DisplayDebugManager.SetDrawColor(FColor::White);

			FString VisName = TEXT("Final");
			if (Modifier)
			{
				const int32 LabelLen = 10;	// TODO: Function of DrawSize.X?
				static int32 InputModifierIdentifierEnd = UInputModifier::StaticClass()->GetName().Len();
				FString ModifierName = Modifier->GetClass()->GetName().Mid(InputModifierIdentifierEnd, LabelLen);	// Modifier name without "InputModifier" at the front, clipped to LabelLen max chars
				VisName = FName::NameToDisplayString(ModifierName, false);
			}
			float Width, Height;
			Canvas->TextSize(GEngine->GetSmallFont(), VisName, Width, Height);
			DrawStringAt(VisName, VisScreenPos.X + (DrawSize.X - Width) / 2, VisScreenPos.Y + DrawSize.Y);

			if (PreSeperator.Len())
			{
				Canvas->TextSize(GEngine->GetSmallFont(), PreSeperator, Width, Height);
				DrawStringAt(PreSeperator, VisScreenPos.X - VisPad.X / 2 - Width / 2, VisScreenPos.Y + DrawSize.Y / 2 - Height / 2);
			}

			VisScreenPos.X += DrawSize.X + VisPad.X;
		};

		auto BuildSampleOffset = [&VisSize](int32 x, int32 y) { return FVector(x / (VisSize.X - 1) * 2.f - 1.f, y / (VisSize.Y - 1) * 2.f - 1.f, 0.f); };


		static TArray<FVector> RunningSampleData;
		RunningSampleData.SetNumUninitialized(VisSize.X * VisSize.Y, EAllowShrinking::No);

		// Reset running data to initial values
		for (int32 y = 0; y < VisSize.Y; ++y)
		{
			for (int32 x = 0; x < VisSize.X; ++x)
			{
				RunningSampleData[x + y * VisSize.X] = BuildSampleOffset(x, y);
			}
		}

		auto BuildVisualizationTexture = [&VisSize, &Action, BuildSampleOffset](const UObject* Owner, const TArray<FVector>& SampleData)
		{
			TArray<FColor> ColorData;
			ColorData.Reserve(SampleData.Num());

			for (int32 y = 0; y < VisSize.Y; ++y)
			{
				for (int32 x = 0; x < VisSize.X; ++x)
				{
					FVector SampleOffset = BuildSampleOffset(x, y);
					FVector Result = SampleData[x + y * VisSize.X];

					if (const UInputModifier* Modifier = Cast<const UInputModifier>(Owner))
					{
						// Modifiers can apply custom visualizations
						FColor Color = Modifier->GetVisualizationColor(FInputActionValue(Action->ValueType, SampleOffset), FInputActionValue(Action->ValueType, Result)).ToFColor(false);
						ColorData.Add(Color);
					}
					else
					{
						// Non-modifiers (e.g. running total) visualize intensity only.
						uint8 Intensity = uint8(255 * FMath::Min(1.f, Result.Size2D()));
						ColorData.Add(FColor(Intensity, Intensity, Intensity));
					}
				}
			}

			FCreateTexture2DParameters Params;
			Params.bDeferCompression = true;
			Params.CompressionSettings = TC_EditorIcon;
			// FImageUtils::CreateTexture2D will return null and throw a fatal error unless we have WITH_EDITOR
#if WITH_EDITOR
			return FImageUtils::CreateTexture2D(VisSize.X, VisSize.Y, ColorData, IEnhancedInputModule::Get().GetLibrary(), FString(), EObjectFlags::RF_NoFlags, Params);
#else 
			return nullptr;
#endif
		};

		bool bDrawnAtLeastOneModifier = false;
		bool bBuiltAtLeastOneModifier = false;

		for (UInputModifier* Modifier : Modifiers)
		{
			FVisualizationTexture* FoundVis = CachedModifierVisualizations.Find((uint64)Modifier);
			UTexture2D* Visualization = FoundVis ? FoundVis->Texture : nullptr;
			if (!Visualization)	// TODO: Running sample calculation requires we always step into this unless all visualizations are already valid
			{
				static TArray<FVector> SampleData;
				SampleData.Reserve(VisSize.X * VisSize.Y);
				SampleData.Reset();
				// Take copies of each modifier to do the sampling, so as not to corrupt internal state.
				UInputModifier* SampleModifier = DuplicateObject<UInputModifier>(Modifier, GetTransientPackage());
				UInputModifier* RunningSampleModifier = DuplicateObject<UInputModifier>(Modifier, GetTransientPackage());
				for (int32 y = 0; y < VisSize.Y; ++y)
				{
					// TODO: How to handle Axis3D? 3D texture and show slice on ActionData->GetValue().Z? Full 3D render?
					for (int32 x = 0; x < VisSize.X; ++x)
					{
						const float DeltaTime = 0.f;// 1.f / 60.f;	// TODO: If we want to sample the whole space at a given time step we need to reset the modifier state each sample and rebuild every frame.

						// Update local result
						FVector Result = SampleModifier->ModifyRaw(PlayerInput, FInputActionValue(Action->ValueType, BuildSampleOffset(x, y)), DeltaTime).Get<FVector>();
						SampleData.Add(Result);

						// Update running result
						FVector& RunningResult = RunningSampleData[x + y * VisSize.X];
						RunningResult = RunningSampleModifier->ModifyRaw(PlayerInput, FInputActionValue(Action->ValueType, RunningResult), DeltaTime).Get<FVector>();

					}
				}
				
				Visualization = CachedModifierVisualizations.Add((uint64)Modifier, FVisualizationTexture(BuildVisualizationTexture(Modifier, SampleData))).Texture;
				bBuiltAtLeastOneModifier = true;
			}

			DrawVisualization(Modifier, Visualization, bDrawnAtLeastOneModifier ? TEXT("+") : TEXT(""));
			bDrawnAtLeastOneModifier = true;
		}

		// Show final running total version
		FVisualizationTexture* FoundRunningVis = CachedModifierVisualizations.Find((uint64)ActionData);
		UTexture2D* RunningVis = FoundRunningVis ? FoundRunningVis->Texture : nullptr;
		if (!RunningVis)
		{
			RunningVis = CachedModifierVisualizations.Add((uint64)ActionData, FVisualizationTexture(BuildVisualizationTexture(PlayerInput, RunningSampleData))).Texture;
		}

		DrawVisualization(nullptr, RunningVis, TEXT("="));

		DisplayDebugManager.SetYPos(FinalYPos);
	}
}

void IEnhancedInputSubsystemInterface::PurgeDebugVisualizations()
{
	CachedModifierVisualizations.Empty();
}
