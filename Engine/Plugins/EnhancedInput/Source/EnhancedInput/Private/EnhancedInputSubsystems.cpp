// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputSubsystems.h"

#include "Components/InputComponent.h"
#include "EnhancedInputDeveloperSettings.h"
#include "EnhancedInputModule.h"
#include "EnhancedInputWorldProcessor.h"
#include "Engine/Canvas.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/PlayerController.h"
#include "InputMappingContext.h"
#include "UserSettings/EnhancedInputUserSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedInputSubsystems)

DEFINE_LOG_CATEGORY(LogWorldSubsystemInput);

// **************************************************************************************************
// *
// * UEnhancedInputLocalPlayerSubsystem
// *
// **************************************************************************************************


namespace UE::Input
{
	static int32 EnableDefaultMappingContexts = 1;
	static FAutoConsoleVariableRef CVarEnableDefaultMappingContexts(TEXT("EnhancedInput.EnableDefaultMappingContexts"),
		EnableDefaultMappingContexts,
		TEXT("Should the UEnhancedInputDeveloperSettings::DefaultMappingContexts be applied to every UEnhancedPlayerInput?"));
}

void UEnhancedInputLocalPlayerSubsystem::Deinitialize()
{
	Super::Deinitialize();

	ValidateTrackedMappingContextsAreUnregistered();
}

void UEnhancedInputLocalPlayerSubsystem::PlayerControllerChanged(APlayerController* NewPlayerController)
{
	Super::PlayerControllerChanged(NewPlayerController);
	
	if (GetDefault<UEnhancedInputDeveloperSettings>()->bEnableUserSettings)
	{
		InitalizeUserSettings();
	}

	if (UE::Input::EnableDefaultMappingContexts)
	{
		// Add any default input mapping contexts that we may need
    	for (const FDefaultContextSetting& DefaultMapping : GetDefault<UEnhancedInputDeveloperSettings>()->DefaultMappingContexts)
    	{
    		if (const UInputMappingContext* IMC = DefaultMapping.InputMappingContext.LoadSynchronous())
    		{
    			if (DefaultMapping.bAddImmediately)
    			{
    				AddMappingContext(IMC, DefaultMapping.Priority);
    			}
    			
    			if (DefaultMapping.bRegisterWithUserSettings && UserSettings)
    			{
    				UserSettings->RegisterInputMappingContext(IMC);
    			}
    		}
    	}	
	}	
}

UEnhancedPlayerInput* UEnhancedInputLocalPlayerSubsystem::GetPlayerInput() const
{
	if (ULocalPlayer* LP = GetLocalPlayer())
	{
		if (APlayerController* PlayerController = LP->GetPlayerController(GetWorld()))
		{
			return Cast<UEnhancedPlayerInput>(PlayerController->PlayerInput);	
		}
	}
	
	return nullptr;
}

UEnhancedInputUserSettings* UEnhancedInputLocalPlayerSubsystem::GetUserSettings() const
{
	return UserSettings;
}

void UEnhancedInputLocalPlayerSubsystem::InitalizeUserSettings()
{
	ULocalPlayer* LP = GetLocalPlayer();

	if (!LP)
	{
		UE_LOGF(LogEnhancedInput, Error, "Failed to initialize user settings, there is no valid local player!");
		return;
	}
	
	// If the user settings are already created then we just want to update it's pointer to the 
	// local player object, as that may have changed with a new player controller.
	if (UserSettings)
	{
		UserSettings->Initialize(LP);
		return;
	}

	if (!GetDefault<UEnhancedInputDeveloperSettings>()->bEnableUserSettings)
	{
		UE_LOGF(LogEnhancedInput, Verbose, "bEnableUserSettings is set to false, skipping creation of UEnhancedInputUserSettings!");
		return;
	}

	// We can't load the user settings without a valid player input object because we get the save file name from it
	if (!GetPlayerInput())
	{
		UE_LOGF(LogEnhancedInput, Warning, "UEnhancedInputLocalPlayerSubsystem for Local Player '%ls' does not have a valid PlayerInput object. Failed to load user settings.", *GetNameSafe(LP));
		return;
	}
	
	UserSettings = UEnhancedInputUserSettings::LoadOrCreateSettings(LP);

	// Bind delegates to the user settings if it was successfully created
	if (ensure(UserSettings))
	{
		BindUserSettingDelegates();
		
		OnPostUserSettingsInitialized.Broadcast(UserSettings);
		
		UE_LOGF(LogEnhancedInput, Log, "Enhanced Input local player subsystem has initialized the user settings!");
	}
}

void UEnhancedInputLocalPlayerSubsystem::ControlMappingsRebuiltThisFrame()
{
	ControlMappingsRebuiltDelegate.Broadcast();
}

void UEnhancedInputLocalPlayerSubsystem::AddMappingContext(const UInputMappingContext* MappingContext, int32 Priority, const FModifyContextOptions& Options)
{
	IEnhancedInputSubsystemInterface::AddMappingContext(MappingContext, Priority, Options);
	
	OnMappingContextAdded.Broadcast(MappingContext);
}

void UEnhancedInputLocalPlayerSubsystem::RemoveMappingContext(const UInputMappingContext* MappingContext, const FModifyContextOptions& Options)
{
	IEnhancedInputSubsystemInterface::RemoveMappingContext(MappingContext, Options);
	
	OnMappingContextRemoved.Broadcast(MappingContext);
}

// **************************************************************************************************
// *
// * UEnhancedInputWorldSubsystem
// *
// **************************************************************************************************

bool UEnhancedInputWorldSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	if (!Super::ShouldCreateSubsystem(Outer) || !FSlateApplication::IsInitialized())
	{
		return false;
	}

	// Getting setting on whether to turn off subsystem or not
	const bool bShouldCreate = GetDefault<UEnhancedInputDeveloperSettings>()->bEnableWorldSubsystem;
	if (!bShouldCreate)
	{
		UE_LOGF(LogWorldSubsystemInput, Verbose, "UEnhancedInputDeveloperSettings::bEnableWorldSubsystem is false, the world subsystem will not be created!");
	}

	return bShouldCreate;
}

void UEnhancedInputWorldSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	UClass* PlayerInputClass = nullptr;
	if (TSoftClassPtr<UEnhancedPlayerInput> DefaultWorldClass = GetDefault<UEnhancedInputDeveloperSettings>()->DefaultWorldInputClass)
	{
		PlayerInputClass = DefaultWorldClass.Get();
	}
	else
	{
		PlayerInputClass = UEnhancedPlayerInput::StaticClass();
	}
		
	PlayerInput = NewObject<UEnhancedPlayerInput>(this, PlayerInputClass, TEXT("EIWorldSubsystem_PlayerInput0"));	
	ensureMsgf(PlayerInput, TEXT("UEnhancedInputWorldSubsystem::Initialize failed to create PlayerInput! This subsystem will not tick!"));
	
	if (ensure(FSlateApplication::IsInitialized()))
	{
		InputPreprocessor = MakeShared<FEnhancedInputWorldProcessor>();
		FSlateApplication::Get().RegisterInputPreProcessor(InputPreprocessor);
	}
	
	// Add any default mapping contexts
	AddDefaultMappingContexts();
	RequestRebuildControlMappings();
}

void UEnhancedInputWorldSubsystem::Deinitialize()
{
	Super::Deinitialize();

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().UnregisterInputPreProcessor(InputPreprocessor);
	}
	
	ValidateTrackedMappingContextsAreUnregistered();

	InputPreprocessor.Reset();
	CurrentInputStack.Empty();
	PlayerInput = nullptr;

	RemoveDefaultMappingContexts();
}

bool UEnhancedInputWorldSubsystem::DoesSupportWorldType(const EWorldType::Type WorldType) const
{
	// The world subsystem shouldn't be used in the editor
	return WorldType == EWorldType::Game || WorldType == EWorldType::PIE;
}

void UEnhancedInputWorldSubsystem::TickPlayerInput(float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UEnhancedInputWorldSubsystem::TickPlayerInput);
	
	static TArray<UInputComponent*> InputStack;
	InputStack.Reset();

	// Build the input stack
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UEnhancedInputWorldSubsystem::TickPlayerInput::BuildInputStack);
		for (int32 i = 0; i < CurrentInputStack.Num(); ++i)
		{
			if (UInputComponent* IC = CurrentInputStack[i].Get())
			{
				InputStack.Push(IC);
			}
			else
			{
				CurrentInputStack.RemoveAt(i--);
			}
		}
	}
	
	// Process input stack on the player input
	PlayerInput->Tick(DeltaTime);
	PlayerInput->ProcessInputStack(InputStack, DeltaTime, GetWorld()->IsPaused());	
}

void UEnhancedInputWorldSubsystem::AddActorInputComponent(AActor* Actor)
{
	if (Actor)
	{
		if (TObjectPtr<UInputComponent> InInputComponent = Actor->InputComponent)
		{
			bool bPushed = false;
			CurrentInputStack.RemoveSingle(InInputComponent);
			for (int32 Index = CurrentInputStack.Num() - 1; Index >= 0; --Index)
			{
				UInputComponent* IC = CurrentInputStack[Index].Get();
				if (IC == nullptr)
				{
					CurrentInputStack.RemoveAt(Index);
				}
				else if (IC->Priority <= InInputComponent->Priority)
				{
					CurrentInputStack.Insert(InInputComponent, Index + 1);
					bPushed = true;
					break;
				}
			}
			if (!bPushed)
			{
				CurrentInputStack.Insert(InInputComponent, 0);
				RequestRebuildControlMappings();
			}
		}
		else
		{
			UE_LOGF(LogWorldSubsystemInput, Error, "Attempted to push a null Input Component to the Enhanced Input World Subsystem!");
		}
	}
}

bool UEnhancedInputWorldSubsystem::RemoveActorInputComponent(AActor* Actor)
{
	if (Actor) 
	{
		if (TObjectPtr<UInputComponent> InInputComponent = Actor->InputComponent)
    	{
    		if (CurrentInputStack.RemoveSingle(InInputComponent) > 0)
    		{
    			InInputComponent->ClearBindingValues();
    			RequestRebuildControlMappings();
    			return true;
    		}
    	}
	}
    return false;
}

bool UEnhancedInputWorldSubsystem::InputKey(const FInputKeyEventArgs& Params)
{
	if (PlayerInput)
	{
		UE_LOGF(LogWorldSubsystemInput, VeryVerbose, "EI %ls World Subsystem InputKey : [%ls]", LexToString(GetWorld()->WorldType), *Params.Key.ToString());
		return PlayerInput->InputKey(Params);
	}

	ensureAlwaysMsgf(false, TEXT("Attempting to input a key to the EnhancedInputWorldSubsystem, but there is no Player Input!"));
	return false;
}

void UEnhancedInputWorldSubsystem::AddDefaultMappingContexts()
{
	if (GetDefault<UEnhancedInputDeveloperSettings>()->bEnableDefaultMappingContexts)
	{
		for (const FDefaultContextSetting& ContextSetting : GetDefault<UEnhancedInputDeveloperSettings>()->DefaultWorldSubsystemMappingContexts)
		{
			if (const UInputMappingContext* IMC = ContextSetting.InputMappingContext.LoadSynchronous())
			{
				if (ContextSetting.bAddImmediately)
				{
					if (!HasMappingContext(IMC))
					{
						FModifyContextOptions Opts = {};
						Opts.bNotifyUserSettings = ContextSetting.bRegisterWithUserSettings;
						AddMappingContext(IMC, ContextSetting.Priority, Opts);	
					}					
				}
				else if (ContextSetting.bRegisterWithUserSettings)
				{
					if (UEnhancedInputUserSettings* Settings = GetUserSettings())
					{
						Settings->RegisterInputMappingContext(IMC);
					}
				}
			}
		}
	}
}

void UEnhancedInputWorldSubsystem::RemoveDefaultMappingContexts()
{
	if (GetDefault<UEnhancedInputDeveloperSettings>()->bEnableDefaultMappingContexts)
	{
		for (const FDefaultContextSetting& ContextSetting : GetDefault<UEnhancedInputDeveloperSettings>()->DefaultWorldSubsystemMappingContexts)
		{
			if (const UInputMappingContext* IMC = ContextSetting.InputMappingContext.LoadSynchronous())
			{
				FModifyContextOptions Opts = {};
				Opts.bNotifyUserSettings = ContextSetting.bRegisterWithUserSettings;
				RemoveMappingContext(IMC, Opts);
			}
		}
	}
}

UEnhancedPlayerInput* UEnhancedInputWorldSubsystem::GetPlayerInput() const
{
	return PlayerInput;
}

void UEnhancedInputWorldSubsystem::ShowDebugInfo(UCanvas* Canvas)
{
	if (!Canvas)
	{
		return;
	}

	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;

	UEnhancedPlayerInput* WorldSubsystemPlayerInput = GetPlayerInput();
	if (!WorldSubsystemPlayerInput)
	{
		DisplayDebugManager.SetDrawColor(FColor::Orange);
		DisplayDebugManager.DrawString(TEXT("This player does not support Enhanced Input."));
		return;
	}
	
	DisplayDebugManager.SetDrawColor(FColor::White);
	DisplayDebugManager.DrawString(FString::Printf(TEXT("World Subsystem from %s"), LexToString(GetWorld()->WorldType)));
	
	ShowMappingContextDebugInfo(Canvas, WorldSubsystemPlayerInput);
}
