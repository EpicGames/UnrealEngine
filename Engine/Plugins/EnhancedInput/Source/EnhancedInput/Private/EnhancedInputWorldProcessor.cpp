// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputWorldProcessor.h"
#include "EnhancedInputSubsystems.h"
#include "Framework/Application/SlateApplication.h"
#include "EnhancedInputDeveloperSettings.h"
#include "UObject/UObjectIterator.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"

void FEnhancedInputWorldProcessor::Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor)
{
	ProcessAccumulatedPointerInput(DeltaTime);
}

bool FEnhancedInputWorldProcessor::HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	FInputKeyEventArgs Params = FInputKeyEventArgs::CreateSimulated(
		InKeyEvent.GetKey(),
		IE_Pressed,
		1.0f,
		InKeyEvent.GetKey().IsAnalog() ? 1 : 0,
		InKeyEvent.GetInputDeviceId());
	
	Params.DeltaTime = SlateApp.GetDeltaTime();
	
	InputKeyToSubsystem(Params);
	
	return IInputProcessor::HandleKeyDownEvent(SlateApp, InKeyEvent);
}

bool FEnhancedInputWorldProcessor::HandleKeyUpEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	FInputKeyEventArgs Params = FInputKeyEventArgs::CreateSimulated(
		InKeyEvent.GetKey(),
		IE_Released,
		0.0f,
		InKeyEvent.GetKey().IsAnalog() ? 1 : 0,
		InKeyEvent.GetInputDeviceId());
	
	Params.DeltaTime = SlateApp.GetDeltaTime();
	
	InputKeyToSubsystem(Params);
	return IInputProcessor::HandleKeyUpEvent(SlateApp, InKeyEvent);
}

bool FEnhancedInputWorldProcessor::HandleAnalogInputEvent(FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent)
{
	FInputKeyEventArgs Params = FInputKeyEventArgs::CreateSimulated(
		InAnalogInputEvent.GetKey(),
		IE_Pressed,
		InAnalogInputEvent.GetAnalogValue(),
		1,
		InAnalogInputEvent.GetInputDeviceId());
	
	Params.DeltaTime = SlateApp.GetDeltaTime();
	
	InputKeyToSubsystem(Params);
	return IInputProcessor::HandleAnalogInputEvent(SlateApp, InAnalogInputEvent);
}

bool FEnhancedInputWorldProcessor::HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	UpdateCachedPointerPosition(MouseEvent);
	
	return IInputProcessor::HandleMouseMoveEvent(SlateApp, MouseEvent);
}

bool FEnhancedInputWorldProcessor::HandleMouseButtonDownEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	FInputKeyEventArgs Params = FInputKeyEventArgs::CreateSimulated(
		MouseEvent.GetEffectingButton(),
		IE_Pressed,
		1.0f,
		0,
		MouseEvent.GetInputDeviceId(),
		MouseEvent.IsTouchEvent());

	Params.DeltaTime = SlateApp.GetDeltaTime();
	
	InputKeyToSubsystem(Params);
	return IInputProcessor::HandleMouseButtonDownEvent(SlateApp, MouseEvent);
}

bool FEnhancedInputWorldProcessor::HandleMouseButtonUpEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	FInputKeyEventArgs Params = FInputKeyEventArgs::CreateSimulated(
		MouseEvent.GetEffectingButton(),
		IE_Released,
		0.0f,
		0,
		MouseEvent.GetInputDeviceId(),
		MouseEvent.IsTouchEvent());
	
	Params.DeltaTime = SlateApp.GetDeltaTime();
	
	InputKeyToSubsystem(Params);
	return IInputProcessor::HandleMouseButtonUpEvent(SlateApp, MouseEvent);
}

bool FEnhancedInputWorldProcessor::HandleMouseButtonDoubleClickEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
	FInputKeyEventArgs Params = FInputKeyEventArgs::CreateSimulated(
		MouseEvent.GetEffectingButton(),
		IE_DoubleClick,
		1.0f,
		0,
		MouseEvent.GetInputDeviceId(),
		MouseEvent.IsTouchEvent());

	Params.DeltaTime = SlateApp.GetDeltaTime();
	
	InputKeyToSubsystem(Params);
	return IInputProcessor::HandleMouseButtonDoubleClickEvent(SlateApp, MouseEvent);
}

bool FEnhancedInputWorldProcessor::HandleMouseWheelOrGestureEvent(FSlateApplication& SlateApp, const FPointerEvent& InWheelEvent, const FPointerEvent* InGestureEvent)
{
	const FKey MouseWheelKey = InWheelEvent.GetWheelDelta() < 0 ? EKeys::MouseScrollDown : EKeys::MouseScrollUp;

	// Input the Mouse wheel key events (mouse scroll down or scroll up) as being pressed and released this frame
	// The SceneViewport Inputs the Mouse Scroll Wheel buttons up and down in the same frame, this replicates that behavior
	{
		FInputKeyEventArgs PressedParams = FInputKeyEventArgs::CreateSimulated(
			MouseWheelKey,
			IE_Pressed,
			1.0f,
			0,
			InWheelEvent.GetInputDeviceId(),
			InWheelEvent.IsTouchEvent());

		PressedParams.DeltaTime = SlateApp.GetDeltaTime();
		
		FInputKeyEventArgs ReleasedParams = PressedParams;
		ReleasedParams.Event = IE_Released;

		InputKeyToSubsystem(PressedParams);
		InputKeyToSubsystem(ReleasedParams);	
	}
	// Input the wheel axis delta to get the MouseWheelAxis button working
	{
		FInputKeyEventArgs Params = FInputKeyEventArgs::CreateSimulated(
			EKeys::MouseWheelAxis,
			IE_Axis,
			InWheelEvent.GetWheelDelta(),
			1,
			InWheelEvent.GetInputDeviceId(),
			InWheelEvent.IsTouchEvent());

		Params.DeltaTime = SlateApp.GetDeltaTime();
		
		InputKeyToSubsystem(Params);
	}
	
	return IInputProcessor::HandleMouseWheelOrGestureEvent(SlateApp, InWheelEvent, InGestureEvent);
}

void FEnhancedInputWorldProcessor::UpdateCachedPointerPosition(const FPointerEvent& MouseEvent)
{
	CachedCursorDelta = MouseEvent.GetCursorDelta();
	
	++NumCursorSamplesThisFrame.X;
	++NumCursorSamplesThisFrame.Y;
}

void FEnhancedInputWorldProcessor::ProcessAccumulatedPointerInput(float DeltaTime)
{
	// Input the MouseX value
	{
		FInputKeyEventArgs Params = FInputKeyEventArgs::CreateSimulated(
			EKeys::MouseX,
			IE_Axis,
			CachedCursorDelta.X,
			NumCursorSamplesThisFrame.X,
			IPlatformInputDeviceMapper::Get().GetDefaultInputDevice());

		Params.DeltaTime = DeltaTime;

		InputKeyToSubsystem(Params);
	}

	// Input the MouseY value
	{
		FInputKeyEventArgs Params = FInputKeyEventArgs::CreateSimulated(
			EKeys::MouseY,
			IE_Axis,
			CachedCursorDelta.Y,
			NumCursorSamplesThisFrame.Y,
			IPlatformInputDeviceMapper::Get().GetDefaultInputDevice());

		Params.DeltaTime = DeltaTime;
		
		InputKeyToSubsystem(Params);
	}
	
	NumCursorSamplesThisFrame = FIntPoint::ZeroValue;
	CachedCursorDelta = FVector2D::ZeroVector;
}


bool FEnhancedInputWorldProcessor::InputKeyToSubsystem(const FInputKeyEventArgs& Params)
{
	bool bRes = false;
	
	if (GetDefault<UEnhancedInputDeveloperSettings>()->bEnableWorldSubsystem)
	{
		// Tell all the world subsystems about the key that has been pressed
		for (TObjectIterator<UEnhancedInputWorldSubsystem> It; It; ++It)
		{
			bRes |= (*It)->InputKey(Params);
		}
	}
	
	return bRes;
}
