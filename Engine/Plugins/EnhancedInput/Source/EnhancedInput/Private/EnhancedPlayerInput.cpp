// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedPlayerInput.h"

#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "EnhancedInputComponent.h"
#include "EnhancedInputDeveloperSettings.h"
#include "EnhancedInputModule.h"
#include "EnhancedInputSubsystemInterface.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerInputDebugging.h"
#include "GameFramework/WorldSettings.h"
#include "HAL/IConsoleManager.h"
#include "InputMappingContext.h"
#include "UserSettings/EnhancedInputUserSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedPlayerInput)

namespace UE::Input
{
	static int32 ShouldOnlyTriggerLastActionInChord = 1;
	static FAutoConsoleVariableRef CVarShouldOnlyTriggerLastActionInChord(TEXT("EnhancedInput.OnlyTriggerLastActionInChord"),
		ShouldOnlyTriggerLastActionInChord,
		TEXT("Should only the last action in a ChordedAction trigger be fired? If this is disabled, then the dependant chords will be fired as well"));

	static int32 ReconcileRemovedMappingDelegates = 1;
	static FAutoConsoleVariableRef CVarReconcileRemovedMappingDelegates(TEXT("EnhancedInput.ReconcileRemovedMappingDelegates"),
		ReconcileRemovedMappingDelegates,
		TEXT("When the mappings are rebuilt, they have been 'in process'. If this is true, we will set their value to zero so that they fire the 'Canceled' event on their next evaluation."));

	static bool bEnableListenerConsumption = true;
	static FAutoConsoleVariableRef CVarEnableListenerConsumption(TEXT("EnhancedInput.EnableListenerConsumption"),
		bEnableListenerConsumption,
		TEXT("If true, then bound listeners have the option to consume any Enhanced Input delegates on lower priority input components."));

	static bool bDetectSubTickTapsForNonAxisInputs = true;
	static FAutoConsoleVariableRef CVarDetectSubTickTapsForNonAxisInputs(TEXT("EnhancedInput.DetectSubTickTapsForNonAxisInputs"),
		bDetectSubTickTapsForNonAxisInputs,
		TEXT("If true, then vast press-release pairs that occur within a tick will trigger for non-axis inputs just as they already do for axis buttons."));
	
	static bool bCorrectTouchBoolActionKeys = true;
	static FAutoConsoleVariableRef CVarCorrectTouchBoolActionKeys(TEXT("EnhancedInput.CorrectTouchBoolActionKeys"),
		bCorrectTouchBoolActionKeys,
		TEXT("If true, boolean input actions mapped to touch keys will apply a bug fix to correctly evaluate IsActuated. This will be removed in a future release."));

	static bool bIgnoreHeldDigitalActionKeysOnFlush = true;
	static FAutoConsoleVariableRef CVarIgnoreHeldKeysOnFlush(TEXT("EnhancedInput.IgnoreHeldDigitalActionKeysOnFlush"),
		bIgnoreHeldDigitalActionKeysOnFlush,
		TEXT("If true, keys that are physically held when FlushPressedKeys is called will be ignored until released and re-pressed. ")
		TEXT("This prevents triggers like Pressed from re-firing after a flush when auto-reconcile restores the key state. This will be removed in a future release."));

	static bool IsKeyValidForAnyKey(const FKey& Key)
	{
		return !Key.IsAnalog() && (Key.IsDigital() || Key.IsButtonAxis());
	}

	static bool IsAnyKeyStateConsideredPressed(const FKey& Key, const FKeyState& State)
	{
		if (!IsKeyValidForAnyKey(Key))
		{
			return false;
		}

		const bool bHasAnyPressedEvents = State.EventCounts[IE_Pressed].Num() > 0 || State.EventAccumulator[IE_Pressed].Num() > 0;

		// Button axis keys may not have the IE_Pressed event set, so check their raw value as well
		return bHasAnyPressedEvents || (Key.IsButtonAxis() && State.RawValue.Size() > 0.0);
	}

	static bool IsAnyKeyStateConsideredReleased(const FKey& Key, const FKeyState& State)
	{
		if (!IsKeyValidForAnyKey(Key))
		{
			return false;
		}

		const bool bHasAnyReleasedEvents = State.EventCounts[IE_Released].Num() > 0 || State.EventAccumulator[IE_Released].Num() > 0;

		// Button axis keys may not have the IE_Released event set, so consider them 'released' if the value is zero
		return bHasAnyReleasedEvents || (Key.IsButtonAxis() && State.RawValue.IsNearlyZero());
	}
}

namespace UE::EnhancedInput
{
	UE_DEFINE_GAMEPLAY_TAG_COMMENT(InputMode_Default, "EnhancedInput.Modes.Default", "The default input mode for Enhanced Input");	
}

UEnhancedPlayerInput::UEnhancedPlayerInput()
	: Super()
	, bIsFlushingInputThisFrame(false)
	, CurrentlyInUseAnyKeySubstitute(NAME_None)
{
	// We don't want to attempt to load stuff on the CDO. Any subobjects on the IMC (triggers, modifiers, Player mappable key settings)
	// will likely not have been loaded yet if they are defined by the end user or in a different module.
	if (HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		return;
	}

	const UEnhancedInputDeveloperSettings* Settings = GetDefault<UEnhancedInputDeveloperSettings>();
	check(Settings);
	
	CurrentInputMode = Settings->DefaultInputMode;
}

// NOTE: Enum order represents firing priority(lowest to highest) and is important as multiple keys bound to the same action may generate differing trigger event states.
enum class ETriggerEventInternal : uint8
{
	None,					// No significant trigger state changes occurred
	Completed,				// Triggering stopped after one or more triggered ticks										ETriggerState (Triggered -> None)
	Started,				// Triggering has begun																		ETriggerState (None -> Ongoing)
	Ongoing,				// Triggering is still being processed														ETriggerState (Ongoing -> Ongoing)
	Canceled,				// Triggering has been canceled	mid processing												ETriggerState (Ongoing -> None)
	StartedAndTriggered,	// Triggering occurred in a single tick (fires both started and triggered events)			ETriggerState (None -> Triggered)
	Triggered,				// Triggering occurred after one or more processing ticks									ETriggerState (Ongoing -> Triggered, Triggered -> Triggered)
};

ETriggerEventInternal UEnhancedPlayerInput::GetTriggerStateChangeEvent(ETriggerState LastTriggerState, ETriggerState NewTriggerState) const
{
	// LastTState	NewTState     Event

	// None		 -> Ongoing		= Started
	// None		 -> Triggered	= Started + Triggered
	// Ongoing	 -> None		= Canceled
	// Ongoing	 -> Ongoing		= Ongoing
	// Ongoing	 -> Triggered	= Triggered
	// Triggered -> Triggered	= Triggered
	// Triggered -> Ongoing		= Ongoing
	// Triggered -> None	    = Completed

	switch (LastTriggerState)
	{
	case ETriggerState::None:
		if (NewTriggerState == ETriggerState::Ongoing)
		{
			return ETriggerEventInternal::Started;
		}
		else if (NewTriggerState == ETriggerState::Triggered)
		{
			return ETriggerEventInternal::StartedAndTriggered;
		}
		break;
	case ETriggerState::Ongoing:
		if (NewTriggerState == ETriggerState::None)
		{
			return ETriggerEventInternal::Canceled;
		}
		else if (NewTriggerState == ETriggerState::Ongoing)
		{
			return ETriggerEventInternal::Ongoing;
		}
		else if (NewTriggerState == ETriggerState::Triggered)
		{
			return ETriggerEventInternal::Triggered;
		}
		break;
	case ETriggerState::Triggered:
		if (NewTriggerState == ETriggerState::Triggered)
		{
			return ETriggerEventInternal::Triggered;	// Don't re-raise Started event for multiple completed ticks.
		}
		else if (NewTriggerState == ETriggerState::Ongoing)
		{
			return ETriggerEventInternal::Ongoing;
		}
		else if (NewTriggerState == ETriggerState::None)
		{
			return ETriggerEventInternal::Completed;
		}
		break;
	}

	return ETriggerEventInternal::None;
}

ETriggerEvent UEnhancedPlayerInput::ConvertInternalTriggerEvent(ETriggerEventInternal InternalEvent) const
{
	switch (InternalEvent)
	{
	case ETriggerEventInternal::None:
		return ETriggerEvent::None;
	case ETriggerEventInternal::Started:
		return ETriggerEvent::Started;
	case ETriggerEventInternal::Ongoing:
		return ETriggerEvent::Ongoing;
	case ETriggerEventInternal::Canceled:
		return ETriggerEvent::Canceled;
	case ETriggerEventInternal::StartedAndTriggered:
	case ETriggerEventInternal::Triggered:
		return ETriggerEvent::Triggered;
	case ETriggerEventInternal::Completed:
		return ETriggerEvent::Completed;
	}
	return ETriggerEvent::None;
}

enum class EKeyEvent : uint8
{
	None,		// Key did not generate an event this tick and is not being held
	Actuated,	// Key has generated an event this tick
	Held,		// Key generated no event, but is in a held state and wants to continue applying modifiers and triggers
};

void UEnhancedPlayerInput::ProcessActionMappingEvent(
	TObjectPtr<const UInputAction> Action,
	float DeltaTime,
	bool bGamePaused,
	FInputActionValue RawKeyValue,
	EKeyEvent KeyEvent,
	const TArray<UInputModifier*>& Modifiers,
	const TArray<UInputTrigger*>& Triggers,
	const bool bHasAlwaysTickTrigger /*= false*/)
{	
	FInputActionInstance& ActionData = FindOrAddActionEventData(Action);

	// Update values and triggers for all actionable mappings each frame
	FTriggerStateTracker TriggerStateTracker;

	// Reset action data on the first event processed for the action this tick.
	bool bResetActionData = !ActionsWithEventsThisTick.Contains(Action);

	// If the key state is changing or the key is actuated and being held (and not coming back up this tick) recalculate its value and resulting trigger state.
	if (KeyEvent != EKeyEvent::None || bHasAlwaysTickTrigger)
	{
		if (bResetActionData)
		{
			ActionsWithEventsThisTick.Add(Action);
			ActionData.Value.Reset();	// TODO: what if default value isn't 0 (e.g. bool value with negate modifier). Move reset out to a pre-pass? This may be confusing as triggering requires key interaction for value processing for performance reasons.
		}

		// Apply modifications to the raw value
		EInputActionValueType ValueType = ActionData.Value.GetValueType();
		FInputActionValue ModifiedValue = ApplyModifiers(Modifiers, FInputActionValue(ValueType, RawKeyValue.Get<FVector>()), DeltaTime);
		//UE_CLOGF(RawKeyValue.GetMagnitudeSq(), LogEnhancedInput, Warning, "Modified %ls -> %ls", *RawKeyValue.ToString(), *ModifiedValue.ToString());

		// Derive an initial trigger state for this mapping using all applicable triggers
		ETriggerState CalcedState = TriggerStateTracker.EvaluateTriggers(this, Triggers, ModifiedValue, DeltaTime);
		// Do this only for no triggers?
		TriggerStateTracker.SetStateForNoTriggers(ModifiedValue.IsNonZero() ? ETriggerState::Triggered : ETriggerState::None);	
		TriggerStateTracker.SetMappingTriggerApplied(Triggers.Num() > 0);

		const EInputActionAccumulationBehavior AccumulationBehavior = ActionData.GetSourceAction()->AccumulationBehavior;

		// Combine values for active events only, selecting the input with the greatest magnitude for each component in each tick.
		if(ModifiedValue.GetMagnitudeSq())
		{
			const int32 NumComponents = FMath::Max(1, int32(ValueType));
			FVector Modified = ModifiedValue.Get<FVector>();
			FVector Merged = ActionData.Value.Get<FVector>();
			for (int32 Component = 0; Component < NumComponents; ++Component)
			{
				switch (AccumulationBehavior)
				{
				// Sometimes you may want to cumulatively merge input. This would allow you to, for example, map WASD to movement and have pressing W and S at the same time
				// completely cancel out input because "W" is a value of +1.0, and "S" is a value of -1.0
				case EInputActionAccumulationBehavior::Cumulative:
				{
					Merged[Component] += Modified[Component];
				}										
				break;

				// By default, we will accept the input with the highest absolute value
				case EInputActionAccumulationBehavior::TakeHighestAbsoluteValue:
				default:
				{
					if (FMath::Abs(Modified[Component]) >= FMath::Abs(Merged[Component]))
					{
						Merged[Component] = Modified[Component];
					}
				}
				break;
				}			
			}
			ActionData.Value = FInputActionValue(ValueType, Merged);
		}
	}

	// Retain the most interesting/triggered tracker.
	ActionData.TriggerStateTracker = FMath::Max(ActionData.TriggerStateTracker, TriggerStateTracker);
}

void UEnhancedPlayerInput::InjectInputForAction(TObjectPtr<const UInputAction> Action, FInputActionValue RawValue, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers)
{
	FInjectedInput Input;
	Input.RawValue = RawValue;
	Input.Modifiers = Modifiers;
	Input.Triggers = Triggers;

	InputsInjectedThisTick.FindOrAdd(Action).Injected.Emplace(MoveTemp(Input));
}

bool UEnhancedPlayerInput::InputKey(const FInputKeyEventArgs& Params)
{
	const bool bResult = Super::InputKey(Params);

	if (Params.Event == IE_Pressed)
	{
		if (Params.Key.IsButtonAxis())
		{
			KeysPressedThisTick.FindOrAdd(Params.Key, FVector(Params.AmountDepressed, 0.0, 0.0));
		}
		else if (UE::Input::bDetectSubTickTapsForNonAxisInputs && Params.Key.IsDigital())
		{
			KeysPressedThisTick.FindOrAdd(Params.Key, FVector(1.0f, 0.0, 0.0));
		}
	}
	else if (Params.Event == IE_Released && UE::Input::bIgnoreHeldDigitalActionKeysOnFlush)
	{
		// Clear bShouldBeIgnored as soon as the release event arrives instead of waiting one full
		// tick (the lifecycle check at the top of EvaluateInputDelegates only clears it after the
		// next tick observes bDown=false && bWasJustFlushed=false). That one-tick lag is the window
		// where a fast re-press lands in the "ignored" state and is silently consumed — the
		// "must press twice to toggle" symptom users see when a UI mapping rebuild flushes the key
		// during the same press that opened the UI.
		for (FEnhancedActionKeyMapping& Mapping : EnhancedActionMappings)
		{
			if (Mapping.bShouldBeIgnored && Mapping.Key == Params.Key)
			{
				Mapping.bShouldBeIgnored = false;
				UE_LOGF(LogEnhancedInput, Verbose, "Key %ls is no longer ignored", *Mapping.Key.ToString());
			}
		}
	}

	return bResult;
}

float UEnhancedPlayerInput::GetEffectiveTimeDilation() const
{
	if (const APlayerController* PC = GetOuterAPlayerController())
	{
		return PC->GetActorTimeDilation();
	}
	else if (const UWorld* World = GetWorld())
	{
		if (const AWorldSettings* WorldSettings = World->GetWorldSettings())
		{
			return WorldSettings->GetEffectiveTimeDilation();	
		}
	}
	return 1.0f;
}

FGameplayTagContainer& UEnhancedPlayerInput::GetCurrentInputMode()
{
	return CurrentInputMode;
}

const FGameplayTagContainer& UEnhancedPlayerInput::GetCurrentInputMode() const
{
	return CurrentInputMode;
}

FString UEnhancedPlayerInput::GetUserSettingsSaveFileName() const
{
	// By default, return the string specified in the settings.
	// You can override this function if you have any more complex scenarios, such as perhaps
	// doing some kind of custom versioning/upgrading of your settings as your changes over time
	// or creating unique save slots for different players.
	return GetDefault<UEnhancedInputDeveloperSettings>()->InputSettingsSaveSlotName;
}

TConstArrayView<const FEnhancedActionKeyMapping> UEnhancedPlayerInput::GetEnhancedActionMappingsView() const
{
	return { EnhancedActionMappings };
}

void UEnhancedPlayerInput::SetCurrentInputMode(const FGameplayTagContainer& NewMode)
{
	CurrentInputMode = NewMode;
}

void UEnhancedPlayerInput::EvaluateKeyMapState(const float DeltaTime, const bool bGamePaused, OUT TArray<TPair<FKey, FKeyState*>>& KeysWithEvents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(EnhPIS_KeyDownPrev);

	const UEnhancedInputDeveloperSettings* Settings = GetDefault<UEnhancedInputDeveloperSettings>();
	bool bWasAnyKeyDownLastFrame = false;

	KeyDownPrevious.Reset();
	KeyDownPrevious.Reserve(GetKeyStateMap().Num());
	for (TPair<FKey, FKeyState>& KeyPair : GetKeyStateMap())
	{
		const FKeyState& KeyState = KeyPair.Value;
		// TODO: Can't just use bDownPrevious as paired axis event edges may not fire due to axial deadzoning/missing axis properties. Need to change how this is detected in PlayerInput.cpp.
		bool bWasDown = KeyState.bDownPrevious || KeyState.EventCounts[IE_Pressed].Num() || KeyState.EventCounts[IE_Repeat].Num();

		bWasDown |=
			(KeyPair.Key.IsAnalog() || KeyPair.Key.IsButtonAxis()) &&
			KeyState.RawValue.SizeSquared() != 0;	// Analog inputs should pulse every (non-zero) tick to retain compatibility with UE4.

		// When UPlayerInput::FlushPressedKeys is called any keys that are down will have their RawValue set to 0, and their bDown/bDownPrevious state will be reset to false.
		// However, their "Value" will not be reset until UPlayerInput::ProcessInputStack. We need to detect if this key was down previously after a flush
		// so that the Enhanced Input action will correctly fire the triggered values.
		const bool bKeyWasJustFlushed = 
			bIsFlushingInputThisFrame && 
			Settings->bSendTriggeredEventsWhenInputIsFlushed && 
			!KeyState.Value.IsZero() && 
			KeyState.RawValue.IsZero() && 
			!KeyState.bDown;

		bWasDown |= bKeyWasJustFlushed;
			
		// Keep track of the state of every key so that when we are done iterating we can have a meaningful value for EKeys::AnyKey
		// Any key does not support analog events, and we want to know if any non-analog key was pressed last frame at all.
		if (UE::Input::IsAnyKeyStateConsideredPressed(KeyPair.Key, KeyState))
		{
			bWasAnyKeyDownLastFrame |= bWasDown;
		}

		KeyDownPrevious.Emplace(KeyPair.Key, bWasDown);
	}

	KeyDownPrevious.Emplace(EKeys::AnyKey, bWasAnyKeyDownLastFrame);
	
	
	Super::EvaluateKeyMapState(DeltaTime, bGamePaused, KeysWithEvents);
}

void UEnhancedPlayerInput::EvaluateInputDelegates(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused, const TArray<TPair<FKey, FKeyState*>>& KeysWithEvents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(EnhPIS_Main);

	Super::EvaluateInputDelegates(InputComponentStack, DeltaTime, bGamePaused, KeysWithEvents);

	// Calculate the current delta time as a fallback in case the Time Dilation is set to 0.0
	// This will ensure that Timed Triggers are calculated in real time
	const UWorld* World = GetWorld();
	// If there is no world, then add 1/60 to the last frame time to try and recover at least some
	// semblance of real time passing. 
	float CurrentTime = World ? World->GetRealTimeSeconds() : LastFrameTime + (1.0f / 60.0f);
	
	// Reset action instance timers where necessary post delegate calls
	for (TPair<TObjectPtr<const UInputAction>, FInputActionInstance>& ActionPair : ActionInstanceData)
	{
		FInputActionInstance& ActionData = ActionPair.Value;
		switch (ActionData.TriggerEvent)
		{
		case ETriggerEvent::None:
		case ETriggerEvent::Canceled:
		case ETriggerEvent::Completed:
			ActionData.ElapsedProcessedTime = 0.f;
			break;
		}
		if (ActionData.TriggerEvent != ETriggerEvent::Triggered)
		{
			ActionData.ElapsedTriggeredTime = 0.f;
		}

		// Delay MappingTriggerState reset until here to allow dependent triggers (e.g. chords) access to this tick's values.
		ActionData.TriggerStateTracker = FTriggerStateTracker();
	}

	// Remove any input actions that are no longer mapped to the player. At this point they will have been evaluated and 
	// fired any "Canceled" input events needed
	for (const UInputAction* IA : ActionsThatHaveBeenRemovedFromMappings)
	{
		ActionInstanceData.Remove(IA);
	}

	// We can clear our queue of mappings that have been removed on the next iteration directly after a key rebuild.
	ActionsThatHaveBeenRemovedFromMappings.Reset();

	LastFrameTime = CurrentTime;
	KeysPressedThisTick.Reset();
	bIsFlushingInputThisFrame = false;
	ConsumedInputActions.Reset();
}

void UEnhancedPlayerInput::PrepareInputDelegatesForEvaluation(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused, const TArray<TPair<FKey, FKeyState*>>& KeysWithEvents)
{
	Super::PrepareInputDelegatesForEvaluation(InputComponentStack, DeltaTime, bGamePaused, KeysWithEvents);

	// Process Action bindings
	ActionsWithEventsThisTick.Reset();

	// Calculate the current delta time as a fallback in case the Time Dilation is set to 0.0
	// This will ensure that Timed Triggers are calculated in real time
	const UWorld* World = GetWorld();
	// If there is no world, then add 1/60 to the last frame time to try and recover at least some
	// semblance of real time passing. 
	float CurrentTime = World ? World->GetRealTimeSeconds() : LastFrameTime + (1.0f / 60.0f);
	RealTimeDeltaSeconds = CurrentTime - LastFrameTime;
	
	// Use non-dilated delta time for processing
	const float Dilation = GetEffectiveTimeDilation();
	const float NonDilatedDeltaTime = Dilation != 0.0f ? DeltaTime / Dilation : RealTimeDeltaSeconds;

	// Handle input devices, applying modifiers and triggers
	for (FEnhancedActionKeyMapping& Mapping : EnhancedActionMappings)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EnhPIS_Devices);

		if (!Mapping.Action)
		{
			continue;
		}

		FKeyState* KeyState = nullptr;

		// If the mapping was to AnyKey, then it won't be in the KeyStateMap and we need to handle it specially. 
		if (Mapping.Key == EKeys::AnyKey)
		{
			// We can just get the first value in the key state map that has been pressed or released, that's what we really care about with this key type
			for (TPair<FKey, FKeyState>& KeyStatePair : KeyStateMap)
			{
				// EKeys::AnyKey will only use non-analog keys. the same as the legacy system.
				if (!UE::Input::IsKeyValidForAnyKey(KeyStatePair.Key))
				{
					continue;
				}

				// If we have no Substitute key, then we can simply use the key state of the first available key with pressed events.
				if (CurrentlyInUseAnyKeySubstitute == NAME_None && UE::Input::IsAnyKeyStateConsideredPressed(KeyStatePair.Key, KeyStatePair.Value))
				{
					KeyState = &KeyStatePair.Value;
					CurrentlyInUseAnyKeySubstitute = KeyStatePair.Key.GetFName();
					break;
				}

				// If we have a substitute key already, then we can just look for the key of this name in the map
				else if (KeyStatePair.Key.GetFName() == CurrentlyInUseAnyKeySubstitute)
				{
					// If the substitute key was just released, then we can reset our currently in 
					// use substitute key so that it can be replaced by something else on the next key press.					
					if (UE::Input::IsAnyKeyStateConsideredReleased(KeyStatePair.Key, KeyStatePair.Value))
					{
						CurrentlyInUseAnyKeySubstitute = NAME_None;
					}

					KeyState = &KeyStatePair.Value;
					break;
				}
			}
		}
		// Virtual Keys have different values which are set at runtime depending on the platform. We need to make sure
		// that we are grabbing the actual FKey name that it represents
		else if (Mapping.Key.IsVirtual())
		{
			KeyState = GetKeyState(Mapping.Key.GetVirtualKey());
		}
		else
		{
			KeyState = GetKeyState(Mapping.Key);
		}
		
		FVector RawKeyValue = KeyState ? KeyState->RawValue : FVector::ZeroVector;
		//UE_CLOGF(RawKeyValue.SizeSquared(), LogEnhancedInput, Warning, "Key %ls - state %ls", *Mapping.Key.GetDisplayName().ToString(), *RawKeyValue.ToString());

		// UPlayerInput::InputTouch stores the screen position in RawValue for all touch events,
		// including Ended. IsActuated() checks RawValue magnitude against the actuation threshold —
		// screen coordinates always exceed it, so the action appears permanently actuated and
		// Release/Hold/Tap triggers stay stuck.
		// - Boolean actions: derive value from bDown so magnitude reflects press state (0 or 1).
		// - Axis1D/2D/3D actions: zero the value once the touch is no longer held; while held the
		//   screen position is the intended value.
		if (UE::Input::bCorrectTouchBoolActionKeys &&
			KeyState &&
			Mapping.Key.IsTouch())
		{
			if (Mapping.Action->ValueType == EInputActionValueType::Boolean)
			{
				RawKeyValue.X = KeyState->bDown;
				RawKeyValue.Y = 0.0f;
			}
			else if (!KeyState->bDown)
			{
				RawKeyValue = FVector::ZeroVector;
			}
		}

		// Should this key be ignored because it was down during a context switch or flush?
		// If so, check if it is back up, otherwise ignore it.
		// bWasJustFlushed bridges the one-tick gap between a flush and the next OS event
		// restoring bDown, preventing premature clearing of the ignore flag.
		if(Mapping.bShouldBeIgnored && KeyState)
		{
			if(KeyState->bDown || KeyState->bWasJustFlushed)
			{
				continue;
			}
			else
			{
				Mapping.bShouldBeIgnored = false;
				UE_LOGF(LogEnhancedInput, Verbose, "Key %ls is no longer ignored", *Mapping.Key.ToString());
			}
		}
		
		// Establish update type.
		bool bDownLastTick = KeyDownPrevious.FindRef(Mapping.Key.GetVirtualKey());
		// TODO: Can't just use bDown as paired axis event edges may not fire due to axial deadzoning/missing axis properties. Need to change how this is detected in PlayerInput.cpp.
		bool bKeyIsDown = KeyState && (KeyState->bDown || KeyState->EventCounts[IE_Pressed].Num() || KeyState->EventCounts[IE_Repeat].Num());
		// Analog inputs should pulse every (non-zero) tick to retain compatibility with UE4. TODO: This would be better handled at the device level.
		bKeyIsDown |= (Mapping.Key.IsAnalog() || Mapping.Key.IsButtonAxis()) && RawKeyValue.SizeSquared() > 0;

		const bool bKeyIsReleased = !bKeyIsDown && bDownLastTick;
		const bool bKeyIsHeld = bKeyIsDown && bDownLastTick;

		const EKeyEvent KeyEvent = bKeyIsHeld ? EKeyEvent::Held : ((bKeyIsDown || bKeyIsReleased) ? EKeyEvent::Actuated : EKeyEvent::None);

		const FVector* PressedThisTickValue = (Mapping.Key == EKeys::AnyKey) ?
			KeysPressedThisTick.Find(FKey(CurrentlyInUseAnyKeySubstitute)) :
			KeysPressedThisTick.Find(Mapping.Key);
		
		// For keys that were pressed and released within the same frame, set their RawValue so that
		// InputTriggers are aware that they have been pressed
		if(PressedThisTickValue && bKeyIsDown && KeyState->EventCounts[IE_Pressed].Num() && KeyState->EventCounts[IE_Released].Num() && RawKeyValue.IsZero())
		{
			RawKeyValue = *PressedThisTickValue;
		}

		// Perform update
		ProcessActionMappingEvent(Mapping.Action, NonDilatedDeltaTime, bGamePaused, RawKeyValue, KeyEvent, Mapping.Modifiers, Mapping.Triggers, Mapping.bHasAlwaysTickTrigger);
	}


	// Strip stored injected input states that weren't re-injected this tick
	for (auto It = LastInjectedActions.CreateIterator(); It; ++It)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EnhPIS_InjectedStrip);
		TObjectPtr<const UInputAction> InjectedAction = *It;

		if (!InjectedAction)
		{
			It.RemoveCurrent();
		}
		else if (!InputsInjectedThisTick.Contains(InjectedAction))
		{
			// Reset action state by "releasing the key".
			ProcessActionMappingEvent(
				InjectedAction,
				NonDilatedDeltaTime,
				bGamePaused,
				FInputActionValue(),
				EKeyEvent::Actuated,
				{},
				{},
				/* bHasAlwaysTickTrigger= */ false);
			
			It.RemoveCurrent();
		}
	}

	// Handle injected inputs, applying modifiers and triggers
	for (TPair<TObjectPtr<const UInputAction>, FInjectedInputArray>& InjectedPair : InputsInjectedThisTick)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EnhPIS_Injected);
		TObjectPtr<const UInputAction> InjectedAction = InjectedPair.Key;
		if (!InjectedAction)
		{
			continue;
		}

		// Update last injection status data
		bool bDownLastTick = false;
		LastInjectedActions.Emplace(InjectedAction, &bDownLastTick);

		EKeyEvent KeyEvent = bDownLastTick ? EKeyEvent::Held : EKeyEvent::Actuated;
		for (FInjectedInput& InjectedInput : InjectedPair.Value.Injected)
		{
			// Perform update
			ProcessActionMappingEvent(InjectedAction, NonDilatedDeltaTime, bGamePaused, InjectedInput.RawValue, KeyEvent, InjectedInput.Modifiers, InjectedInput.Triggers);
		}
	}
	InputsInjectedThisTick.Reset();


	// Post tick action instance updates
	for (TPair<TObjectPtr<const UInputAction>, FInputActionInstance>& ActionPair : ActionInstanceData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EnhPIS_PostTick);

		TObjectPtr<const UInputAction> Action = ActionPair.Key;
		FInputActionInstance& ActionData = ActionPair.Value;
		ETriggerState TriggerState = ETriggerState::None;

		if (ActionsWithEventsThisTick.Contains(Action))
		{
			// Apply action modifiers
			FInputActionValue RawValue = ActionData.Value; 
			ActionData.Value = ApplyModifiers(ActionData.Modifiers, ActionData.Value, NonDilatedDeltaTime);

			// Update what state to use for this data in the case of there being no triggers, otherwise we can get incorrect triggered
			// states even if the modified value is Zero
			if(ActionData.Value.Get<FVector>() != RawValue.Get<FVector>())
			{
				ActionData.TriggerStateTracker.SetStateForNoTriggers(ActionData.Value.IsNonZero() ? ETriggerState::Triggered : ETriggerState::None);	
			}

			ETriggerState PrevState = ActionData.TriggerStateTracker.GetState();
			// Evaluate action triggers. We must always call EvaluateTriggers to update any internal state, even when paused.
			TriggerState = ActionData.TriggerStateTracker.EvaluateTriggers(this, ActionData.Triggers, ActionData.Value, NonDilatedDeltaTime);
			TriggerState = ActionData.TriggerStateTracker.GetMappingTriggerApplied() ? FMath::Min(TriggerState, PrevState) : TriggerState;
			
			// However, if the game is paused invalidate trigger unless the action allows it.
			// TODO: Potential issues with e.g. hold event that's canceled due to pausing, but jumps straight back to its "triggered" state on unpause if the user continues to hold the key.
			if (bGamePaused && !Action->bTriggerWhenPaused)
			{
				TriggerState = ETriggerState::None;
			}
		}

		// Use the new trigger state to determine a trigger event based on changes from the previous trigger state.
		ActionData.TriggerEventInternal = GetTriggerStateChangeEvent(ActionData.LastTriggerState, TriggerState);
		ActionData.TriggerEvent = ConvertInternalTriggerEvent(ActionData.TriggerEventInternal);
		ActionData.LastTriggerState = TriggerState;
		// Evaluate time per action after establishing the internal trigger state across all mappings
		ActionData.ElapsedProcessedTime += TriggerState != ETriggerState::None ? NonDilatedDeltaTime : 0.f;
		ActionData.ElapsedTriggeredTime += (ActionData.TriggerEvent == ETriggerEvent::Triggered) ? NonDilatedDeltaTime : 0.f;
		// Track the time that this trigger was last used
		if(TriggerState == ETriggerState::Triggered)
		{
			ActionData.LastTriggeredWorldTime = CurrentTime;
		}
	}
}

bool UEnhancedPlayerInput::EvaluateInputComponentDelegates(UInputComponent* const InputComponent, const TArray<TPair<FKey, FKeyState*>>& KeysWithEvents, const float DeltaTime, const bool bGamePaused)
{
	bool bBlocksInput = false;
	const bool bCheckConsumedKeys = IEnhancedInputSubsystemInterface::TrackingKeysForAllActionMappings();

	static TArray<FKey> KeysToConsume;
	static TArray<FKey> KeysToConsumeBeforeLegacy;

#if DEV_ONLY_KEY_BINDINGS_AVAILABLE
	// Cache modifier key states for debug key bindings
	const bool bAlt = IsAltPressed(), bCtrl = IsCtrlPressed(), bShift = IsShiftPressed(), bCmd = IsCmdPressed();
#endif
	
	// We only want to process enhanced input components in this subclass
	if (UEnhancedInputComponent* const IC = Cast<UEnhancedInputComponent>(InputComponent))
	{
		// Trigger bound event delegates
		static TArray<TUniquePtr<FEnhancedInputActionEventBinding>> TriggeredDelegates;
		const bool bShouldCheckForLegacyKeyConsumption = IC->ShouldCheckForLegacyKeyConsumption();
		for (const TUniquePtr<FEnhancedInputActionEventBinding>& Binding : IC->GetActionEventBindings())
		{
			// PERF: Lots of map lookups! Group EnhancedActionBindings by Action?
			if (const FInputActionInstance* ActionData = FindActionInstanceData(Binding->GetAction()))
			{
				const FKeyConsumptionOptions* const ConsumptionData = KeyConsumptionData.Find(ActionData->GetSourceAction());

				if (bCheckConsumedKeys && bShouldCheckForLegacyKeyConsumption && (ActionData->LastTriggerState != ETriggerState::None || ActionData->TriggerEvent != ETriggerEvent::None))
				{
					// Don't do this action if its triggering keys were consumed elsewhere
					if (ConsumptionData)
					{
						bool bHasActiveConsumedKey = false;
						bool bHasActiveUnconsumedKey = false;

						for (const FKey& KeyToConsume : ConsumptionData->KeysToConsume)
						{
							KeysToConsume.AddUnique(KeyToConsume);
							// Skip any action whose triggering keys have already been consumed.
							if (FKeyState* KeyState = KeyStateMap.Find(KeyToConsume))
							{
								if (KeyState->bDown || KeyState->bDownPrevious || KeyState->EventCounts[IE_Pressed].Num())
								{
									if (KeyState->bConsumed)
									{
										bHasActiveConsumedKey = true;
									}
									else
									{
										bHasActiveUnconsumedKey = true;
									}
								}
							}
						}

						// We can't just skip an Action if we don't find any unconsumed Keys associated with it, because some actions may not have a key (eg: if they were Injected).
						// Therefore, only skip this action if both:
						//  1. It is associated with one or more Key that is or was pressed but has already been consumed -- this indicates that a conflict with a higher-in-the-stack component HAS occurred;
						//  2. It is NOT associated with any Key that is or was pressed but has NOT already consumed -- as, despite any other conflicts that may have occurred, this would clearly show a non-conflicting request for this action has also occurred.
						if (bHasActiveConsumedKey && !bHasActiveUnconsumedKey)
						{
							if (UE::Input::bEnableListenerConsumption)
							{
								ConsumedInputActions.Add(Binding->GetAction());
							}
							else
							{
								continue;
							}
						}
					}
				}

				const ETriggerEvent BoundTriggerEvent = Binding->GetTriggerEvent();
				// Raise appropriate delegate to report on event state
				if (ActionData->TriggerEvent == BoundTriggerEvent ||
					(BoundTriggerEvent == ETriggerEvent::Started && ActionData->TriggerEventInternal == ETriggerEventInternal::StartedAndTriggered))	// Triggering in a single tick should also fire the started event.
				{
					// Record intent to trigger started as well as triggered
					// EmplaceAt 0 for the "Started" event it is always guaranteed to fire before Triggered
					if (BoundTriggerEvent == ETriggerEvent::Started)
					{
						TriggeredDelegates.EmplaceAt(0, Binding->Clone());
					}
					else
					{
						TriggeredDelegates.Emplace(Binding->Clone());
					}

					// Keep track of the triggered actions this tick so that we can quickly look them up later when determining chorded action state
					if (BoundTriggerEvent == ETriggerEvent::Triggered)
					{
						TriggeredActionsThisTick.Add(ActionData->GetSourceAction());
					}
				}

				// If this delegate is bound to an action that has an event and is flagged to consume legacy keys, then mark it as such.
				if (ConsumptionData)
				{
					if (static_cast<uint8>(ConsumptionData->EventsToCauseConsumption & ActionData->TriggerEvent) != 0)
					{
						// Consume all keys that are mapped to this input action with the proper trigger values
						for (const FKey& KeyToConsume : ConsumptionData->KeysToConsume)
						{
							if (FKeyState* KeyState = KeyStateMap.Find(KeyToConsume))
							{
								KeysToConsumeBeforeLegacy.AddUnique(KeyToConsume);
							}
						}
					}
				}
			}
		}

		// Action all delegates that triggered this tick, in the order in which they triggered.
		for (TUniquePtr<FEnhancedInputActionEventBinding>& Delegate : TriggeredDelegates)
		{
			TObjectPtr<const UInputAction> DelegateAction = Delegate->GetAction();
			bool bCanTrigger = true;

			// Skip input actions which have been marked for consumption by a higher priority UEnhancedInputComponent
			if (UE::Input::bEnableListenerConsumption && ConsumedInputActions.Contains(DelegateAction))
			{
				UE_LOGF(LogEnhancedInput, Verbose, "Input action '%ls' was consumed by a higher priority input component listener. It will no longer fire", *DelegateAction->GetName());
				continue;
			}

			if (UE::Input::ShouldOnlyTriggerLastActionInChord)
			{
				// If this delegate is referenced by a UInputTriggerChordAction::ChordAction
				// then we only want to trigger it the referencing action is not triggered
				for (const UEnhancedPlayerInput::FDependentChordTracker& DepAction : DependentChordActions)
				{
					if (DepAction.DependantAction && DepAction.DependantAction == DelegateAction)
					{
						bCanTrigger &= !TriggeredActionsThisTick.Contains(DepAction.SourceAction);
						if(!bCanTrigger)
						{
							UE_LOGF(LogEnhancedInput, Verbose, "'%ls' action was cancelled, its dependant on '%ls'", *DelegateAction->GetName(), *DepAction.SourceAction->GetName());
						}
					}
				}
			}

			if (bCanTrigger)
			{
				// Search for the action instance data a second time as a previous delegate call may have deleted it.
				if (FInputActionInstance* ActionData = const_cast<FInputActionInstance*>(FindActionInstanceData(DelegateAction)))
				{
					// For events that have started and triggered on the same frame, the event will always be 
					// "Triggered", because that is the latest input state that has been evaluated. While this is the 
					// correct state, it can be annoying to end users trying to bind the same function to multiple
					// events and then determine which state they are in, because it will skip the "Started" flag.
					// By "artificially" setting the trigger event on the action data here we will "force" the 
					// event to match up to that of the delegate that we are firing.
					if (ActionData->TriggerEventInternal == ETriggerEventInternal::StartedAndTriggered)
					{
						const ETriggerEvent OriginalEvent = ActionData->TriggerEvent;
						ActionData->TriggerEvent = Delegate->GetTriggerEvent();

						Delegate->Execute(*ActionData);

						ActionData->TriggerEvent = OriginalEvent;
					}
					else
					{
						Delegate->Execute(*ActionData);
					}

#if UE_PLAYER_INPUT_INCLUDE_DEBUG
					UE::Input::FPlayerInputDebugging::BroadcastPlayerInputDelegateExecuted(
						this,
						{
							.InputComponent = InputComponent,
							.ListeningObject = Delegate->GetUObject(),
							.ActionName = GetPathNameSafe(ActionData->GetSourceAction()),
							.InputValue = ActionData->GetValue().Get<FVector>(),
						});
#endif
				}

				// Keep track of any delegates which want to consume input away from other lower priority listeners
				// bound to this same input action.
				if (UE::Input::bEnableListenerConsumption && Delegate->ShouldConsume())
				{
					ConsumedInputActions.Add(DelegateAction);
				}
			}
		}
		TriggeredDelegates.Reset();
		TriggeredActionsThisTick.Reset();

		// Update action value bindings
		for (const FEnhancedInputActionValueBinding& Binding : IC->GetActionValueBindings())
		{
			if (const UInputAction* Action = Binding.GetAction())
			{
				// PERF: Lots of map lookups! Group EnhancedActionBindings by Action?
				if (const FInputActionInstance* ActionData = FindActionInstanceData(Action))
				{
					Binding.CurrentValue = ActionData->GetValue();
				}
				// If there is no action instance data related to this action, then reset the binding's value to zero
				// to ensure that it gets its Completed state sent to any listeners
				else
				{
					Binding.CurrentValue = FInputActionValue(Action->ValueType, FVector::ZeroVector);
				}
			}
		}


#if DEV_ONLY_KEY_BINDINGS_AVAILABLE
		// DebugKeyBindings are intended to be used to enable/toggle debug functionality only and have reduced functionality compared to old style key bindings. Limitations/differences include:
		// No support for the 'Any Key' concept. Explicit key binds only.
		// They will always fire, and cannot mask each other or action bindings (i.e. no bConsumeInput option)
		// Chords are supported, but there is no chord masking protection. Exact chord combinations must be met. So a binding of Ctrl + A will not fire if Ctrl + Alt + A is pressed.
		static TArray<TUniquePtr<FInputDebugKeyBinding>> TriggeredDebugDelegates;
		for (const TUniquePtr<FInputDebugKeyBinding>& KeyBinding : IC->GetDebugKeyBindings())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(EnhPIS_DebugKeys);

			ensureMsgf(KeyBinding->Chord.Key != EKeys::AnyKey, TEXT("Debug key bindings don't support 'any key'!"));

			// We match modifier key state here to explicitly block unmodified debug actions whilst modifier keys are held down, rather than allow e.g. E through on Alt + E.
			// This acts as a simplified version of chord masking.
			if (KeyBinding->Chord.bAlt == bAlt &&
				KeyBinding->Chord.bCtrl == bCtrl &&
				KeyBinding->Chord.bShift == bShift &&
				KeyBinding->Chord.bCmd == bCmd)
			{
				// TODO: Support full chord masking? Not worth the extra effort for debug keys?
				if (!bGamePaused || KeyBinding->bExecuteWhenPaused)
				{
					FKeyState* KeyState = KeyStateMap.Find(KeyBinding->Chord.Key);
					// We always want to update any analog debug events, like Gamepad axis 
					if ((KeyState && KeyState->EventCounts[KeyBinding->KeyEvent].Num() > 0) || KeyBinding->Chord.Key.IsAnalog())
					{
						// Record intent to trigger
						TriggeredDebugDelegates.Add(KeyBinding->Clone());
					}
				}
			}
		}

		// Action all debug delegates that triggered this tick, in the order in which they triggered.
		for (TUniquePtr<FInputDebugKeyBinding>& Delegate : TriggeredDebugDelegates)
		{
			const FKeyState* KeyState = GetKeyState(Delegate->Chord.Key);
			
			FInputActionValue ActionValue(KeyState ? KeyState->RawValue : FVector::ZeroVector);
			
			Delegate->Execute(ActionValue);
		}
		TriggeredDebugDelegates.Reset();
#endif

		bBlocksInput = IC->bBlockInput;
	}

	// Actions that specifically opt into consuming legacy keys need to do so before we process legacy bindings on this component
	for (int32 KeyIndex = 0; KeyIndex < KeysToConsumeBeforeLegacy.Num(); ++KeyIndex)
	{
		if (FKeyState* KeyState = KeyStateMap.Find(KeysToConsumeBeforeLegacy[KeyIndex]))
		{
			KeyState->bConsumed = true;
		}
	}
	
	const bool bLegacyBlocksInput = Super::EvaluateInputComponentDelegates(InputComponent, KeysWithEvents, DeltaTime, bGamePaused);

	if (bCheckConsumedKeys && !bBlocksInput)
	{
		// Remaining keys to be consumed before we proceed down the stack
		for (int32 KeyIndex = 0; KeyIndex < KeysToConsume.Num(); ++KeyIndex)
		{
			if (FKeyState* KeyState = KeyStateMap.Find(KeysToConsume[KeyIndex]))
			{
				KeyState->bConsumed = true;
			}
		}
	}

	KeysToConsume.Reset();
	KeysToConsumeBeforeLegacy.Reset();

	return bBlocksInput || bLegacyBlocksInput;
}

void UEnhancedPlayerInput::EvaluateBlockedInputComponent(UInputComponent* InputComponent)
{
	Super::EvaluateBlockedInputComponent(InputComponent);

	if (UEnhancedInputComponent* IC = Cast<UEnhancedInputComponent>(InputComponent))
	{
		for (const FEnhancedInputActionValueBinding& Binding : IC->GetActionValueBindings())
		{
			Binding.CurrentValue.Reset();
		}
	}
}

void UEnhancedPlayerInput::FlushPressedKeys()
{
	// Before flushing, mark any mappings with currently-held keys as ignored until
	// the key is physically released and re-pressed. Without this, the flush resets bDown=false
	// and when auto-reconcile (bAutoReconcilePressedEventsOnFirstRepeat) later restores the key
	// via IE_Repeat, triggers like Pressed see a fresh actuation edge and re-fire.
	if (UE::Input::bIgnoreHeldDigitalActionKeysOnFlush)
	{
		for (FEnhancedActionKeyMapping& Mapping : EnhancedActionMappings)
		{
			// We only want to do this for digital keys which are mapped to boolean input actions
			// because analog keys will not have a IE_Pressed/IE_Released, it will just report the next
			// analog event value, and we dont want to ignore that. If we did ignore it, then it would make players
			// "reset" their analog values back to zero, which is not desirable
			if (Mapping.Action && 
				Mapping.Key.IsDigital())
			{
				const FKeyState* KeyState = GetKeyState(Mapping.Key);
				if (KeyState && (KeyState->bDown || KeyState->RawValue.SizeSquared() > 0))
				{
					// Don't mark as ignored if a release event is already queued for this tick.
					// This covers fast taps (mobile touch button injections, scripted Press+Release
					// pairs) where the key transiently appears held but the release is in flight.
					// EventCounts holds events already swapped in by ProcessInputStack this tick;
					// EventAccumulator holds events not yet swapped. Checking both keeps the fix
					// correct regardless of whether the flush runs before or after the input swap.
					const bool bReleaseQueuedThisTick =
						!KeyState->EventCounts[IE_Released].IsEmpty() ||
						!KeyState->EventAccumulator[IE_Released].IsEmpty();

					if (bReleaseQueuedThisTick)
					{
						continue;
					}

					Mapping.bShouldBeIgnored = true;
					UE_LOGF(LogEnhancedInput, Verbose, "Key %ls mapped to Input Action %ls will be ignored until released", *Mapping.Key.ToString(), *GetNameSafe(Mapping.Action));
				}
			}
		}
	}

	Super::FlushPressedKeys();

	bIsFlushingInputThisFrame = true;
}

FInputActionValue UEnhancedPlayerInput::GetActionValue(TObjectPtr<const UInputAction> ForAction) const
{
	if (!ForAction)
	{
		UE_LOGF(LogEnhancedInput, Error, "[%hs] Attempting to get the value of an invalid Input action!", __func__);
		return FInputActionValue();
	}

	const FInputActionInstance* ActionData = FindActionInstanceData(ForAction);
	return ActionData ? ActionData->GetValue() : FInputActionValue(ForAction->ValueType, FInputActionValue::Axis3D::ZeroVector);
}

int32 UEnhancedPlayerInput::AddMapping(const FEnhancedActionKeyMapping& Mapping)
{
	// Keep track of what keys are bound in Enhanced Input so that we can query it with IsKeyHandledByAction if desired.
	++EnhancedKeyBinds.FindOrAdd(Mapping.Key);

	// Flag the key mappings as having changed and requiring a rebuild
	bKeyMapsBuilt = false;

	// Before adding a new mapping, look for a FEnhancedActionKeyMapping that is the same as this newly requested mapping.
	// We use the ".Equals" function here to compare instead of the raw FEnhancedActionKeyMapping::operator==()
	// because we want to compare the triggers/modifiers by their object types, not their exact values.
	// The exact values will never be the same because they are instanced.
	for (int32 Index = 0; Index < EnhancedActionMappings.Num(); ++Index)
	{
		const FEnhancedActionKeyMapping& ExistingMapping = EnhancedActionMappings[Index];
		if (ExistingMapping.Equals(Mapping))
		{
			return Index;
		}
	}
	
	return EnhancedActionMappings.Add(Mapping);
}

void UEnhancedPlayerInput::ClearAllMappings()
{
	EnhancedActionMappings.Reset();
	EnhancedKeyBinds.Reset();

	bKeyMapsBuilt = false;
}

template<typename T>
void UEnhancedPlayerInput::GatherActionEventDataForActionMap(const T& ActionMap, TMap<TObjectPtr<const UInputAction>, FInputActionInstance>& FoundActionEventData) const
{
	for (const typename T::ElementType& Pair : ActionMap)
	{
		TObjectPtr<const UInputAction> Action = Pair.Key;
		if (FInputActionInstance* ActionData = ActionInstanceData.Find(Action))
		{
			FoundActionEventData.Add(Action, *ActionData);
		}
	}
}

void UEnhancedPlayerInput::ConditionalBuildKeyMappings_Internal() const
{
	Super::ConditionalBuildKeyMappings_Internal();

	// Remove any ActionEventData without a corresponding entry in EnhancedActionMappings or the injection maps
	for (auto Itr = ActionInstanceData.CreateIterator(); Itr; ++Itr)
	{
		TObjectPtr<const UInputAction> Action = Itr.Key();

		auto HasActionMapping = [&Action](const FEnhancedActionKeyMapping& Mapping) { return Mapping.Action == Action; };

		if (!LastInjectedActions.Contains(Action) &&
			!InputsInjectedThisTick.Contains(Action) &&		// This will be empty for most calls, but could potentially contain data.
			//EngineDefinedActionMappings.ContainsByPredicate(HasActionMapping) && // TODO: EngineDefinedActionMappings are non-rebindable action/key pairings but we have our own systems to handle this...
			!EnhancedActionMappings.ContainsByPredicate(HasActionMapping) && 
			!ActionsThatHaveBeenRemovedFromMappings.Contains(Action))
		{
			Itr.RemoveCurrent();
		}
	}

	bKeyMapsBuilt = true;
}

FInputActionValue UEnhancedPlayerInput::ApplyModifiers(const TArray<UInputModifier*>& Modifiers, FInputActionValue RawValue, float DeltaTime) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(EnhPIS_Modifiers);

	FInputActionValue ModifiedValue = RawValue;
	for (UInputModifier* Modifier : Modifiers)
	{
		if (Modifier)
		{
			// Enforce that type is kept to RawValue type between modifiers.
			ModifiedValue = FInputActionValue(RawValue.GetValueType(), Modifier->ModifyRaw(this, ModifiedValue, DeltaTime).Get<FInputActionValue::Axis3D>());
		}
	}
	return ModifiedValue;
}

bool UEnhancedPlayerInput::IsKeyHandledByAction(FKey Key) const
{
	// Determines if the key event is handled or not.
	return EnhancedKeyBinds.Contains(Key) || Super::IsKeyHandledByAction(Key);
}

void UEnhancedPlayerInput::NotifyInputActionsUnmapped(const TSet<const UInputAction*>& RemovedInputActions)
{
	if (UE::Input::ReconcileRemovedMappingDelegates)
	{
		// Instead of totally removing the action instance data so that it stops being processed,
		// we should instead set the value to zero. This will make it so that upon the next evaluation 
		// of this action instance data, the "Canceled" event will be fired instead there being
		// no notification at all. We will queue these action instances for removal from the 
		// instance data after they have been re-evaluated
		for (const UInputAction* Action : RemovedInputActions)
		{
			if (FInputActionInstance* ActionData = ActionInstanceData.Find(Action))
			{
				ActionData->Value.Reset();
			}
		}

		ActionsThatHaveBeenRemovedFromMappings = RemovedInputActions;
	}
	// ... this is the legacy behavior of just removing the data from the instance data cache
	else
	{
		for (const UInputAction* Action : RemovedInputActions)
		{			
			ActionInstanceData.Remove(Action);
		}
		
		UE_LOGF(LogEnhancedInput, VeryVerbose, "[%hs] EnhancedInput.ReconcileRemovedMappingDelegates is false! Using the legacy behavior to remove action instance data. Canceled events will not be fired.", __func__);
	}
}

FInputActionInstance& UEnhancedPlayerInput::FindOrAddActionEventData(TObjectPtr<const UInputAction> Action) const
{
	FInputActionInstance* Instance = ActionInstanceData.Find(Action);
	if (!Instance)
	{
		Instance = &ActionInstanceData.Emplace(Action, FInputActionInstance(Action));
	}
	return *Instance;
}

void UEnhancedPlayerInput::InitializeMappingActionModifiers(const FEnhancedActionKeyMapping& Mapping)
{
	if (Mapping.Action)
	{
		// Perform a modifier calculation pass on default data to initialize values correctly.
		FInputActionInstance& EventData = FindOrAddActionEventData(Mapping.Action);
		EventData.Value = ApplyModifiers(Mapping.Modifiers, EventData.Value, 0.f);	// Uses EventData.Value to provide the correct EInputActionValueType
	}
}


