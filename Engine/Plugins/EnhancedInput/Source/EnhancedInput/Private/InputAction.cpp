// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputAction.h"

#include "HAL/IConsoleManager.h"
#include "PlayerMappableKeySettings.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputAction)

#define LOCTEXT_NAMESPACE "EnhancedInputAction"

namespace UE::EnhancedInput
{
	static bool bAlwaysGetRealValueFromActionInstanceData = 1;
	static FAutoConsoleVariableRef CVarAlwaysGetRealValueFromActionInstanceData(TEXT("EnhancedInput.bAlwaysGetRealValueFromActionInstanceData"),
		bAlwaysGetRealValueFromActionInstanceData,
		TEXT("If true, then input action instances will always return their true value instead of being conditoinal on the trigger state. Note: This CVar will eventually be removed"));
}

FInputActionInstance::FInputActionInstance(const UInputAction* InSourceAction)
	: SourceAction(InSourceAction)
{
	if (ensureMsgf(SourceAction != nullptr, TEXT("Trying to create an FInputActionInstance without a source action")))
	{
		Value = FInputActionValue(SourceAction->ValueType, FVector::ZeroVector);

		Triggers.Reserve(SourceAction->Triggers.Num());
		for (UInputTrigger* ToDuplicate : SourceAction->Triggers)
		{
			if (ToDuplicate)
			{
				Triggers.Add(DuplicateObject<UInputTrigger>(ToDuplicate, nullptr));
			}
		}

		for (UInputModifier* ToDuplicate : SourceAction->Modifiers)
		{
			if (ToDuplicate)
			{
				Modifiers.Add(DuplicateObject<UInputModifier>(ToDuplicate, nullptr));
			}
		}

	}
}

FInputActionValue FInputActionInstance::GetValue() const
{
	if (UE::EnhancedInput::bAlwaysGetRealValueFromActionInstanceData)
	{
		return Value;
	}
	
	// The legacy behavior only returned the true value when in the "Triggered" state
	// which would result in bool (digital) input actions being false on the "started" phase,
	// which is confusing. 
	return
		TriggerEvent == ETriggerEvent::Triggered ?
		Value :
		FInputActionValue(Value.GetValueType(), FInputActionValue::Axis3D::ZeroVector);
 } 

ETriggerState FInputActionInstance::GetEvaluatedActionTriggerState() const
{
	return TriggerStateTracker.GetState();
}

// Calculate a collective representation of trigger state from evaluations of all triggers in one or more trigger groups.
ETriggerState FTriggerStateTracker::EvaluateTriggers(const UEnhancedPlayerInput* PlayerInput, const TArray<UInputTrigger*>& Triggers, FInputActionValue ModifiedValue, float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(EnhPIS_Triggers);

	// Note: No early outs permitted (e.g. whilst bBlocking)! All triggers must be evaluated to update their internal state/delta time.

	for (UInputTrigger* Trigger : Triggers)
	{
		if (!Trigger)
		{
			continue;
		}

		bEvaluatedTriggers = true;

		ETriggerState CurrentState = Trigger->UpdateState(PlayerInput, ModifiedValue, DeltaTime);

		// Automatically update the last value, avoiding the trigger having to track it.
		Trigger->LastValue = ModifiedValue;

		switch (Trigger->GetTriggerType())
		{
		case ETriggerType::Explicit:
			bFoundExplicit = true;
			bAnyExplictTriggered |= (CurrentState == ETriggerState::Triggered);
			bFoundActiveTrigger |= (CurrentState != ETriggerState::None);
			break;
		case ETriggerType::Implicit:
			bAllImplicitsTriggered &= (CurrentState == ETriggerState::Triggered);
			bFoundActiveTrigger |= (CurrentState != ETriggerState::None);
			break;
		case ETriggerType::Blocker:
			// Ignore dedicated blockers, IsBlocking handles the state update
			break;
		}

		bBlocking |= Trigger->IsBlocking(CurrentState);
	}

	return GetState();
}

ETriggerState FTriggerStateTracker::GetState() const
{
	if(!bEvaluatedTriggers)
	{
		return NoTriggerState;
	}

	if (bBlocking)
	{
		return ETriggerState::None;
	}

	bool bTriggered = ((!bFoundExplicit || bAnyExplictTriggered) && bAllImplicitsTriggered);
	return bTriggered ? ETriggerState::Triggered : (bFoundActiveTrigger ? ETriggerState::Ongoing : ETriggerState::None);
}

// TODO: Hacky. This is the state we should return if we have evaluated no valid triggers. Set during action evaluation based on final ModifiedValue.
void FTriggerStateTracker::SetStateForNoTriggers(ETriggerState State)
{
	NoTriggerState = State;
}

#if WITH_EDITOR
// Record input action property changes for later processing

TSet<const UInputAction*> UInputAction::ActionsWithModifiedValueTypes;
TSet<const UInputAction*> UInputAction::ActionsWithModifiedTriggers;

void UInputAction::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// if our value changes were to the trigger or modifier array broadcast the change
	if (!PropertyChangedEvent.MemberProperty || PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UInputAction, Modifiers))
	{
		OnModifiersChanged.Broadcast();
	}
	if (!PropertyChangedEvent.MemberProperty || PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UInputAction, Triggers))
	{
		OnTriggersChanged.Broadcast();
	}

	// If our value type changes we need to inform any blueprint InputActionEx nodes that refer to this action
	if (!PropertyChangedEvent.MemberProperty || PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UInputAction, ValueType))
	{
		ActionsWithModifiedValueTypes.Add(this);
	}
	if (!PropertyChangedEvent.MemberProperty || PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UInputAction, Triggers))
	{
		ActionsWithModifiedTriggers.Add(this);
	}
}

EDataValidationResult UInputAction::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);

	// Validate the triggers
	bool bContainsComboTrigger = false;
	bool bContainsNonComboTrigger = false;
	for (const UInputTrigger* Trigger : Triggers)
	{
		if (Trigger)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// check if it the trigger is a combo or not
			Trigger->IsA(UInputTriggerCombo::StaticClass()) ? bContainsComboTrigger = true : bContainsNonComboTrigger = true;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			Result = CombineDataValidationResults(Result, Trigger->IsDataValid(Context));
		}
		else
		{
			Result = EDataValidationResult::Invalid;
			Context.AddError(LOCTEXT("NullInputTrigger", "There cannot be a null Input Trigger on an Input Action!"));
		}
	}
	if (bContainsComboTrigger && bContainsNonComboTrigger)
	{
		Result = EDataValidationResult::Invalid;
		Context.AddError(LOCTEXT("ComboAndNonComboInputTrigger", "Combo triggers are not intended to interact with other input triggers. Consider adding the combo and other triggers later in a context or creating a seperate action for the combo."));
	}
	
	// Validate the modifiers
	for (const UInputModifier* Modifier : Modifiers)
	{
		if (Modifier)
		{
			Result = CombineDataValidationResults(Result, Modifier->IsDataValid(Context));
		}
		else
		{
			Result = EDataValidationResult::Invalid;
			Context.AddError(LOCTEXT("NullInputModifier", "There cannot be a null Input Modifier on an Input Action!"));
		}		
	}
	
	// Validate Settings
	if (PlayerMappableKeySettings != nullptr)
	{
		Result = CombineDataValidationResults(Result, PlayerMappableKeySettings->IsDataValid(Context));
	}

	return Result;
}

ETriggerEventsSupported UInputAction::GetSupportedTriggerEvents() const
{
	ETriggerEventsSupported SupportedTriggers = ETriggerEventsSupported::None;

	bool bTriggersAdded = false;
	
	if(!Triggers.IsEmpty())
	{
		for(const UInputTrigger* Trigger : Triggers)
		{
			if(Trigger)
			{
				EnumAddFlags(SupportedTriggers, Trigger->GetSupportedTriggerEvents());
				bTriggersAdded = true;
			}
		}
	}

	// If there are no triggers on an action, then it can be instant (a key is pressed/release) or happening over time(key is held down)
	if(!bTriggersAdded)
	{
		EnumAddFlags(SupportedTriggers, ETriggerEventsSupported::Instant | ETriggerEventsSupported::Uninterruptible);
	}

	return SupportedTriggers;
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
