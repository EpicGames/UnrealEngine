// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputDebugKeyDelegateBinding.h"
#include "EnhancedInputComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputDebugKeyDelegateBinding)

void UInputDebugKeyDelegateBinding::BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const
{
#if DEV_ONLY_KEY_BINDINGS_AVAILABLE
	UEnhancedInputComponent* Component = Cast<UEnhancedInputComponent>(InputComponent);
	if (!Component)
	{
		return;
	}

	for(const FBlueprintInputDebugKeyDelegateBinding& Binding : InputDebugKeyDelegateBindings)
	{
		Component->BindDebugKey(Binding.InputChord, Binding.InputKeyEvent, ObjectToBindTo, Binding.FunctionNameToBind, Binding.bExecuteWhenPaused);
	}
#endif
}

