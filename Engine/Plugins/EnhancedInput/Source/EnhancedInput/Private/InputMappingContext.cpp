// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputMappingContext.h"

#include "EnhancedInputDeveloperSettings.h"
#include "EnhancedInputLibrary.h"
#include "EnhancedInputModule.h"
#include "Misc/DataValidation.h"
#include "PlayerMappableKeySettings.h"
#include "UObject/FortniteSeasonBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputMappingContext)

#define LOCTEXT_NAMESPACE "InputMappingContext"

#if WITH_EDITOR
EDataValidationResult UInputMappingContext::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);

	auto ValidateMapping = [&Result, &Context](const FEnhancedActionKeyMapping& Mapping)
	{
		Result = CombineDataValidationResults(Result, Mapping.IsDataValid(Context));
	};

	// Run validation for every key mapping
	ForEachKeyMapping(ValidateMapping);
	
	return Result;
}
#endif	// WITH_EDITOR

void UInputMappingContext::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteSeasonBranchObjectVersion::GUID);
}

void UInputMappingContext::PostLoad()
{
	Super::PostLoad();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (GetLinkerCustomVersion(FFortniteSeasonBranchObjectVersion::GUID) < FFortniteSeasonBranchObjectVersion::EnhancedInputMappingContextProfileMappingsUpdate)
	{
		// We converted from a TArray<FEnhancedActionKeyMapping> to a struct so that it is consistent with the profile override types
		DefaultKeyMappings.Mappings = MoveTemp(Mappings);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UInputMappingContext::ForEachKeyMapping(TFunctionRef<void(const FEnhancedActionKeyMapping&)>&& Func) const
{
	for (const FEnhancedActionKeyMapping& Mapping : DefaultKeyMappings.Mappings)
	{
		Func(Mapping);
	}
	
	for (const TPair<FString, FInputMappingContextMappingData>& ProfileMapping : MappingProfileOverrides)
	{
		for (const FEnhancedActionKeyMapping& Mapping : ProfileMapping.Value.Mappings)
		{
			Func(Mapping);
		}
	}
}

bool UInputMappingContext::ShouldFilterMappingByInputMode() const
{
	return InputModeFilterOptions != EMappingContextInputModeFilterOptions::DoNotFilter;
}

FGameplayTagQuery UInputMappingContext::GetInputModeQuery() const
{
	switch (InputModeFilterOptions)
	{
	case EMappingContextInputModeFilterOptions::UseProjectDefaultQuery:
		{
			return GetDefault<UEnhancedInputDeveloperSettings>()->DefaultMappingContextInputModeQuery;	
		}
	case EMappingContextInputModeFilterOptions::UseCustomQuery:
		{
			return InputModeQueryOverride;
		}
	case EMappingContextInputModeFilterOptions::DoNotFilter:
	default:
		{
			ensureMsgf(false, TEXT("Unexpected filter options %u, returning default"), (uint8)InputModeFilterOptions);
			return FGameplayTagQuery{};	
		}
	}
}

bool UInputMappingContext::ShouldShowInputModeQuery()
{
	return GetDefault<UEnhancedInputDeveloperSettings>()->bEnableInputModeFiltering;
}

bool UInputMappingContext::HasMappingsForProfile(const FString& ProfileId) const
{
	return MappingProfileOverrides.Find(ProfileId) != nullptr;
}

TArray<FString> UInputMappingContext::GetProfilesWithOverridenMappings() const
{
	TArray<FString> Profiles;
	MappingProfileOverrides.GetKeys(Profiles);
	
	return Profiles;
}

const TArray<FEnhancedActionKeyMapping>& UInputMappingContext::GetMappingsForProfile(const FString& ProfileId) const
{
	if (const FInputMappingContextMappingData* OverrideMappings = MappingProfileOverrides.Find(ProfileId))
	{
		return OverrideMappings->Mappings;
	}
	
	return DefaultKeyMappings.Mappings;
}

bool UInputMappingContext::HasMappingForInputAction(const UInputAction* Action) const
{
	if (!Action)
	{
		return false;
	}

	auto HasReferencesToMapping = [Action](const FEnhancedActionKeyMapping& Mapping) -> bool
	{
		return Mapping.Action == Action;
	};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Mappings.ContainsByPredicate(HasReferencesToMapping))
	{
		return true;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	if (DefaultKeyMappings.Mappings.ContainsByPredicate(HasReferencesToMapping))
	{
		return true;
	}

	for (const TPair<FString, FInputMappingContextMappingData>& OverrideMappings : MappingProfileOverrides)
	{
		if (OverrideMappings.Value.Mappings.ContainsByPredicate(HasReferencesToMapping))
		{
			return true;
		}
	}

	return false;
}

FEnhancedActionKeyMapping& UInputMappingContext::MapKey(const UInputAction* Action, FKey ToKey)
{
	IEnhancedInputModule::Get().GetLibrary()->RequestRebuildControlMappingsUsingContext(this);
	return DefaultKeyMappings.Mappings.Add_GetRef(FEnhancedActionKeyMapping(Action, ToKey));
}

void UInputMappingContext::UnmapKey(const UInputAction* Action, FKey Key)
{
	int32 MappingIdx = DefaultKeyMappings.Mappings.IndexOfByPredicate([&Action, &Key](const FEnhancedActionKeyMapping& Other) { return Other.Action == Action && Other.Key == Key; });
	if (MappingIdx != INDEX_NONE)
	{
		DefaultKeyMappings.Mappings.RemoveAtSwap(MappingIdx);
		IEnhancedInputModule::Get().GetLibrary()->RequestRebuildControlMappingsUsingContext(this);
	}
}

void UInputMappingContext::UnmapAllKeysFromAction(const UInputAction* Action)
{
	int32 Found = DefaultKeyMappings.Mappings.RemoveAllSwap([&Action](const FEnhancedActionKeyMapping& Mapping) { return Mapping.Action == Action; });
	if (Found > 0)
	{
		IEnhancedInputModule::Get().GetLibrary()->RequestRebuildControlMappingsUsingContext(this);
	}
}

void UInputMappingContext::UnmapAll()
{
	if (DefaultKeyMappings.Mappings.Num())
	{
		DefaultKeyMappings.Mappings.Empty();
		IEnhancedInputModule::Get().GetLibrary()->RequestRebuildControlMappingsUsingContext(this);
	}
}

#undef LOCTEXT_NAMESPACE
