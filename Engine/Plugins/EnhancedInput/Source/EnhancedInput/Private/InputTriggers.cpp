// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputTriggers.h"

#include "EnhancedInputModule.h"
#include "EnhancedPlayerInput.h"
#include "HAL/IConsoleManager.h"
#include "Misc/DataValidation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputTriggers)

#define LOCTEXT_NAMESPACE "EnhancedInputTriggers"

namespace UE::EnhancedInput
{
	static TAutoConsoleVariable<bool> CVarCheckInitialStateForComboTrigger
	(
		TEXT("EnhancedInput.Triggers.bCheckInitalStateForComboTrigger"),
		false,
		TEXT("When true, combo triggers will check initial state (First input action in the combo array) and return 'Ongoing' if the Combo Step Completion State is met. ")
		TEXT("Note: Setting this to true was Combo trigger behavior as of 5.4 and before."),
		ECVF_Default
	);
}

namespace UE::Input
{
	FString LexToString(const ETriggerEvent TriggerEvent)
	{
		if (TriggerEvent == ETriggerEvent::None)
		{
			return TEXT("None");
		}

		FString Result = TEXT("");
#define TRIGGER_STATE(StatusFlag, DisplayName) if( EnumHasAllFlags(TriggerEvent, StatusFlag) ) Result += (FString(DisplayName) + TEXT("|"));
		TRIGGER_STATE(ETriggerEvent::Triggered, TEXT("Triggered"));
		TRIGGER_STATE(ETriggerEvent::Started, TEXT("Started"));
		TRIGGER_STATE(ETriggerEvent::Ongoing, TEXT("Ongoing"));
		TRIGGER_STATE(ETriggerEvent::Canceled, TEXT("Canceled"));
		TRIGGER_STATE(ETriggerEvent::Completed, TEXT("Completed"));
#undef TRIGGER_STATE

		Result.RemoveFromEnd(TEXT("|"));
		return Result;
	}

	static FString LexToString(const ETriggerState State)
	{
		switch (State)
		{
		case ETriggerState::Triggered: return TEXT("Triggered");
		case ETriggerState::Ongoing: return TEXT("Ongoing");
		case ETriggerState::None: return TEXT("None");
		}

		return TEXT("Invalid");
	}
}


// Abstract trigger bases
ETriggerState UInputTrigger::UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime)
{
	return IsActuated(ModifiedValue) ? ETriggerState::Triggered : ETriggerState::None;
};

void UInputTrigger::OnTriggerReinstanced(const UInputTrigger* OldTrigger)
{
	if (!OldTrigger)
	{
		return;
	}

	// The old trigger must be guaranteed to be the same class
	check(OldTrigger->GetClass() == GetClass());
	
	LastValue = OldTrigger->LastValue;
	ReceiveTriggerReinstanced(OldTrigger);
}

bool UInputTrigger::IsSupportedTriggerEvent(const ETriggerEventsSupported SupportedEvents, const ETriggerEvent Event)
{
	if(SupportedEvents == ETriggerEventsSupported::All)
	{
		return true;
	}
	else if(SupportedEvents == ETriggerEventsSupported::None)
	{
		return false;
	}
	
	// Check the bitmask of SupportedEvent types for each ETriggerEvent
	switch (Event)
	{
	case ETriggerEvent::Started:
		return EnumHasAnyFlags(SupportedEvents, ETriggerEventsSupported::Uninterruptible | ETriggerEventsSupported::Ongoing);
	case ETriggerEvent::Ongoing:
		return EnumHasAnyFlags(SupportedEvents, ETriggerEventsSupported::Uninterruptible | ETriggerEventsSupported::Ongoing);
	case ETriggerEvent::Canceled:
		return EnumHasAnyFlags(SupportedEvents, ETriggerEventsSupported::Ongoing);
	// Triggered can happen from Instant, Overtime, or Cancelable trigger events.
	case ETriggerEvent::Triggered:
		return EnumHasAnyFlags(SupportedEvents, (ETriggerEventsSupported::Instant | ETriggerEventsSupported::Uninterruptible | ETriggerEventsSupported::Ongoing));
		// Completed is supported by every UInputTrigger
	case ETriggerEvent::Completed:
		return EnumHasAnyFlags(SupportedEvents, ETriggerEventsSupported::All);
	case ETriggerEvent::None:
	default:
		return false;
	}	
}

ETriggerState UInputTriggerTimedBase::UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime)
{
	ETriggerState State = ETriggerState::None;

	// Transition to Ongoing on actuation. Update the held duration.
	if (IsActuated(ModifiedValue))
	{
		State = ETriggerState::Ongoing;
		HeldDuration = CalculateHeldDuration(PlayerInput, DeltaTime);
	}
	else
	{
		// Reset duration
		HeldDuration = 0.0f;
	}

	return State;
}

void UInputTriggerTimedBase::ReceiveTriggerReinstanced_Implementation(const UInputTrigger* OldTrigger)
{
	if (const UInputTriggerTimedBase* OldTimed = CastChecked<UInputTriggerTimedBase>(OldTrigger))
	{
		HeldDuration = OldTimed->HeldDuration;
	}
}

float UInputTriggerTimedBase::CalculateHeldDuration(const UEnhancedPlayerInput* const PlayerInput, const float DeltaTime) const
{
	// We may not have a PlayerInput object during automation tests, so default to 1.0f if we don't have one.
	// This will mean that TimeDilation has no effect.
	const float TimeDilation = PlayerInput ? PlayerInput->GetEffectiveTimeDilation() : 1.0f;
	
	// Calculates the new held duration, applying time dilation if desired
	return HeldDuration + (!bAffectedByTimeDilation ? DeltaTime : DeltaTime * TimeDilation);
}


// Implementations

ETriggerState UInputTriggerDown::UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime)
{
	// Triggered on down.
	return IsActuated(ModifiedValue) ? ETriggerState::Triggered : ETriggerState::None;
}

ETriggerState UInputTriggerPressed::UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime)
{
	// Triggered on transition to actuated.
	return IsActuated(ModifiedValue) && !IsActuated(LastValue) ? ETriggerState::Triggered : ETriggerState::None;
}

ETriggerState UInputTriggerReleased::UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime)
{
	// Ongoing on hold
	if (IsActuated(ModifiedValue))
	{
		return ETriggerState::Ongoing;
	}

	// Triggered on release
	if (IsActuated(LastValue))
	{
		return ETriggerState::Triggered;
	}

	return ETriggerState::None;
}

ETriggerState UInputTriggerHold::UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime)
{
	// Update HeldDuration and derive base state
	ETriggerState State = Super::UpdateState_Implementation(PlayerInput, ModifiedValue, DeltaTime);

	// Trigger when HeldDuration reaches the threshold
	bool bIsFirstTrigger = !bTriggered;
	bTriggered = HeldDuration >= HoldTimeThreshold;
	if (bTriggered)
	{
		return (bIsFirstTrigger || !bIsOneShot) ? ETriggerState::Triggered : ETriggerState::None;
	}

	return State;
}

void UInputTriggerHold::ReceiveTriggerReinstanced_Implementation(const UInputTrigger* OldTrigger)
{
	Super::ReceiveTriggerReinstanced_Implementation(OldTrigger);
	
	if (const UInputTriggerHold* OldHold = CastChecked<UInputTriggerHold>(OldTrigger))
	{
		bTriggered = OldHold->bTriggered;
	}
}

ETriggerState UInputTriggerHoldAndRelease::UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime)
{
	// Evaluate the updated held duration prior to calling Super to update the held timer
	// This stops us failing to trigger if the input is released on the threshold frame due to HeldDuration being 0.
	const float TickHeldDuration = CalculateHeldDuration(PlayerInput, DeltaTime);

	// Update HeldDuration and derive base state
	ETriggerState State = Super::UpdateState_Implementation(PlayerInput, ModifiedValue, DeltaTime);

	// Trigger if we've passed the threshold and released
	if (TickHeldDuration >= HoldTimeThreshold && State == ETriggerState::None)
	{
		State = ETriggerState::Triggered;
	}

	return State;
}

ETriggerState UInputTriggerTap::UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime)
{
	float LastHeldDuration = HeldDuration;

	// Updates HeldDuration
	ETriggerState State = Super::UpdateState_Implementation(PlayerInput, ModifiedValue, DeltaTime);

	// Only trigger if pressed then released quickly enough
	if (IsActuated(LastValue) && State == ETriggerState::None && LastHeldDuration < TapReleaseTimeThreshold)
	{
		State = ETriggerState::Triggered;
	}
	else if (HeldDuration >= TapReleaseTimeThreshold)
	{
		// Once we pass the threshold halt all triggering until released
		State = ETriggerState::None;
	}

	return State;
}

UInputTriggerRepeatedTap::UInputTriggerRepeatedTap()
{
	// The time window between taps is the whole point of this trigger. Without always-tick,
	// UEnhancedPlayerInput::ProcessActionMappingEvent skips evaluation on frames where the key
	// has no event, and the action state collapses from Ongoing to None between taps — firing
	// a spurious Canceled and resetting NumberOfTapsSinceLastTrigger. See UE-226494.
	bShouldAlwaysTick = true;
}

ETriggerState UInputTriggerRepeatedTap::UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime)
{
	const float LastHeldDuration = HeldDuration;

	// Updates the held duration.
	// This will return "none" if the key is not currently actuated, and "ongoing" if the key is currently being held down.
	ETriggerState State = Super::UpdateState_Implementation(PlayerInput, ModifiedValue, DeltaTime);
	
	const bool bHasSingleTap = 
		IsActuated(LastValue) &&						// If the key was actuated last frame...
		State == ETriggerState::None &&					// and it now is not...
		LastHeldDuration < TapReleaseTimeThreshold;		// and the amount of time that they key was held down before it was released is within our "single tap" time threshold...	
	
	const double CurrentTime = FPlatformTime::Seconds();

	// Is the amount of time which has passed within the acceptable time frame to be considered a "repeat" tap? 
	const bool bIsWithinValidRepeatTimeRange = CurrentTime <= RepeatTime;

	// Preserve Ongoing between taps so a pending repeat doesn't bounce through None
	// (which would emit a spurious Canceled). Requires bShouldAlwaysTick set in the
	// constructor so the pipeline keeps evaluating us on frames where the key is idle.
	// bAwaitingRepeatTap covers the window after the first tap — NumberOfTapsSinceLastTrigger
	// is only incremented from the second tap onward, so the counter alone can't carry
	// Ongoing across the gap before the first increment.
	State = (NumberOfTapsSinceLastTrigger > 0 || bAwaitingRepeatTap || (bHasSingleTap && IsActuated(LastValue))) ? ETriggerState::Ongoing : State;

	// If too much time has passed to be within a valid repeat range, and a repeat sequence is pending,
	// then we need to cancel this trigger.
	if (!bIsWithinValidRepeatTimeRange && (NumberOfTapsSinceLastTrigger > 0 || bAwaitingRepeatTap))
	{
		State = ETriggerState::None;
	}
	// If the key is currently being held down for longer then is allowed for a single "tap", then we also need to cancel.
	else if (HeldDuration >= TapReleaseTimeThreshold)
	{
		State = ETriggerState::None;
	}
	// Otherwise, if we have detected a single tap and are within the allowed time range for repeating a tap, increment
	// our tap count and see if we should trigger.
	else if (bHasSingleTap && bIsWithinValidRepeatTimeRange)
	{
		++NumberOfTapsSinceLastTrigger;
		const bool bHasReachedRepeatThreshold = NumberOfTapsSinceLastTrigger >= (NumberOfTapsWhichTriggerRepeat - 1);

		if (bHasReachedRepeatThreshold)
		{
			State = ETriggerState::Triggered;
		}
		else
		{
			State = ETriggerState::Ongoing;
		}
	}

	// Keep track of the next time range which is acceptable for another repeat, and mark that a
	// repeat sequence is now in progress so idle frames between taps stay Ongoing.
	if (bHasSingleTap)
	{
		RepeatTime = CurrentTime + RepeatDelay;
		bAwaitingRepeatTap = true;
	}

	// If the trigger has been cancelled somehow, reset the number of taps we have
	if (State == ETriggerState::None || State == ETriggerState::Triggered)
	{
		NumberOfTapsSinceLastTrigger = 0;
		bAwaitingRepeatTap = false;
	}
	
	UE_LOGF(LogEnhancedInput, Verbose, "Repeated Tap InputState:  %d / %d taps :: %ls", NumberOfTapsSinceLastTrigger, NumberOfTapsWhichTriggerRepeat, *UE::Input::LexToString(State));
	
	return State;
}

FString UInputTriggerRepeatedTap::GetDebugState() const
{
	return FString::Printf(TEXT("Repeated Taps:%d/%d"), NumberOfTapsSinceLastTrigger, NumberOfTapsWhichTriggerRepeat);
}

void UInputTriggerPulse::ReceiveTriggerReinstanced_Implementation(const UInputTrigger* OldTrigger)
{
	Super::ReceiveTriggerReinstanced_Implementation(OldTrigger);
	
	if (const UInputTriggerPulse* OldPulse = CastChecked<UInputTriggerPulse>(OldTrigger))
	{
		TriggerCount = OldPulse->TriggerCount;
	}
}

ETriggerState UInputTriggerPulse::UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime)
{
	// Update HeldDuration and derive base state
	ETriggerState State = Super::UpdateState_Implementation(PlayerInput, ModifiedValue, DeltaTime);

	if (State == ETriggerState::Ongoing)
	{
		// If the repeat count limit has not been reached
		if (TriggerLimit == 0 || TriggerCount < TriggerLimit)
		{
			// Trigger when HeldDuration exceeds the interval threshold, optionally trigger on initial actuation
			if (HeldDuration > (Interval * (bTriggerOnStart ? TriggerCount : TriggerCount + 1)))
			{
				++TriggerCount;
				State = ETriggerState::Triggered;
			}
		}
		else
		{
			State = ETriggerState::None;
		}
	}
	else
	{
		// Reset repeat count
		TriggerCount = 0;
	}

	return State;
}

#if WITH_EDITOR
EDataValidationResult UInputTriggerChordAction::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);
	
	// You can't evaluate the combo if there are no combo steps!
	if (!ChordAction)
	{
		Result = EDataValidationResult::Invalid;
		Context.AddError(LOCTEXT("NullChordedAction", "A valid action is required for the Chorded Action input trigger!"));
	}

	return Result;
}
#endif

ETriggerState UInputTriggerChordAction::UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime)
{
	// Inherit state from the chorded action
	const FInputActionInstance* EventData = PlayerInput->FindActionInstanceData(ChordAction);
	return EventData ? EventData->GetEvaluatedActionTriggerState() : ETriggerState::None;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

UInputTriggerCombo::UInputTriggerCombo()
{
	bShouldAlwaysTick = true;
}

ETriggerState UInputTriggerCombo::UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime)
{
	if (ComboActions.IsEmpty())
	{
		UE_LOGF(LogEnhancedInput, Warning, "A Combo Trigger has no combo actions and will not work properly!");
		return ETriggerState::None;
	}
	
	if (const UInputAction* CurrentAction = ComboActions[CurrentComboStepIndex].ComboStepAction)
	{
		// loop through all cancel actions and check if they've fired
		for (FInputCancelAction InputCancelActionData : InputCancelActions)
		{
			const UInputAction* CancelAction = InputCancelActionData.CancelAction;
			if (CancelAction && CancelAction != CurrentAction)
			{
				const FInputActionInstance* CancelState = PlayerInput->FindActionInstanceData(CancelAction);
				// Check the cancellation state against the states that should cancel the combo
				if (CancelState && (InputCancelActionData.CancellationStates & static_cast<uint8>(CancelState->GetTriggerEvent())))
				{
					// Cancel action firing!
					CurrentComboStepIndex = 0;
					CurrentAction = ComboActions[CurrentComboStepIndex].ComboStepAction;	// Reset for fallthrough
					break;
				}
			}
		}
		// loop through all combo actions and check if a combo action fired out of order
		for (FInputComboStepData ComboStep : ComboActions)
		{
			if (ComboStep.ComboStepAction && ComboStep.ComboStepAction != CurrentAction)
			{
				const FInputActionInstance* CancelState = PlayerInput->FindActionInstanceData(ComboStep.ComboStepAction);
				// Check the combo action state against the states that should complete this step
				if (CancelState && (ComboStep.ComboStepCompletionStates & static_cast<uint8>(CancelState->GetTriggerEvent())))
				{
					// Other combo action firing - should cancel
					CurrentComboStepIndex = 0;
					CurrentAction = ComboActions[CurrentComboStepIndex].ComboStepAction;	// Reset for fallthrough
					break;
				}
			}
		}

		// Reset if we take too long to hit the action
		if (CurrentComboStepIndex > 0)
		{
			CurrentTimeBetweenComboSteps += DeltaTime;
			if (CurrentTimeBetweenComboSteps >= ComboActions[CurrentComboStepIndex].TimeToPressKey)
			{
				CurrentComboStepIndex = 0;
				CurrentAction = ComboActions[CurrentComboStepIndex].ComboStepAction;	// Reset for fallthrough			
			}
		}

		const FInputActionInstance* CurrentState = PlayerInput->FindActionInstanceData(CurrentAction);
		// check to see if current action is in one of it's completion states - if so advance the combo to the next combo action
		if (CurrentState && (ComboActions[CurrentComboStepIndex].ComboStepCompletionStates & static_cast<uint8>(CurrentState->GetTriggerEvent())))
		{
			CurrentComboStepIndex++;
			CurrentTimeBetweenComboSteps = 0;
			// check to see if we've completed all actions in the combo
			if (CurrentComboStepIndex >= ComboActions.Num())
			{
				CurrentComboStepIndex = 0;
				return ETriggerState::Triggered;
			}
		}

		if (CurrentComboStepIndex > 0)
		{
			return ETriggerState::Ongoing;
		}

		if (UE::EnhancedInput::CVarCheckInitialStateForComboTrigger.GetValueOnAnyThread())
		{
			// Really should account for first combo action being mid-trigger...
			const FInputActionInstance* InitialState = PlayerInput->FindActionInstanceData(ComboActions[0].ComboStepAction);
			if (InitialState && InitialState->GetTriggerEvent() > ETriggerEvent::None) // || Cancelled!
			{
				return ETriggerState::Ongoing;
			}
		}
		
		CurrentTimeBetweenComboSteps = 0;
	}
	return ETriggerState::None;
};

#if WITH_EDITOR
EDataValidationResult UInputTriggerCombo::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);

	Context.AddWarning(LOCTEXT("ComboTriggerDeprecated", "The input combo trigger is deprecated and will be removed in a future release"));
	
	// You can't evaluate the combo if there are no combo steps!
	if (ComboActions.IsEmpty())
	{
		Result = EDataValidationResult::Invalid;
		Context.AddError(LOCTEXT("NoComboSteps", "There must be at least one combo step in the Combo Trigger!"));
	}

	// Making sure combo completion states have at least one state
	for (const FInputComboStepData& ComboStep : ComboActions)
	{
		if (ComboStep.ComboStepCompletionStates == 0)
		{
			Result = EDataValidationResult::Invalid;
			Context.AddError(FText::Format(LOCTEXT("NoCompletionStates", "There must be at least one completion state in ComboStep Completion States in the {0} combo step in order to progress the combo!"), FText::FromString(ComboStep.ComboStepAction.GetName())));
		}
	}

	// Making sure cancellation states have at least one state
	for (const FInputCancelAction& CancelAction : InputCancelActions)
	{
		if (CancelAction.CancellationStates == 0)
		{
			Result = EDataValidationResult::Invalid;
			Context.AddError(FText::Format(LOCTEXT("NoCancellationStates", "There must be at least one cancellation state in Cancellation States in the {0} cancel action in order to cancel the combo!"), FText::FromString(CancelAction.CancelAction.GetName())));
		}
	}

	return Result;
}
#endif // WITH_EDITOR

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
