// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerMappableInputConfig.h"
#include "InputMappingContext.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayerMappableInputConfig)

#define LOCTEXT_NAMESPACE "PlayerMappableInputConfig"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

UPlayerMappableInputConfig::UPlayerMappableInputConfig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ConfigName(GetFName())
	, ConfigDisplayName()
{
	ResetToDefault();
}

void UPlayerMappableInputConfig::ResetToDefault()
{
}

#if WITH_EDITOR

EDataValidationResult UPlayerMappableInputConfig::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	if(ConfigName == NAME_None)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("AssetPath"), FText::FromString(GetPathName()));
		const FText NoNameError = FText::Format(LOCTEXT("NoNameError", "'{AssetPath}' does not have a valid ConfigName!"), Args);
		Context.AddError(NoNameError);
		
		Result = CombineDataValidationResults(Result, EDataValidationResult::Invalid);
	}
	
	// Check that every context is valid and all player mappable mappings have names
	for (const TPair<TObjectPtr<UInputMappingContext>, int32>& ContextPair : Contexts)
	{
		if (UInputMappingContext* IMC = ContextPair.Key.Get())
		{
			Result = CombineDataValidationResults(Result, IMC->IsDataValid(Context));
			if (Result == EDataValidationResult::Invalid)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("AssetPath"), FText::FromString(IMC->GetPathName()));
				const FText NoMappingError = FText::Format(LOCTEXT("NoNamePlayerMappingError", "'{AssetPath}' has an invalid mapping."), Args);
				Context.AddError(NoMappingError);
			}
		}
		else
		{
			Result = CombineDataValidationResults(Result, EDataValidationResult::Invalid);
			
			FFormatNamedArguments Args;
			Args.Add(TEXT("AssetPath"), FText::FromString(GetPathName()));

			const FText NullContextError = FText::Format(LOCTEXT("NullContextError", "There is a null mapping context in '{AssetPath}'"), Args);
			Context.AddError(NullContextError);
		}
	}
	
	return CombineDataValidationResults(Result, EDataValidationResult::Valid);
}

#endif

void UPlayerMappableInputConfig::ForEachDefaultPlayerMappableKey(TFunctionRef<void(const FEnhancedActionKeyMapping&)> Operation) const
{
	for (const TPair<TObjectPtr<UInputMappingContext>, int32>& ContextPair: Contexts)
	{
		if(const UInputMappingContext* IMC = ContextPair.Key.Get())
		{
			for(const FEnhancedActionKeyMapping& Mapping : IMC->GetMappings())
			{
				if(Mapping.IsPlayerMappable())
				{
					Operation(Mapping);
				}
			}
		}
	}
}

TArray<FEnhancedActionKeyMapping> UPlayerMappableInputConfig::GetPlayerMappableKeys() const
{
	TArray<FEnhancedActionKeyMapping> OutMappings;
	auto GatherMappings = [&OutMappings](const FEnhancedActionKeyMapping& Options)
	{
		OutMappings.Add(Options);
	};
	
	ForEachDefaultPlayerMappableKey(GatherMappings);

	return OutMappings;
}

FEnhancedActionKeyMapping UPlayerMappableInputConfig::GetMappingByName(const FName MappingName) const
{
	FEnhancedActionKeyMapping OutMapping;
	
	TArray<FEnhancedActionKeyMapping> MappableKeys = GetPlayerMappableKeys();
	FEnhancedActionKeyMapping* ExistingMapping = MappableKeys.FindByPredicate([MappingName](const FEnhancedActionKeyMapping& Mapping) { return Mapping.GetMappingName() == MappingName; });

	if(ExistingMapping)
	{
		OutMapping = *ExistingMapping;
	}
	return OutMapping;
}

TArray<FEnhancedActionKeyMapping> UPlayerMappableInputConfig::GetKeysBoundToAction(const UInputAction* InAction) const
{
	TArray<FEnhancedActionKeyMapping> OutMappings;
	auto GatherMappings = [&OutMappings, InAction](const FEnhancedActionKeyMapping& Options)
	{
		if(Options.Action == InAction)
		{
			OutMappings.Add(Options);
		}
	};
	
	ForEachDefaultPlayerMappableKey(GatherMappings);
	
	return OutMappings;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE