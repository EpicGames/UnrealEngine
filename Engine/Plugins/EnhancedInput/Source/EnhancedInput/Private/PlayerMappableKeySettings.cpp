// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerMappableKeySettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayerMappableKeySettings)

#define LOCTEXT_NAMESPACE "EnhancedActionKeySetting"

#if WITH_EDITOR

#include "Misc/DataValidation.h"
#include "UObject/UObjectIterator.h"

EDataValidationResult UPlayerMappableKeySettings::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);
	if (Name == NAME_None)
	{
		Result = EDataValidationResult::Invalid;
		Context.AddError(LOCTEXT("InvalidPlayerMappableKeySettingsName", "A Player Mappable Key Settings must have a valid 'Name'"));
	}
	return Result;
}

#if WITH_EDITOR
void UPlayerMappableKeySettings::PostLoad()
{
	UObject::PostLoad();

	// Move any gameplay tag profiles into the new string based system
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!SupportedKeyProfiles.IsEmpty())
	{
		for (const FGameplayTag& Tag : SupportedKeyProfiles)
		{
			SupportedKeyProfileIds.Add(Tag.ToString());
		}

		// Empty the old data so it doesn't happen every time we run post load.
		SupportedKeyProfiles = FGameplayTagContainer::EmptyContainer;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif

const TArray<FName>& UPlayerMappableKeySettings::GetKnownMappingNames()
{
	static TArray<FName> OutNames;
	OutNames.Reset();
	
    for (TObjectIterator<UPlayerMappableKeySettings> Itr; Itr; ++Itr)
    {
    	if (IsValid(*Itr))
    	{
    		OutNames.Add(Itr->Name);	
    	}
    }

    return OutNames;
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
