// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserSettings/EnhancedInputUserSettings.h"

#include "Algo/Find.h"
#include "DrawDebugHelpers.h"	// required for ENABLE_DRAW_DEBUG define
#include "Engine/Engine.h"		// for GEngine definition
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "EnhancedActionKeyMapping.h"
#include "EnhancedInputDeveloperSettings.h"
#include "EnhancedInputLibrary.h"
#include "EnhancedInputModule.h"
#include "EnhancedInputSubsystems.h"
#include "GameFramework/InputSettings.h"
#include "HAL/IConsoleManager.h"
#include "InputAction.h"
#include "InputMappingContext.h"
#include "Internationalization/Text.h"
#include "Kismet/GameplayStatics.h"
#include "NativeGameplayTags.h"
#include "PlayerMappableKeySettings.h"
#include "SaveGameSystem.h"
#include "TimerManager.h"

#if ENABLE_DRAW_DEBUG
#include "Engine/Canvas.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedInputUserSettings)

#define LOCTEXT_NAMESPACE "EnhancedInputMappableUserSettings"

namespace UE::EnhancedInput
{
	UE_DEFINE_GAMEPLAY_TAG_STATIC(TAG_DefaultProfileIdentifier, "InputUserSettings.Profiles.Default");
	
	static const FString DefaultProfileIdentifierString = TEXT("InputUserSettings.Profiles.Default");
	
	static const FText DefaultProfileDisplayName = LOCTEXT("Default_Profile_name", "Default Profile");
	
	UE_DEFINE_GAMEPLAY_TAG_STATIC(TAG_InvalidMappingName, "InputUserSettings.FailureReasons.InvalidMappingName");
	UE_DEFINE_GAMEPLAY_TAG_STATIC(TAG_NoKeyProfile, "InputUserSettings.FailureReasons.NoKeyProfile");
	UE_DEFINE_GAMEPLAY_TAG_STATIC(TAG_NoMatchingMappings, "InputUserSettings.FailureReasons.NoMatchingMappings");
	UE_DEFINE_GAMEPLAY_TAG_STATIC(TAG_NoMappingRowFound, "InputUserSettings.FailureReasons.NoMappingRowFound");

	static void DumpAllKeyProfilesToLog(const TArray<FString>& Args)
	{
		// Dump every local player subsystem's logs
		UEnhancedInputLibrary::ForEachSubsystem([Args](IEnhancedInputSubsystemInterface* Subsystem)
		{
			if (const UEnhancedInputUserSettings* Settings = Subsystem->GetUserSettings())
			{
				if (const UEnhancedPlayerMappableKeyProfile* Profile = Settings->GetActiveKeyProfile())
				{
					Profile->DumpProfileToLog();	
				}
			}
		});
	}
	
	static void SaveAllKeyProfilesToSlot(const TArray<FString>& Args)
	{
		UEnhancedInputLibrary::ForEachSubsystem([Args](IEnhancedInputSubsystemInterface* Subsystem)
		{
			if (UEnhancedInputUserSettings* Settings = Subsystem->GetUserSettings())
			{
				Settings->AsyncSaveSettings();
			}
		});
	}
	
	static void SetActiveKeyProfile(const TArray<FString>& Args)
	{
		if (!Args.IsValidIndex(0))
		{
			UE_LOGF(LogEnhancedInput, Error, "EnhancedInput.SetActiveKeyProfile command failed - you must provide a desired key profile name! Example usage:\n\t EnhancedInput.SetActiveKeyProfile PresetA");
			return;
		}

		const FString& DesiredKeyProfile = Args[0];
		
		UEnhancedInputLibrary::ForEachSubsystem([DesiredKeyProfile](IEnhancedInputSubsystemInterface* Subsystem)
		{
			if (UEnhancedInputUserSettings* Settings = Subsystem->GetUserSettings())
			{
				Settings->SetActiveKeyProfile(DesiredKeyProfile);
			}
		});
	}
	
	static FAutoConsoleCommand ConsoleCommandDumpProfileToLog(
		TEXT("EnhancedInput.DumpKeyProfileToLog"),
		TEXT(""),
		FConsoleCommandWithArgsDelegate::CreateStatic(UE::EnhancedInput::DumpAllKeyProfilesToLog));

	static FAutoConsoleCommand ConsoleCommandDumpSaveKeyProfilesToSlot(
		TEXT("EnhancedInput.SaveKeyProfilesToSlot"),
		TEXT("Save the user input settings object with the Save Game to slot system"),
		FConsoleCommandWithArgsDelegate::CreateStatic(UE::EnhancedInput::SaveAllKeyProfilesToSlot));

	static FAutoConsoleCommand ConsoleCommandSetKeyProfile(
		TEXT("EnhancedInput.SetActiveKeyProfile"),
		TEXT("Sets the current Enhanced Input user key profile to the one provided."),
		FConsoleCommandWithArgsDelegate::CreateStatic(UE::EnhancedInput::SetActiveKeyProfile));
}

///////////////////////////////////////////////////////////
// FMapPlayerKeyArgs

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMapPlayerKeyArgs::FMapPlayerKeyArgs()
	: MappingName(NAME_None)
	, Slot(EPlayerMappableKeySlot::Unspecified)
	, NewKey(EKeys::Invalid)
	, HardwareDeviceId(NAME_None)
	, bCreateMatchingSlotIfNeeded(true)
	, bDeferOnSettingsChangedBroadcast (false)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

///////////////////////////////////////////////////////////
// FPlayerKeyMapping

FPlayerKeyMapping::FPlayerKeyMapping()
	: MappingName(NAME_None)
	, DisplayName(FText::GetEmpty())
	, DisplayCategory(FText::GetEmpty())
	, Slot(EPlayerMappableKeySlot::Unspecified)
	, bIsDirty(false)
	, DefaultKey(EKeys::Invalid)
	, CurrentKey(EKeys::Invalid)
	, HardwareDeviceId(FHardwareDeviceIdentifier::Invalid)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, AssociatedInputAction(nullptr)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	, AssociatedInputActionSoft(nullptr)
{
}

FPlayerKeyMapping::FPlayerKeyMapping(
	const FEnhancedActionKeyMapping& OriginalMapping,
	EPlayerMappableKeySlot InSlot /* = EPlayerMappableKeySlot::Unspecified */,
	const FHardwareDeviceIdentifier& InHardwareDevice /* = FHardwareDeviceIdentifier::Invalid */)
	: MappingName(OriginalMapping.GetMappingName())
	, DisplayName(OriginalMapping.GetDisplayName())
	, DisplayCategory(OriginalMapping.GetDisplayCategory())
	, Slot(InSlot)
	, bIsDirty(false)
	, DefaultKey(OriginalMapping.Key)
	, CurrentKey(OriginalMapping.Key)
	, HardwareDeviceId(InHardwareDevice)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, AssociatedInputAction(nullptr)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	, AssociatedInputActionSoft(OriginalMapping.Action)
{
}

FPlayerKeyMapping::FPlayerKeyMapping(const FPlayerKeyMapping& Other)
{
	MappingName = Other.MappingName;
	DisplayName = Other.DisplayName;
	DisplayCategory = Other.DisplayCategory;
	Slot = Other.Slot;
	bIsDirty = Other.bIsDirty;
	DefaultKey = Other.DefaultKey;
	CurrentKey = Other.CurrentKey;
	HardwareDeviceId = Other.HardwareDeviceId;
	AssociatedInputActionSoft = Other.AssociatedInputActionSoft;
}

// The default constructor creates an invalid mapping. Use this as a way to return references
// to an invalid mapping for BP functions
FPlayerKeyMapping FPlayerKeyMapping::InvalidMapping = FPlayerKeyMapping();

bool FPlayerKeyMapping::IsCustomized() const
{
	return (CurrentKey != DefaultKey);
}

bool FPlayerKeyMapping::IsValid() const
{
	return MappingName.IsValid() && CurrentKey.IsValid();
}

const FKey& FPlayerKeyMapping::GetCurrentKey() const
{
	return IsCustomized() ? CurrentKey : DefaultKey;
}

const FKey& FPlayerKeyMapping::GetDefaultKey() const
{
	return DefaultKey;
}

FString FPlayerKeyMapping::ToString() const
{
	const UEnum* PlayerMappableEnumClass = StaticEnum<EPlayerMappableKeySlot>();
	check(PlayerMappableEnumClass);
	return
		FString::Printf(TEXT("Mapping Name: '%s'  Slot: '%s'  Default Key: '%s'  Player Mapped Key: '%s'  HardwareDevice:  '%s'   AssociatedInputAction: '%s'"),
			*MappingName.ToString(),
			*PlayerMappableEnumClass->GetNameStringByValue(static_cast<int64>(Slot)),
			*DefaultKey.ToString(),
			*CurrentKey.ToString(),
			*HardwareDeviceId.ToString(),
			AssociatedInputActionSoft.IsValid() ? *AssociatedInputActionSoft.Get()->GetFName().ToString() : TEXT("NULL INPUT ACTION"));
}

const FName FPlayerKeyMapping::GetMappingName() const
{
	return MappingName;
}

const FText& FPlayerKeyMapping::GetDisplayName() const
{
	return DisplayName;
}

const FText& FPlayerKeyMapping::GetDisplayCategory() const
{
	return DisplayCategory;
}

EPlayerMappableKeySlot FPlayerKeyMapping::GetSlot() const
{
	return Slot;
}

const FHardwareDeviceIdentifier& FPlayerKeyMapping::GetHardwareDeviceId() const
{
	return HardwareDeviceId;
}

EHardwareDevicePrimaryType FPlayerKeyMapping::GetPrimaryDeviceType() const
{
	return HardwareDeviceId.PrimaryDeviceType;
}

EHardwareDeviceSupportedFeatures::Type FPlayerKeyMapping::GetHardwareDeviceSupportedFeatures() const
{
	return static_cast<EHardwareDeviceSupportedFeatures::Type>(HardwareDeviceId.SupportedFeaturesMask);
}

const UInputAction* FPlayerKeyMapping::GetAssociatedInputAction() const
{
	return AssociatedInputActionSoft.Get();
}

uint32 GetTypeHash(const FPlayerKeyMapping& InMapping)
{
	uint32 Hash = 0;
	Hash = HashCombine(Hash, GetTypeHash(InMapping.MappingName));
	Hash = HashCombine(Hash, GetTypeHash(InMapping.Slot));
	Hash = HashCombine(Hash, GetTypeHash(InMapping.CurrentKey));
	Hash = HashCombine(Hash, GetTypeHash(InMapping.HardwareDeviceId));
	return Hash;
}

void FPlayerKeyMapping::ResetToDefault()
{
	// Only mark as dirty during a reset to default if the mapping was set to a different key
	// This avoids unnecessarily marking keys as dirty if you do a reset of the entire key profile
	if (IsCustomized())
	{
		bIsDirty = true;
	}
	
	CurrentKey = DefaultKey;	
}

void FPlayerKeyMapping::SetCurrentKey(const FKey& NewKey)
{
	if (NewKey != CurrentKey)
	{
		bIsDirty = true;
	}
	
	CurrentKey = NewKey;
}

void FPlayerKeyMapping::SetHardwareDeviceId(const FHardwareDeviceIdentifier& InDeviceId)
{
	HardwareDeviceId = InDeviceId;
}

void FPlayerKeyMapping::UpdateMetadataFromActionKeyMapping(const FEnhancedActionKeyMapping& OriginalMapping)
{
	ensure(OriginalMapping.IsPlayerMappable() && OriginalMapping.GetMappingName() == MappingName);
	
	DisplayCategory = OriginalMapping.GetDisplayCategory();
	DisplayName = OriginalMapping.GetDisplayName();
	AssociatedInputActionSoft = OriginalMapping.Action;
}

void FPlayerKeyMapping::UpdateDefaultKeyFromActionKeyMapping(const FEnhancedActionKeyMapping& OriginalMapping)
{
	ensure(OriginalMapping.IsPlayerMappable() && OriginalMapping.GetMappingName() == MappingName);

	DefaultKey = OriginalMapping.Key;

	// If the mapping is the same as the default key, make sure that it is not marked as
	// dirty so that we don't serialize it
	if (!IsCustomized())
	{
		bIsDirty = false;
	}
}

bool FPlayerKeyMapping::operator==(const FPlayerKeyMapping& Other) const
{
	return
	       MappingName			== Other.MappingName
		&& Slot					== Other.Slot
		&& HardwareDeviceId		== Other.HardwareDeviceId
		&& CurrentKey			== Other.CurrentKey
		&& DefaultKey			== Other.DefaultKey;
}

bool FPlayerKeyMapping::operator!=(const FPlayerKeyMapping& Other) const
{
	return !FPlayerKeyMapping::operator==(Other);
}

FPlayerKeyMapping& FPlayerKeyMapping::operator=(FPlayerKeyMapping& Other)
{
	MappingName = Other.MappingName;
	DisplayName = Other.DisplayName;
	DisplayCategory = Other.DisplayCategory;
	Slot = Other.Slot;
	bIsDirty = Other.bIsDirty;
	DefaultKey = Other.DefaultKey;
	CurrentKey = Other.CurrentKey;
	HardwareDeviceId = Other.HardwareDeviceId;
	AssociatedInputActionSoft = Other.AssociatedInputActionSoft;

	return *this;
}

FPlayerKeyMapping& FPlayerKeyMapping::operator=(const FPlayerKeyMapping& Other)
{
	MappingName = Other.MappingName;
	DisplayName = Other.DisplayName;
	DisplayCategory = Other.DisplayCategory;
	Slot = Other.Slot;
	bIsDirty = Other.bIsDirty;
	DefaultKey = Other.DefaultKey;
	CurrentKey = Other.CurrentKey;
	HardwareDeviceId = Other.HardwareDeviceId;
	AssociatedInputActionSoft = Other.AssociatedInputActionSoft;

	return *this;
}

const bool FPlayerKeyMapping::IsDirty() const
{
	return bIsDirty;
}

///////////////////////////////////////////////////////////
// UEnhancedPlayerMappableKeyProfile

void UEnhancedPlayerMappableKeyProfile::ResetToDefault()
{
	// Reset every player mapping to the default key value
	for (TPair<FName, FKeyMappingRow>& Pair : PlayerMappedKeys)
	{
		for (FPlayerKeyMapping& Mapping : Pair.Value.Mappings)
		{
			Mapping.ResetToDefault();
		}
	}
	
	UE_LOGF(LogEnhancedInput, Verbose, "Reset Player Mappable Key Profile '%ls' to default values", *ProfileIdentifierString);
}

void UEnhancedPlayerMappableKeyProfile::EquipProfile()
{
	UE_LOGF(LogEnhancedInput, Verbose, "Equipping Player Mappable Key Profile '%ls'", *ProfileIdentifierString);
}

void UEnhancedPlayerMappableKeyProfile::UnEquipProfile()
{
	UE_LOGF(LogEnhancedInput, Verbose, "Unequipping Player Mappable Key Profile '%ls'", *ProfileIdentifierString);
}

void UEnhancedPlayerMappableKeyProfile::SetDisplayName(const FText& NewDisplayName)
{
	DisplayName = NewDisplayName;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FGameplayTag& UEnhancedPlayerMappableKeyProfile::GetProfileIdentifer() const
{
	return ProfileIdentifier;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const FString& UEnhancedPlayerMappableKeyProfile::GetProfileIdString() const
{
	return ProfileIdentifierString;
}

const FText& UEnhancedPlayerMappableKeyProfile::GetProfileDisplayName() const
{
	return DisplayName;
}

const TMap<FName, FKeyMappingRow>& UEnhancedPlayerMappableKeyProfile::GetPlayerMappingRows() const
{
	return PlayerMappedKeys;
}

void UEnhancedPlayerMappableKeyProfile::ResetMappingToDefault(const FName InMappingName)
{
	if (FKeyMappingRow* MappingRow = FindKeyMappingRowMutable(InMappingName))
	{
		for (FPlayerKeyMapping& Mapping : MappingRow->Mappings)
		{
			Mapping.ResetToDefault();
		}
	}
}

FKeyMappingRow* UEnhancedPlayerMappableKeyProfile::FindKeyMappingRowMutable(const FName InMappingName)
{
	return const_cast<FKeyMappingRow*>(FindKeyMappingRow(InMappingName));
}

const FKeyMappingRow* UEnhancedPlayerMappableKeyProfile::FindKeyMappingRow(const FName InMappingName) const
{
	return PlayerMappedKeys.Find(InMappingName);
}

void UEnhancedPlayerMappableKeyProfile::DumpProfileToLog() const
{
	UE_LOGF(LogEnhancedInput, Log, "%ls", *ToString());
}

FString UEnhancedPlayerMappableKeyProfile::ToString() const
{
	TStringBuilder<1024> Builder;
	Builder.Appendf(TEXT("Key Profile '%s' has %d key mappings\n"), *ProfileIdentifierString, PlayerMappedKeys.Num());
	
	for (const TPair<FName, FKeyMappingRow>& Pair : PlayerMappedKeys)
	{
		Builder.Append(Pair.Key.ToString());
		
		for (const FPlayerKeyMapping& Mapping : Pair.Value.Mappings)
		{
			Builder.Append(Mapping.ToString());
			Builder.Append("\n\t");
		}
	}

	return Builder.ToString();
}

int32 UEnhancedPlayerMappableKeyProfile::GetMappedKeysInRow(const FName MappingName, TArray<FKey>& OutKeys) const
{
	OutKeys.Reset();
	
	if (const FKeyMappingRow* MappingRow = FindKeyMappingRow(MappingName))
	{
		for (const FPlayerKeyMapping& Mapping : MappingRow->Mappings)
		{
			OutKeys.Add(Mapping.GetCurrentKey());
		}
	}
	else
	{
		UE_LOGF(LogEnhancedInput, Verbose, "Player Mappable Key Profile '%ls' doesn't have any mappings for '%ls'", *ProfileIdentifierString, *MappingName.ToString());
	}
	return OutKeys.Num();
}

int32 UEnhancedPlayerMappableKeyProfile::GetPlayerMappedKeysForRebuildControlMappings(const FEnhancedActionKeyMapping& DefaultMapping, TArray<FKey>& OutKeys) const
{
	// Don't bother trying to query for keys on a non-player mappable action mapping
	if (!DefaultMapping.IsPlayerMappable())
	{
		return 0;
	}
	
	FPlayerMappableKeyQueryOptions QueryOptions = {};

	QueryOptions.MappingName = DefaultMapping.GetMappingName();
	QueryOptions.KeyToMatch = DefaultMapping.Key;
	QueryOptions.bMatchBasicKeyTypes = true;
	QueryOptions.bMatchKeyAxisType = false;

	// Note for subclasses, if you want to filter based on the current input device then you can
	// do this!
	// if (const UInputDeviceSubsystem* DeviceSubsystem = UInputDeviceSubsystem::Get())
	// {
	// 	FHardwareDeviceIdentifier MostRecentDevice = DeviceSubsystem->GetMostRecentlyUsedHardwareDevice(OwningUserId);
	// 	QueryOptions.RequiredDeviceType = MostRecentDevice.PrimaryDeviceType;
	// 	QueryOptions.RequiredDeviceFlags = MostRecentDevice.SupportedFeaturesMask;
	// }
	
	return QueryPlayerMappedKeys(QueryOptions, OutKeys);
}

int32 UEnhancedPlayerMappableKeyProfile::QueryPlayerMappedKeys(const FPlayerMappableKeyQueryOptions& Options, TArray<FKey>& OutKeys) const
{
	OutKeys.Reset();

	if (const FKeyMappingRow* MappingRow = FindKeyMappingRow(Options.MappingName))
	{
		for (const FPlayerKeyMapping& PlayerMapping : MappingRow->Mappings)
		{
			if (DoesMappingPassQueryOptions(PlayerMapping, Options))
			{
				OutKeys.Add(PlayerMapping.GetCurrentKey());
			}
		}
	}
	else
	{
		UE_LOGF(LogEnhancedInput, Verbose, "Player Mappable Key Profile '%ls' doesn't have any mappings for '%ls'", *ProfileIdentifierString, *Options.MappingName.ToString());
	}

	return OutKeys.Num();
}

bool UEnhancedPlayerMappableKeyProfile::DoesMappingPassQueryOptions(const FPlayerKeyMapping& PlayerMapping, const FPlayerMappableKeyQueryOptions& Options) const
{
	if (Options.KeyToMatch.IsValid())
	{
		const FKey& A = Options.KeyToMatch;
		const FKey& B = PlayerMapping.GetCurrentKey();
	
		// Ensure that the player mapped key matches the one set in the key profile
		if (Options.bMatchBasicKeyTypes)
		{
			const bool bKeyTypesMatch = 
				A.IsGamepadKey() == B.IsGamepadKey() &&
				A.IsTouch() == B.IsTouch() &&
				A.IsGesture() == B.IsGesture();

			if (!bKeyTypesMatch)
			{
				return false;
			}
		}

		if (Options.bMatchKeyAxisType)
		{
			const bool bKeyAxisMatch =
				A.IsAxis1D() == B.IsAxis1D() &&
				A.IsAxis2D() == B.IsAxis2D() &&
				A.IsAxis3D() == B.IsAxis3D();

			if (!bKeyAxisMatch)
			{
				return false;
			}
		}
	}

	// If you have requested a specific key slot, then filter down by that.
	// Do not filter if the slot to match is Unspecified.
	if (Options.SlotToMatch != EPlayerMappableKeySlot::Unspecified &&
		Options.SlotToMatch != PlayerMapping.GetSlot())
	{
		return false;
	}

	// Match hardware device info per mapping
	const FHardwareDeviceIdentifier& PlayerMappingDevice = PlayerMapping.GetHardwareDeviceId();
	
	// Filter mappings based on their primary hardware device type
	if (Options.RequiredDeviceType != EHardwareDevicePrimaryType::Unspecified &&
		Options.RequiredDeviceType != PlayerMappingDevice.PrimaryDeviceType)
	{
		return false;
	}

	// Filter mappings based on their hardware device's supported features
	if (Options.RequiredDeviceFlags != EHardwareDeviceSupportedFeatures::Type::Unspecified &&
		!PlayerMappingDevice.HasAllSupportedFeatures(static_cast<EHardwareDeviceSupportedFeatures::Type>(Options.RequiredDeviceFlags)))
	{
		return false;
	}

	return true;
}

int32 UEnhancedPlayerMappableKeyProfile::GetMappingNamesForKey(const FKey& InKey, TArray<FName>& OutMappingNames) const
{
	OutMappingNames.Reset();

	for (const TPair<FName, FKeyMappingRow>& Pair : PlayerMappedKeys)
	{
		for (const FPlayerKeyMapping& Mapping : Pair.Value.Mappings)
		{
			if (Mapping.GetCurrentKey() == InKey)
			{
				// We know that this action has the key mapped to it, so there is no need to continue checking
				// the rest of it's mappings
				OutMappingNames.Add(Pair.Key);
				break;
			}
		}
	}

	return OutMappingNames.Num();
}

bool FKeyMappingRow::HasAnyMappings() const
{
	return !Mappings.IsEmpty();
}

void UEnhancedPlayerMappableKeyProfile::Serialize(FArchive& Ar)
{
	// See note in header!
	Super::Serialize(Ar);
}

FPlayerKeyMapping* UEnhancedPlayerMappableKeyProfile::FindKeyMapping(const FMapPlayerKeyArgs& InArgs) const
{
	// Get the current mappings for the desired action name.
	if (const FKeyMappingRow* MappingRow = PlayerMappedKeys.Find(InArgs.MappingName))
	{
		// If mapping already exists for the given slot and hardware device, then we can
		// just change that key
		const FPlayerKeyMapping* Mapping = Algo::FindByPredicate(MappingRow->Mappings, [&InArgs](const FPlayerKeyMapping& Mapping)
		{
			return Mapping.GetSlot() == InArgs.Slot && Mapping.GetHardwareDeviceId().HardwareDeviceIdentifier == InArgs.HardwareDeviceId;
		});
		return const_cast<FPlayerKeyMapping*>(Mapping);
	}
	return nullptr;
}

void UEnhancedPlayerMappableKeyProfile::K2_FindKeyMapping(FPlayerKeyMapping& OutKeyMapping, const FMapPlayerKeyArgs& InArgs) const
{
	if (FPlayerKeyMapping* FoundMapping = FindKeyMapping(InArgs))
	{
		OutKeyMapping = *FoundMapping;
	}
	else
	{
		OutKeyMapping = FPlayerKeyMapping::InvalidMapping;
	}
}

///////////////////////////////////////////////////////////
// UEnhancedInputUserSettings

UEnhancedInputUserSettings* UEnhancedInputUserSettings::LoadOrCreateSettings(ULocalPlayer* LocalPlayer)
{
	UEnhancedInputUserSettings* Settings = nullptr;

	if (!LocalPlayer)
	{
		UE_LOGF(LogEnhancedInput, Log, "Unable to determine an owning Local Player for the given Enhanced Player Input object");
		return nullptr;
	}

	const FString& SaveFileName = GetSaveFilename(LocalPlayer);
	
	// If the save game exists, load it.
	if (UGameplayStatics::DoesSaveGameExist(SaveFileName, LocalPlayer->GetLocalPlayerIndex()))
	{
		USaveGame* Slot = UGameplayStatics::LoadGameFromSlot(SaveFileName, LocalPlayer->GetLocalPlayerIndex());
		Settings = Cast<UEnhancedInputUserSettings>(Slot);
	}

	// If there is no settings save game object, then we can create on
	// based on the class type set in the developer settings
	const UEnhancedInputDeveloperSettings* DevSettings = GetDefault<UEnhancedInputDeveloperSettings>();
	UClass* SettingsClass = DevSettings->UserSettingsClass ? DevSettings->UserSettingsClass.Get() : UEnhancedInputUserSettings::StaticClass();

	// This property is marked as "NoClear", so this should be impossible.
	if (!ensureMsgf(SettingsClass, TEXT("Invalid Enhanced Input User settings class!")))
	{
		return nullptr;
	}
	
	// If the settings are null (they dont exist yet) or the class has changed, we need to create a new object.
	if (Settings == nullptr)
	{
		Settings = Cast<UEnhancedInputUserSettings>(UGameplayStatics::CreateSaveGameObject(SettingsClass));
	}
	// If the class has changed, then we should call into a customizable upgrade path notifying you of the change.
	// This can happen if you change the class in the editor and there is still an old save file on disk,
	// or if you change the class of your user settings object and would to keep some of the old data.
	else if (Settings->GetClass() != SettingsClass)
	{
		UE_LOGF(
			LogEnhancedInput,
			Log,
			"[%hs] Enhanced Input User Settings class has been changed from '%ls' to '%ls'. A new settings save object will be created (your saved settings will be reset). Override HandleSettingsClassChangedOnLoad_Impl to retain data.",
			__func__, *GetNameSafe(Settings->GetClass()), *GetNameSafe(SettingsClass));

		// Create a new settings object here with the new class which it has been changed to
		UEnhancedInputUserSettings* NewSettingsObject = Cast<UEnhancedInputUserSettings>(UGameplayStatics::CreateSaveGameObject(SettingsClass));

		// Allow you to upgrade the old settings if you ever change the class of your user settings type
		NewSettingsObject->HandleSettingsClassChangedOnLoad(Settings);

		// Store our new settings object as the return value
		Settings = NewSettingsObject;
	}

	if (ensure(Settings))
	{
		Settings->Initialize(LocalPlayer);
    	Settings->ApplySettings();	
	}

	return Settings;
}

void UEnhancedInputUserSettings::Initialize(ULocalPlayer* InLocalPlayer)
{
	// If the local player hasn't changed, then we don't need to do anything.
	// This may be the case if the Player Controller has changed
	if (InLocalPlayer == GetLocalPlayer())
	{
		return;
	}
	
	OwningLocalPlayer = InLocalPlayer;
	ensureMsgf(OwningLocalPlayer.IsValid(), TEXT("UEnhancedInputUserSettings is missing an owning local player!"));

	// Create a default key mapping profile in the case where one doesn't exist
	if (!GetActiveKeyProfile())
	{
		const ULocalPlayer* LP = GetLocalPlayer();
		
		FPlayerMappableKeyProfileCreationArgs Args = {};
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Args.ProfileIdentifier = UE::EnhancedInput::TAG_DefaultProfileIdentifier;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Args.ProfileStringIdentifier = UE::EnhancedInput::DefaultProfileIdentifierString;
		Args.UserId = LP ? LP->GetPlatformUserId() : PLATFORMUSERID_NONE;
		Args.DisplayName = UE::EnhancedInput::DefaultProfileDisplayName;
		Args.bSetAsCurrentProfile = true;
		
		CreateNewKeyProfile(Args);
	}
}

void UEnhancedInputUserSettings::ApplySettings()
{
	UE_LOGF(LogEnhancedInput, Verbose, "Enhanced Input User Settings applied!");

	OnSettingsApplied.Broadcast();
}

void UEnhancedInputUserSettings::SaveSettings()
{
	if (ULocalPlayer* OwningPlayer = GetLocalPlayer())
	{
		UGameplayStatics::SaveGameToSlot(this, GetSaveFilename(OwningPlayer), OwningPlayer->GetLocalPlayerIndex());
		UE_LOGF(LogEnhancedInput, Verbose, "Enhanced Input User Settings saved!");
	}
	else
	{
		UE_LOGF(LogEnhancedInput, Warning, "Attempting to save Enhanced Input User settings without an owning local player!");
	}
}

void UEnhancedInputUserSettings::AsyncSaveSettings()
{
	if (const ULocalPlayer* OwningPlayer = GetLocalPlayer())
	{
		FAsyncSaveGameToSlotDelegate SavedDelegate;
		SavedDelegate.BindUObject(this, &UEnhancedInputUserSettings::OnAsyncSaveComplete);
		
		UGameplayStatics::AsyncSaveGameToSlot(this, GetSaveFilename(OwningPlayer), OwningPlayer->GetLocalPlayerIndex(), SavedDelegate);
	}
	else
	{
		UE_LOGF(LogEnhancedInput, Error, "[UEnhancedInputUserSettings::AsyncSaveSettings] Failed async save, there is no owning local player!");
	}
}

void UEnhancedInputUserSettings::HandleSettingsClassChangedOnLoad(UEnhancedInputUserSettings* OldSettingsObject)
{
	// This function is called when the user settings class has changed
	check(OldSettingsObject);
	
	// This should only ever be called for settings objects of two different classes
	check(OldSettingsObject->GetClass() != GetClass());

	// Call the overridable virtual function to allow games to customize this behavior
	HandleSettingsClassChangedOnLoad_Impl(OldSettingsObject);
}

void UEnhancedInputUserSettings::HandleSettingsClassChangedOnLoad_Impl(UEnhancedInputUserSettings* OldSettingsObject)
{
	// Copy over any values of properties from the old settings object to this new one for initialization.
	// This will keep things like the key mappings, profile id, and other UPROPERTY's that are the same across
	// the two objects.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	CurrentProfileIdentifier = OldSettingsObject->CurrentProfileIdentifier;
	SavedKeyProfiles = OldSettingsObject->SavedKeyProfiles;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	CurrentProfileIdentifierString = OldSettingsObject->CurrentProfileIdentifierString;
	SavedKeyProfilesMap = OldSettingsObject->SavedKeyProfilesMap;

	// Note: Calling any Serialize functions, or something like CopyPropertiesForUnrelatedObjects, will cause
	// errors to be logged because there will be a size mismatch between the two types, as they are different classes.
	// This is why we naively assign the data we would like to keep above instead of calling a more complex serialization function.
}

FString UEnhancedInputUserSettings::GetSaveFilename(const ULocalPlayer* LP)
{
	check(LP);

	UEnhancedPlayerInput* PlayerInput = LP->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>()->GetPlayerInput();
	check(PlayerInput);

	return PlayerInput->GetUserSettingsSaveFileName();
}

void UEnhancedInputUserSettings::OnAsyncSaveComplete(const FString& SlotName, const int32 UserIndex, bool bSuccess)
{
	if (bSuccess)
	{
		UE_LOGF(LogEnhancedInput, Log, "UEnhancedInputUserSettings::OnAsyncSaveComplete] Async save of slot '%ls' for user index '%d' completed successfully!", *SlotName, UserIndex);
	}
	else
	{
		UE_LOGF(LogEnhancedInput, Error, "[UEnhancedInputUserSettings::OnAsyncSaveComplete] Failed async save of slot '%ls' for user index '%d'", *SlotName, UserIndex);
	}
}

namespace UE::EnhancedInput
{
	/**
	* The save version for player mappable keys serialization in Enhanced Input. 
	* When adding a new version, make sure to set "Current"! 
	*/
	struct FPlayerMappableSaveVersion
	{
		/**
		 * The version number which this feature shipped with. Needed for loading previous versions of the save files
		 */
		static constexpr int32 Base = 1;

		/**
		 * Save version for when we converted UEnhancedPlayerMappableKeyProfile to be identified with an FString instead
		 * of with a FGameplayTags
		 */
		static constexpr int32 ConvertProfileIdFromTagToString = 2;

		/**
		 * The current save version of this feature. In the future, make sure to increment this if a new version is added.
		 */
		static constexpr int32 Current = ConvertProfileIdFromTagToString;
	};	

	/**
	* Data saved which represents a single dirty key mapping.
	*/
	struct FKeyMappingSaveData
	{
		friend FArchive& operator<<(FArchive& Ar, FKeyMappingSaveData& Data)
		{
			Ar << Data.ActionName;
			Ar << Data.CurrentKeyName;
			Ar << Data.HardwareDeviceId;
			Ar << Data.Slot;
			return Ar;
		}
		
		/**
		* The name of the key mapping which is dirty.
		*/
		FName ActionName = NAME_None;

		/**
		* The currently set key that should be used and was set by the player.
		*/
		FName CurrentKeyName = NAME_None;

		/**
		* Specific hardware which this mapping should be applied to if not invalid.
		*/
		FHardwareDeviceIdentifier HardwareDeviceId = FHardwareDeviceIdentifier::Invalid;

		/**
		* Which slot this mapping is for. 
		*/
		EPlayerMappableKeySlot Slot = EPlayerMappableKeySlot::Unspecified;
	};

	/** Struct used to store info about the mappable profile subobjects */
	struct FMappableKeysHeader
	{
		// Support for loading in legacy files
		friend FArchive& operator<<(FArchive& Ar, FMappableKeysHeader& Header)
		{
			// We should only ever be going down this code path when loading an old version. 
			// To save new data, we use the "Serialize" function below which takes a version number. 
			check(Ar.IsLoading());
			
			// HEY! Don't add or modify anything that is being serialized here. Use the new Serialize function which takes a version instead.
			FGameplayTag ProfileIdentifier;
			Ar << ProfileIdentifier;
			Ar << Header.ClassPath;
			Ar << Header.ObjectPath;
			Ar << Header.DirtyMappings;
			// HEY! Don't add or modify anything that is being serialized here.

			// Set the new FString profile ID based on the legacy gameplay tag.
			Header.ProfileIdentifierString = ProfileIdentifier.ToString();
			
			return Ar;
		}
		
		/**
		* Serializes the given Header while considering the given version.
		*/
		static FArchive& Serialize(FArchive& Ar, FMappableKeysHeader& Header, const int32 Version)
		{			
			Ar << Header.ClassPath;
			Ar << Header.ObjectPath;
			Ar << Header.DirtyMappings;
			Ar << Header.ProfileIdentifierString;

			// Note: When adding new properties to be serialized in the future, make sure to respect the version number! 
			// i.e.
			// 
			// if (Version >= FPlayerMappableSaveVersion::NewVersionName)
			// { 
			//		Ar << Header.NewProperty; 
			// }
			
			return Ar;
		}
		
		FString ProfileIdentifierString;
		FString ClassPath;
		FString ObjectPath;
		TArray<FKeyMappingSaveData> DirtyMappings;
	};
}

void UEnhancedInputUserSettings::Serialize(FArchive& Ar)
{
	// We always want to ensure that we call the Super::Serialize function here because
	// not doing that can introduce many hard to track down serialization bugs and not include
	// the UObject info of this settings object.
	Super::Serialize(Ar);

	if (IsTemplate() || Ar.IsCountingMemory())
	{
		return;
	}
	
	int32 SaveVersion = UE::EnhancedInput::FPlayerMappableSaveVersion::Current;
	Ar << SaveVersion;
	
	// Detect a mis-match of byte streams, i.e. file corruption
	// The version should be less than or equal to the current version. It can be "less than" if loading an 
	// older save file version.
	ensure(SaveVersion <= UE::EnhancedInput::FPlayerMappableSaveVersion::Current);
	
	TArray<UE::EnhancedInput::FMappableKeysHeader> Headers;
	if (Ar.IsSaving())
	{
		UObject* Outer = this;
		
		for (TPair<FString, UEnhancedPlayerMappableKeyProfile*> ProfilePair : SavedKeyProfilesMap)
		{
			UE::EnhancedInput::FMappableKeysHeader Header =
				{
					.ProfileIdentifierString = ProfilePair.Key,
					.ClassPath = ProfilePair.Value->GetClass()->GetPathName(),
					.ObjectPath = ProfilePair.Value->GetPathName(Outer),
					.DirtyMappings = {}
				};

			// Serialize only dirty mappings here, not every mapping in the key profile.
			for (TPair<FName, FKeyMappingRow>& MappingRow : ProfilePair.Value->PlayerMappedKeys)
			{
				for (FPlayerKeyMapping& Mapping : MappingRow.Value.Mappings)
				{
					// We want to save any dirty or customized key mappings
					if (Mapping.IsDirty() || Mapping.IsCustomized())
					{
						Header.DirtyMappings.Push(
							{
								.ActionName = Mapping.MappingName,
								.CurrentKeyName = Mapping.CurrentKey.GetFName(),
								.HardwareDeviceId = Mapping.HardwareDeviceId, 
								.Slot = Mapping.Slot
							});
						
						// Since we are about to save this key mapping, we no longer need to treat
						// it as dirty
						Mapping.bIsDirty = false;
					}
				}
			}

			Headers.Push(Header);
		}
	}

	// We need to support this legacy loading path from before we serialized out the headers with consideration to the version.
	if (Ar.IsLoading() && SaveVersion < UE::EnhancedInput::FPlayerMappableSaveVersion::ConvertProfileIdFromTagToString)
	{
		Ar << (Headers);
	}
	else
	{
		// Serialize out all the headers, need to track the number of them so that we can read them back in
		int32 NumberHeaders = Headers.Num();
		Ar << NumberHeaders;

		// When loading, make sure that there are some values in the header array that we can populate when serializing
		if (Ar.IsLoading())
		{
			Headers.AddZeroed(NumberHeaders);
		}
		
		for (int32 i = 0; i < NumberHeaders; ++i)
		{	
			UE::EnhancedInput::FMappableKeysHeader::Serialize(Ar, Headers[i], SaveVersion);
		}
	}
	
	// Now we can actually re-populate our runtime data with the info from the headers that we just loaded in
	if (Ar.IsLoading())
	{
		// When loading, we want to empty the current saved key profile map so that it is populated with 
		// only data from the archive.
		//
		// We only care about the dirty data when serializing. If other profiles currently exist here, they were added 
		// via the Registration of IMC assets and would need to be re-registered. We will re-populate the registered IMCs
		// after we load everything that was dirty.
		SavedKeyProfilesMap.Empty();		

		for (const UE::EnhancedInput::FMappableKeysHeader& Header : Headers)
		{
			if (const UClass* FoundClass = FindObject<UClass>(nullptr, *Header.ClassPath))
			{
				UEnhancedPlayerMappableKeyProfile* NewProfile = NewObject<UEnhancedPlayerMappableKeyProfile>(/* outer */ this, /* class */ FoundClass);
				NewProfile->ProfileIdentifierString = Header.ProfileIdentifierString;

				// Add the new profile to the settings object. This will replace the old profile if one existed
				SavedKeyProfilesMap.Add(Header.ProfileIdentifierString, NewProfile);				
				
				// Add any player saved keys to this profile!
				for (const UE::EnhancedInput::FKeyMappingSaveData& SavedKeyData : Header.DirtyMappings)
				{
					FKeyMappingRow& MappingRow = NewProfile->PlayerMappedKeys.FindOrAdd(SavedKeyData.ActionName);

					FPlayerKeyMapping PlayerMapping = {};
					PlayerMapping.MappingName = SavedKeyData.ActionName;
					PlayerMapping.CurrentKey = FKey(SavedKeyData.CurrentKeyName);
					PlayerMapping.Slot = SavedKeyData.Slot;
					PlayerMapping.HardwareDeviceId = SavedKeyData.HardwareDeviceId;
					
					MappingRow.Mappings.Add(PlayerMapping);
				}


				// Note: The currently registered Input Mapping contexts will be re-registered on these
				// newly created profiles after we are done loading all previous data.

				// Ensure that the owning local player is up to date
				if (const ULocalPlayer* LP = GetLocalPlayer())
				{
					NewProfile->OwningUserId = LP->GetPlatformUserId();
				}
			}
		}
	}

	// This is the final piece of serializing the key profiles. It's a sanity and safety check.
	// Because we are saving subobjects here, we need to serialize their UObject information as well
	FString SavedObjectTerminator = TEXT("ObjectEnd");
	
	for (const TPair<FString, TObjectPtr<UEnhancedPlayerMappableKeyProfile>>& ProfilePair : SavedKeyProfilesMap)
	{
		// FYI: This call to serialize will only be serializing the basic UObject info of this key profile,
		// not all the individual key mappings :) 
		ProfilePair.Value->Serialize(Ar);

		// Save a terminator after each subobject
		Ar << SavedObjectTerminator;	

		// When you are serializing to an external source, like some other UObject that you want to upload to a custom backend,
		// this ensure might be hit if that external object has been changed and the data no longer perfectly matches up. 
		// This can be annoying in the editor, so we will just log instead of ensuring for now to make it less annoying
		// for devs iterating.
		// 
		// For example, if you are working in multiple different release streams or something, and have to bounce backwards
		// to an older version and your save profile has the newer versions data in it, this ensure would be triggered because
		// the object terminator does not perfectly match up.
		// This is something that should never happen in a game build, but can happen quite often in an editor build for devs
		// with multiple workspaces
#if WITH_EDITOR
		UE_CLOGF(SavedObjectTerminator != TEXT("ObjectEnd"), LogEnhancedInput, Error, "Serialization size mismatch! Possible over-read or over-write of this buffer.");
#else
		if (!ensure(SavedObjectTerminator == TEXT("ObjectEnd")))
		{
			UE_LOGF(LogEnhancedInput, Error, "Serialization size mismatch! Possible over-read or over-write of this buffer.");
			break;
		}
#endif	// WITH_EDITOR
	}

	// Now that we have finished loading all the data that was in the archive, we should re-register all the 
	// mapping contexts to the newly created key profiles.
	if (Ar.IsLoading())
	{
		// This will re-create any profiles which may have existed before this serialize load call was initiated and ensure
		// that each profile has its default key mappings set up for each IMC which has unique defaults for specific profiles
		for (const TObjectPtr<const UInputMappingContext>& IMC : RegisteredMappingContexts)
		{
			RegisterInputMappingContextInternal(IMC);
		}
	}
}

ULocalPlayer* UEnhancedInputUserSettings::GetLocalPlayer() const
{
	return OwningLocalPlayer.Get();
}

void UEnhancedInputUserSettings::MapPlayerKey(const FMapPlayerKeyArgs& InArgs, FGameplayTagContainer& FailureReason)
{
	if (!InArgs.MappingName.IsValid())
	{
		FailureReason.AddTag(UE::EnhancedInput::TAG_InvalidMappingName);
		return;
	}
	
	// Get the key profile that was specific
	UEnhancedPlayerMappableKeyProfile* KeyProfile = !InArgs.ProfileIdString.IsEmpty() ? GetKeyProfileWithId(InArgs.ProfileIdString) : GetActiveKeyProfile();
	if (!KeyProfile)
	{
		FailureReason.AddTag(UE::EnhancedInput::TAG_NoKeyProfile);
		return;
	}

	// If this mapping already exists, we can simply change it's key and be done
	if (FPlayerKeyMapping* FoundMapping = KeyProfile->FindKeyMapping(InArgs))
	{
		// Then set the player mapped key
		FoundMapping->SetCurrentKey(InArgs.NewKey);
		OnKeyMappingUpdated(FoundMapping, InArgs, false);
		
		if(InArgs.bDeferOnSettingsChangedBroadcast  && !DeferredSettingsChangedTimerHandle.IsValid())
		{
			if(UWorld* World = GetWorld())
			{
				TWeakObjectPtr<UEnhancedInputUserSettings> WeakThis = this;
				DeferredSettingsChangedTimerHandle = World->GetTimerManager().SetTimerForNextTick(FTimerDelegate::CreateWeakLambda(this, 
					[WeakThis]
					{
						if(WeakThis.IsValid())
						{
							WeakThis->OnSettingsChanged.Broadcast(WeakThis.Get());
							WeakThis->DeferredSettingsChangedTimerHandle.Invalidate();
						}
					}));
			}
		}
		else
		{
			OnSettingsChanged.Broadcast(this);
		}
	}
	// If it doesn't exist, then we need to make it if there is a valid action name
	else if (FKeyMappingRow* MappingRow = KeyProfile->FindKeyMappingRowMutable(InArgs.MappingName))
	{
		// If one doesn't exist, then we need to create a new mapping in the given slot.
		// 
		// In order to populate the default values correctly, we only do this if we know that
		// mappings exist for it
		const int32 NumMappings = MappingRow->Mappings.Num();
		if (InArgs.bCreateMatchingSlotIfNeeded && NumMappings > 0)
		{
			const FPlayerKeyMapping* ExistingMapping = MappingRow->Mappings.FindArbitraryElement();
			
			// Add a default mapping to this row
			FPlayerKeyMapping PlayerMappingData = {};
			PlayerMappingData.MappingName = InArgs.MappingName;
			PlayerMappingData.Slot = InArgs.Slot;
			// This is a new mapping, and should be dirty upon creation
			PlayerMappingData.bIsDirty = true;

			// Check for known hardware device Id's if one has been specified
			if (!InArgs.HardwareDeviceId.IsNone())
			{
				if (const UInputPlatformSettings* PlatformSettings = UInputPlatformSettings::Get())
				{
					if (const FHardwareDeviceIdentifier* Hardware = PlatformSettings->GetHardwareDeviceForClassName(InArgs.HardwareDeviceId))
					{
						PlayerMappingData.HardwareDeviceId = *Hardware;	
					}
					else
					{
						UE_LOGF(LogEnhancedInput, Log, "[UEnhancedInputUserSettings::MapPlayerKey] Unable to find a matching Hardware Device Identifier with the HardwareDeviceId of '%ls'", *InArgs.HardwareDeviceId.ToString());
					}
				}
			}
			
			// This mapping never existed in the default IMC, so the default mapping will be the default
			// EKeys::Invalid and we only need to track the player mapped key
			PlayerMappingData.SetCurrentKey(InArgs.NewKey);
			PlayerMappingData.DisplayName = ExistingMapping->DisplayName;
			PlayerMappingData.DisplayCategory = ExistingMapping->DisplayCategory;
			PlayerMappingData.AssociatedInputActionSoft = ExistingMapping->AssociatedInputActionSoft;
			
			const FSetElementId SetElem = MappingRow->Mappings.Add(PlayerMappingData);
			OnKeyMappingUpdated(&MappingRow->Mappings.Get(SetElem), InArgs, false);
			
			if(InArgs.bDeferOnSettingsChangedBroadcast  && !DeferredSettingsChangedTimerHandle.IsValid())
			{
				if(UWorld* World = GetWorld())
				{
					TWeakObjectPtr<UEnhancedInputUserSettings> WeakThis = this;
					DeferredSettingsChangedTimerHandle = World->GetTimerManager().SetTimerForNextTick(FTimerDelegate::CreateWeakLambda(this, 
						[WeakThis]
						{
							if(WeakThis.IsValid())
							{
								WeakThis->OnSettingsChanged.Broadcast(WeakThis.Get());
								WeakThis->DeferredSettingsChangedTimerHandle.Invalidate();
							}
						}));
				}
			}
			else
			{
				OnSettingsChanged.Broadcast(this);
			}
		}
	}
	else
	{
		// If there is neither an existing key mapping or a key mapping row, then mapping the key has failed
		FailureReason.AddTag(UE::EnhancedInput::TAG_NoMappingRowFound);
		UE_LOGF(LogEnhancedInput, Warning, "[UEnhancedInputUserSettings::MapPlayerKey] Failed to map a player key for '%ls'", *InArgs.MappingName.ToString());
	}
}

void UEnhancedInputUserSettings::UnMapPlayerKey(const FMapPlayerKeyArgs& InArgs, FGameplayTagContainer& FailureReason)
{
	if (!InArgs.MappingName.IsValid())
	{
		FailureReason.AddTag(UE::EnhancedInput::TAG_InvalidMappingName);
		return;
	}
	
	// Get the key profile that was specified
	UEnhancedPlayerMappableKeyProfile* KeyProfile = !InArgs.ProfileIdString.IsEmpty() ? GetKeyProfileWithId(InArgs.ProfileIdString) : GetActiveKeyProfile();
	if (!KeyProfile)
	{
		FailureReason.AddTag(UE::EnhancedInput::TAG_NoKeyProfile);
		return;
	}

	if (FPlayerKeyMapping* FoundMapping = KeyProfile->FindKeyMapping(InArgs))
	{
		// Then set the player mapped key
		FoundMapping->ResetToDefault();

		// The settings have only changed if the mapping is dirty now
		if (FoundMapping->IsDirty())
		{
			OnKeyMappingUpdated(FoundMapping, InArgs, true);

			if (InArgs.bDeferOnSettingsChangedBroadcast  && !DeferredSettingsChangedTimerHandle.IsValid())
			{
				if (UWorld* World = GetWorld())
				{
					TWeakObjectPtr<UEnhancedInputUserSettings> WeakThis = this;
					DeferredSettingsChangedTimerHandle = World->GetTimerManager().SetTimerForNextTick(FTimerDelegate::CreateWeakLambda(this, 
						[WeakThis]
						{
							if (WeakThis.IsValid())
							{
								WeakThis->OnSettingsChanged.Broadcast(WeakThis.Get());
								WeakThis->DeferredSettingsChangedTimerHandle.Invalidate();
							}
						}));
				}
			}
			else
			{
				OnSettingsChanged.Broadcast(this);
			}
		}
		
		UE_LOGF(LogEnhancedInput, Verbose, "[UEnhancedInputUserSettings::MapPlayerKey] Reset keymapping to default: '%ls'", *FoundMapping->ToString());
	}
	// if a mapping doesn't exist, then we can't unmap it
	else
	{
		FailureReason.AddTag(UE::EnhancedInput::TAG_NoMatchingMappings);
	}
}

void UEnhancedInputUserSettings::ResetAllPlayerKeysInRow(const FMapPlayerKeyArgs& InArgs, FGameplayTagContainer& FailureReason)
{
	if (!InArgs.MappingName.IsValid())
	{
		FailureReason.AddTag(UE::EnhancedInput::TAG_InvalidMappingName);
		return;
	}

	// Get the key profile that was specified
	UEnhancedPlayerMappableKeyProfile* KeyProfile = !InArgs.ProfileIdString.IsEmpty() ? GetKeyProfileWithId(InArgs.ProfileIdString) : GetActiveKeyProfile();
	if (!KeyProfile)
	{
		FailureReason.AddTag(UE::EnhancedInput::TAG_NoKeyProfile);
		return;
	}
	
	if (FKeyMappingRow* ExistingMappings = KeyProfile->PlayerMappedKeys.Find(InArgs.MappingName))
	{
		for (FPlayerKeyMapping& Mapping : ExistingMappings->Mappings)
		{
			// Then set the player mapped key
			Mapping.ResetToDefault();

			// The settings have only changed if the mapping is dirty now
			if (Mapping.IsDirty())
			{
				OnKeyMappingUpdated(&Mapping, InArgs, true);
				OnSettingsChanged.Broadcast(this);
			}
			UE_LOGF(LogEnhancedInput, Verbose, "[UEnhancedInputUserSettings::MapPlayerKey] Reset keymapping to default: '%ls'", *Mapping.ToString());
		}
	}
	else
	{
		FailureReason.AddTag(UE::EnhancedInput::TAG_NoMatchingMappings);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UEnhancedInputUserSettings::ResetKeyProfileToDefault(const FGameplayTag& ProfileId, FGameplayTagContainer& FailureReason)
{
	return ResetKeyProfileIdToDefault(ProfileId.ToString(), FailureReason);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UEnhancedInputUserSettings::ResetKeyProfileIdToDefault(const FString& ProfileId, FGameplayTagContainer& FailureReason)
{
	if (UEnhancedPlayerMappableKeyProfile* KeyProfile = GetKeyProfileWithId(ProfileId))
	{
		KeyProfile->ResetToDefault();
		OnSettingsChanged.Broadcast(this);
	}
	else
	{
		FailureReason.AddTag(UE::EnhancedInput::TAG_NoKeyProfile);
	}
}

void UEnhancedInputUserSettings::OnKeyMappingUpdated(FPlayerKeyMapping* ChangedMapping, const FMapPlayerKeyArgs& InArgs, const bool bIsBeingUnmapped)
{
	// Do nothing by default
}

const TSet<FPlayerKeyMapping>& UEnhancedInputUserSettings::FindMappingsInRow(const FName MappingName) const
{
	if (UEnhancedPlayerMappableKeyProfile* KeyProfile = GetActiveKeyProfile())
	{
		FKeyMappingRow& ExistingMappings = KeyProfile->PlayerMappedKeys.FindOrAdd(MappingName);
		return ExistingMappings.Mappings;
	}
	
	UE_LOGF(LogEnhancedInput, Error, "There is no current mappable key profile! No mappings will be returned.");
	
	static TSet<FPlayerKeyMapping> EmptyMappings;
	return EmptyMappings;
}

const FPlayerKeyMapping* UEnhancedInputUserSettings::FindCurrentMappingForSlot(const FName ActionName, const EPlayerMappableKeySlot InSlot) const
{
	const TSet<FPlayerKeyMapping>& AllMappings = FindMappingsInRow(ActionName);
	for (const FPlayerKeyMapping& Mapping : AllMappings)
	{
		if (Mapping.Slot == InSlot)
		{
			return &Mapping;
		}
	}
	
	UE_LOGF(LogEnhancedInput, Warning, "No mappings could be found for action '%ls'", *ActionName.ToString());
	return nullptr;
}

const UInputAction* UEnhancedInputUserSettings::FindInputActionForMapping(const FName MappingName) const
{
	const TSet<FPlayerKeyMapping>& AllMappings = FindMappingsInRow(MappingName);
	// Return the first instance of the input action from the player mappings. All of them will have the same Input Action pointer
	// as long as they are in the same row.
	for (const FPlayerKeyMapping& Mapping : AllMappings)
	{
		if (const UInputAction* Action = Mapping.GetAssociatedInputAction())
		{
			return Action;
		}
	}
	return nullptr;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UEnhancedInputUserSettings::SetKeyProfile(const FGameplayTag& InProfileId)
{
	return SetActiveKeyProfile(InProfileId.ToString());
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UEnhancedInputUserSettings::SetActiveKeyProfile(const FString& InProfileId)
{
	const FString OriginalProfileId = GetActiveKeyProfileId();
	UEnhancedPlayerMappableKeyProfile* OriginalProfile = GetActiveKeyProfile();

	// If this key profile is already equipped, then there is nothing to do
	if (InProfileId.Equals(OriginalProfileId))
	{
		UE_LOGF(LogEnhancedInput, Log, "Key profile '%ls' is already currently equipped. Nothing will happen.", *InProfileId);
		return false;
	}
	
	if (const TObjectPtr<UEnhancedPlayerMappableKeyProfile>* NewProfile = SavedKeyProfilesMap.Find(InProfileId))
	{
		// Unequip the original profile if there was one
		if (OriginalProfile)
		{
			OriginalProfile->UnEquipProfile();
		}

		// Equip the new profile
		NewProfile->Get()->EquipProfile();

		// Keep track of what the current profile is now
		CurrentProfileIdentifierString = InProfileId;

		// TODO: Register all the input mapping contexts with the new key profile to
		// ensure that it has all the default key mappings

		// Let any listeners know that the mapping profile has changed
		OnKeyProfileChanged.Broadcast(NewProfile->Get());
	}
	else
	{
		UE_LOGF(LogEnhancedInput, Error, "No profile with name '%ls' exists! Did you call CreateNewKeyProfile at any point?", *InProfileId);
		return false;
	}

	UE_LOGF(LogEnhancedInput, Verbose, "Successfully changed Key Profile from '%ls' to '%ls'", *OriginalProfileId, *CurrentProfileIdentifierString);
	return true;
}

bool UEnhancedInputUserSettings::SetKeyProfileToDefault()
{
	if (const UEnhancedPlayerMappableKeyProfile* DefaultProfile = GetDefaultKeyProfile())
	{
		return SetActiveKeyProfile(DefaultProfile->GetProfileIdString());	
	}
	else
	{
		return false;
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FGameplayTag& UEnhancedInputUserSettings::GetCurrentKeyProfileIdentifier() const
{
	return CurrentProfileIdentifier;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const FString& UEnhancedInputUserSettings::GetActiveKeyProfileId() const
{
	return CurrentProfileIdentifierString;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UEnhancedPlayerMappableKeyProfile* UEnhancedInputUserSettings::GetCurrentKeyProfile() const
{
	return GetActiveKeyProfile();	
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UEnhancedPlayerMappableKeyProfile* UEnhancedInputUserSettings::GetActiveKeyProfile() const
{
	return GetKeyProfileWithId(GetActiveKeyProfileId());
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FPlayerMappableKeyProfileCreationArgs::FPlayerMappableKeyProfileCreationArgs()
	: ProfileType(GetDefault<UEnhancedInputDeveloperSettings>()->DefaultPlayerMappableKeyProfileClass.Get())
	, ProfileIdentifier(FGameplayTag::EmptyTag)
	, ProfileStringIdentifier()
	, UserId(PLATFORMUSERID_NONE)
	, DisplayName(FText::GetEmpty())
	, bSetAsCurrentProfile(true)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const TMap<FGameplayTag, TObjectPtr<UEnhancedPlayerMappableKeyProfile>>& UEnhancedInputUserSettings::GetAllSavedKeyProfiles() const
{
	return SavedKeyProfiles;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const TMap<FString, TObjectPtr<UEnhancedPlayerMappableKeyProfile>>& UEnhancedInputUserSettings::GetAllAvailableKeyProfiles() const
{
	return SavedKeyProfilesMap;
}

bool UEnhancedInputUserSettings::IsKeyProfileAvailable(const FString& ProfileId) const
{
	return SavedKeyProfilesMap.Contains(ProfileId);
}

UEnhancedPlayerMappableKeyProfile* UEnhancedInputUserSettings::GetDefaultKeyProfile() const
{
	if (const TObjectPtr<UEnhancedPlayerMappableKeyProfile>* Found = SavedKeyProfilesMap.Find(UE::EnhancedInput::DefaultProfileIdentifierString))
	{
		return Found->Get();
	}
	return nullptr;
}

UEnhancedPlayerMappableKeyProfile* UEnhancedInputUserSettings::CreateNewKeyProfile(const FPlayerMappableKeyProfileCreationArgs& InArgs)
{
	if (!InArgs.ProfileType)
	{
		UE_LOGF(LogEnhancedInput, Error, "Invalid ProfileType given CreateNewKeyProfile!");
		return nullptr;
	}
	
	UEnhancedPlayerMappableKeyProfile* OutProfile = GetKeyProfileWithId(InArgs.ProfileStringIdentifier);
	
	// Check for an existing profile of this name
	if (OutProfile)
	{
		UE_LOGF(LogEnhancedInput, Warning, "A key profile with the name '%ls' already exists! Use a different name.", *InArgs.ProfileStringIdentifier);
	}
	else
	{
		// Create a new mapping profile
		OutProfile = NewObject<UEnhancedPlayerMappableKeyProfile>(/* outer */ this, /* class */ InArgs.ProfileType);
		OutProfile->ProfileIdentifierString = InArgs.ProfileStringIdentifier;
		OutProfile->DisplayName = InArgs.DisplayName;
		OutProfile->OwningUserId = InArgs.UserId;
		
		SavedKeyProfilesMap.Add(InArgs.ProfileStringIdentifier, OutProfile);
		
		// We need to populate this key profile with all the known key mappings so that it's up to date 
		for (const TObjectPtr<const UInputMappingContext>& IMC : RegisteredMappingContexts)
		{
			RegisterKeyMappingsToProfile(*OutProfile, IMC);
		}
	}
	
	// set as current
	if (InArgs.bSetAsCurrentProfile)
	{
		SetActiveKeyProfile(InArgs.ProfileStringIdentifier);
	}

	UE_LOGF(LogEnhancedInput, Verbose, "Completed creation of key mapping profile '%ls'", *OutProfile->ProfileIdentifierString);
	
	return OutProfile;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UEnhancedPlayerMappableKeyProfile* UEnhancedInputUserSettings::GetKeyProfileWithIdentifier(const FGameplayTag& ProfileId) const
{
	return GetKeyProfileWithId(ProfileId.ToString());
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UEnhancedPlayerMappableKeyProfile* UEnhancedInputUserSettings::GetKeyProfileWithId(const FString& ProfileId) const
{
	if (const TObjectPtr<UEnhancedPlayerMappableKeyProfile>* ExistingProfile = SavedKeyProfilesMap.Find(ProfileId))
	{		
		return ExistingProfile->Get();
	}
	return nullptr;
}

bool UEnhancedInputUserSettings::RegisterInputMappingContexts(const TSet<UInputMappingContext*>& MappingContexts)
{
	bool bResult = false;

	for (UInputMappingContext* IMC : MappingContexts)
	{
		bResult &= RegisterInputMappingContext(IMC);
	}

	return bResult;
}

bool UEnhancedInputUserSettings::RegisterInputMappingContext(const UInputMappingContext* IMC)
{
	if (!IMC)
	{
		UE_LOGF(LogEnhancedInput, Error, "Attempting to register a null mapping context with the user settings!");
		ensure(false);
		return false;
	}

	// There is no need to re-register an IMC if it is has already been registered.
	if (RegisteredMappingContexts.Contains(IMC))
	{
		return false;
	}
	
	// Keep track of all the registered IMC's
	RegisteredMappingContexts.Add(IMC);

	return RegisterInputMappingContextInternal(IMC);
}

bool UEnhancedInputUserSettings::RegisterInputMappingContextInternal(const UInputMappingContext* IMC)
{
	if (!IMC)
	{
		UE_LOGF(LogEnhancedInput, Error, "Attempting to register a null mapping context with the user settings!");
		ensure(false);
		return false;
	}	

	bool bResult = true;

	const ULocalPlayer* LP = GetLocalPlayer();

	// Create any key profiles for mapping contexts which have specific overrides if we need to
	const TArray<FString> ProfilesSpecifiedInMappingContext = IMC->GetProfilesWithOverridenMappings();
	for (const FString& ProfileId : ProfilesSpecifiedInMappingContext)
	{
		if (SavedKeyProfilesMap.Contains(ProfileId))
		{
			continue;
		}

		FPlayerMappableKeyProfileCreationArgs Args = {};
		Args.ProfileStringIdentifier = ProfileId;
		Args.UserId = LP ? LP->GetPlatformUserId() : PLATFORMUSERID_NONE;
		Args.DisplayName = UE::EnhancedInput::DefaultProfileDisplayName;
		Args.bSetAsCurrentProfile = false;
		
		CreateNewKeyProfile(Args);
	}

	// Register the mappings of this IMC to every saved key profile
	for (TPair<FString, TObjectPtr<UEnhancedPlayerMappableKeyProfile>>& Pair : SavedKeyProfilesMap)
	{
		bResult &= RegisterKeyMappingsToProfile(*Pair.Value, IMC);
	}

	if (bResult)
	{
		OnMappingContextRegistered.Broadcast(IMC);
	}
	
	return bResult;
}

bool UEnhancedInputUserSettings::RegisterKeyMappingsToProfile(UEnhancedPlayerMappableKeyProfile& Profile, const UInputMappingContext* IMC)
{
	if (!IMC)
	{
		UE_LOGF(LogEnhancedInput, Error, "Attempting to register a null mapping context with the user settings!");
		ensure(false);
		return false;
	}

	typedef TMap<FName, EPlayerMappableKeySlot> FSlotCountToMappingNames;
	static TMap<FHardwareDeviceIdentifier, FSlotCountToMappingNames> MappingsPerDeviceTypes;
	MappingsPerDeviceTypes.Reset();
	
	for (const FEnhancedActionKeyMapping& KeyMapping : IMC->GetMappingsForProfile(Profile.GetProfileIdString()))
	{
		// Skip over non-player mappable keys
		if (!KeyMapping.IsPlayerMappable())
		{
			continue;
		}

		// Only mappings with empty profile or the current profile should be added - same as it's done by IEnhancedInputSubsystemInterface::RebuildControlMappings
		const UPlayerMappableKeySettings* KeySettings = KeyMapping.GetPlayerMappableKeySettings();
		
		// If this mapping has specified a specific key profile, and the current profile isn't it, then don't add this key mapping
		if (KeySettings && !KeySettings->SupportedKeyProfileIds.IsEmpty() && !KeySettings->SupportedKeyProfileIds.Contains(Profile.GetProfileIdString()))
		{
			continue;
		}

		// Get the unique FName for the "Mapping name". This is set on the UInputAction
		// but can be overridden on each FEnhancedActionKeyMapping to create multiple
		// mapping options for a single Input Action. 
		const FName MappingName = KeyMapping.GetMappingName();

		// Find or create a mapping row 
		FKeyMappingRow& MappingRow = Profile.PlayerMappedKeys.FindOrAdd(MappingName);

		// Determine what Slot and Hardware device this Action Mapping should go in
		const FHardwareDeviceIdentifier MappingDeviceType = DetermineHardwareDeviceForActionMapping(KeyMapping, IMC);
		
		FSlotCountToMappingNames& MappingNameToSlotCount = MappingsPerDeviceTypes.FindOrAdd(MappingDeviceType);

		// Determine what slot is available for the mapping to be placed in
		EPlayerMappableKeySlot& MappingSlot = MappingNameToSlotCount.FindOrAdd(MappingName);
		
		bool bUpdatedExistingMapping = false;

		static TArray<FSetElementId> MappingsRegistered;
		MappingsRegistered.Reset();
		
		// Iterate any existing mappings in this row to ensure that the metadata and default key values are correct.
		// At this stage, keys will only exist here if they are player customized
		for (auto It = MappingRow.Mappings.CreateIterator(); It; ++It)
		{			
			FPlayerKeyMapping& ExistingMapping = *It;
			// We only want to update the default _key_ for an existing mapping if it is from
			// the same slot as this Action Mapping would be going in.
			// This is because players can map keys to slots are not defined in the input mapping context,
			// thus their default key should be EKeys::Invalid.
			// For example, you define a single key mapping of "Jump" to space bar in your IMC.
			// The mapping to spacebar is in EPlayerMappableKeySlot::First. The player adds a second
			// key mapping of "Jump" to the X key, and keeps the spacebar mapping. That is in EPlayerMappableKeySlot::Second
			// We should no update the default key of second slot because it's default value should be EKeys::Invalid.
			if (ExistingMapping.GetSlot() == MappingSlot && ExistingMapping.HardwareDeviceId.PrimaryDeviceType == MappingDeviceType.PrimaryDeviceType)
			{
				ExistingMapping.UpdateDefaultKeyFromActionKeyMapping(KeyMapping);
				bUpdatedExistingMapping = true;
			}
			
			// We always want to ensure that the metadata on a player key mapping is up to date
			// from the Input Mapping Context.
			ExistingMapping.UpdateMetadataFromActionKeyMapping(KeyMapping);
			
			// Keep track of this mapping ID in the set so that we can register later
			MappingsRegistered.Emplace(It.GetId());
		}

		// If the mapping was not found in the existing row, then the player has not mapped anything to it.
		// We need to create it based on the original key mapping
		if (!bUpdatedExistingMapping)
		{
			// Add a default mapping to this row
			FSetElementId ElemId = MappingRow.Mappings.Add({ KeyMapping, MappingSlot, MappingDeviceType });

			// Keep track of this mapping ID in the set so that we can register later
			MappingsRegistered.Emplace(ElemId);
		}

		for (const FSetElementId& RegisteredMappingId : MappingsRegistered)
		{
			if (RegisteredMappingId.IsValidId())
			{
				OnKeyMappingRegisteredToProfile(&Profile, MappingRow.Mappings.Get(RegisteredMappingId), KeyMapping);
			}			
		}

		// Increment the mapping slot to keep track of how many slots are in use by which hardware device
		MappingSlot = static_cast<EPlayerMappableKeySlot>(FMath::Min<uint8>(static_cast<uint8>(MappingSlot) + 1, static_cast<uint8>(EPlayerMappableKeySlot::Max)));
	}

	UE_LOGF(LogEnhancedInput, Verbose, "Registered IMC with UEnhancedInputUserSettings: %ls", *IMC->GetFName().ToString());
	return true;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UEnhancedInputUserSettings::OnKeyMappingRegistered(FPlayerKeyMapping& RegisteredMapping, const FEnhancedActionKeyMapping& SourceMapping)
{
	// Does nothing by default
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UEnhancedInputUserSettings::OnKeyMappingRegisteredToProfile(UEnhancedPlayerMappableKeyProfile* Profile, FPlayerKeyMapping& RegisteredMapping, const FEnhancedActionKeyMapping& SourceMapping)
{
	// Call the old version for back compat
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OnKeyMappingRegistered(RegisteredMapping, SourceMapping);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Otherwise, do nothing by default
}

FHardwareDeviceIdentifier UEnhancedInputUserSettings::DetermineHardwareDeviceForActionMapping(const FEnhancedActionKeyMapping& ActionMapping, const UInputMappingContext* IMC) const
{
	return FHardwareDeviceIdentifier::Invalid;
}

bool UEnhancedInputUserSettings::UnregisterInputMappingContext(const UInputMappingContext* IMC)
{
	return RegisteredMappingContexts.Remove(IMC) != INDEX_NONE;
}

bool UEnhancedInputUserSettings::UnregisterInputMappingContexts(const TSet<UInputMappingContext*>& MappingContexts)
{
	bool bResult = false;

	for (UInputMappingContext* IMC : MappingContexts)
	{
		bResult |= UnregisterInputMappingContext(IMC);
	}

	return bResult;
}

const TSet<TObjectPtr<const UInputMappingContext>>& UEnhancedInputUserSettings::GetRegisteredInputMappingContexts() const
{
	return RegisteredMappingContexts;
}

bool UEnhancedInputUserSettings::IsMappingContextRegistered(const UInputMappingContext* IMC) const
{
	return RegisteredMappingContexts.Contains(IMC);
}

#if ENABLE_DRAW_DEBUG

namespace UE::EnhancedInput::DebugColors
{
	static const FColor& HeaderColor = FColor::Magenta;
	static const FColor& TitleColor = FColor::Orange;
	static const FColor& NormalColor = FColor::Silver;
	static const FColor& ErrorColor = FColor::Red;
	static const FColor& KeyMappingRow = FColor::Emerald;
	static const FColor& KeyMappingColor = FColor::Silver;
};

#endif	// ENABLE_DRAW_DEBUG

void UEnhancedInputUserSettings::ShowDebugInfo(UCanvas* Canvas) const
{
#if ENABLE_DRAW_DEBUG

	if (!Canvas)
	{
		return;
	}

	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;

	const ULocalPlayer* LP = GetLocalPlayer();	

	// Draw the "Header" of this debug info
	{
		DisplayDebugManager.SetFont(GEngine->GetLargeFont());
		DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::HeaderColor);
		DisplayDebugManager.DrawString(FString::Printf(TEXT("\n\nInput User Settings (%s)\n------------------------------"), *GetNameSafe(GetClass())));
	}

	// Draw some info about the local player, return if it doesn't exist.
	{
		DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::TitleColor);
		DisplayDebugManager.DrawString(TEXT("\t Local Player:"));

		if (!LP)
		{
			DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::ErrorColor);
			DisplayDebugManager.DrawString(TEXT("Invalid Local Player on the Input User Settings!"));
			return;
		}
		else
		{			
			DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::NormalColor);
			DisplayDebugManager.DrawString(FString::Printf(TEXT("\t \t %s \n \t PlatformUserId: %d"), *LP->GetFName().ToString(), LP->GetPlatformUserId().GetInternalId()));			
		}
	}

	ShowDebugInfoInternal(Canvas);

	// Draw the "Footer" of this debug info
	{
		DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::HeaderColor);
		DisplayDebugManager.DrawString(TEXT("----------------------------\n(end of Input User Settings)\n------------------------------\n"));
	}	

#endif	// ENABLE_DRAW_DEBUG
}

void UEnhancedInputUserSettings::ShowDebugInfoInternal(UCanvas* Canvas) const
{
#if ENABLE_DRAW_DEBUG
	check(Canvas);

	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;

	DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::TitleColor);
	DisplayDebugManager.DrawString(FString::Printf(TEXT("Player Key Profiles")));
	DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::NormalColor);

	// number of key profiles	
	DisplayDebugManager.DrawString(FString::Printf(TEXT("\t There are '%d' saved key profiles"), SavedKeyProfilesMap.Num()));

	// currently active profile
	if (UEnhancedPlayerMappableKeyProfile* CurrentProfile = GetActiveKeyProfile())
	{
		DisplayDebugManager.DrawString(FString::Printf(TEXT("\t Currently Active Profile: %s (%s)"), *CurrentProfile->GetProfileIdString(), *GetNameSafe(CurrentProfile->GetClass())));
	}
	else
	{
		DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::ErrorColor);
		DisplayDebugManager.DrawString(TEXT("There is no currently active profile!"));
	}

	// Draw debug info about the player mappings
	for (const TPair<FString, TObjectPtr<UEnhancedPlayerMappableKeyProfile>>& ProfilePair : SavedKeyProfilesMap)
	{
		UEnhancedPlayerMappableKeyProfile* Profile = ProfilePair.Value;
		if (!Profile)
		{
			continue;
		}

		const UEnum* const SlotEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/EnhancedInput.EPlayerMappableKeySlot"));
		const auto& GetSlotString = [&SlotEnum](const EPlayerMappableKeySlot Value) -> FString
		{
			if (!SlotEnum)
			{
				return TEXT("Unknown Slot Enum State!");
			}

			return SlotEnum->GetNameStringByValue(static_cast<int64>(Value));
		};

		DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::TitleColor);
		DisplayDebugManager.DrawString(FString::Printf(TEXT("Profile: %s (%s)\n"), *Profile->GetProfileIdString(), *GetNameSafe(Profile->GetClass())));
		
		DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::NormalColor);
		DisplayDebugManager.DrawString(TEXT("\t -------------------- "));

		DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::TitleColor);
		DisplayDebugManager.DrawString(TEXT("\t Player Key Mappings:\n"));

		for (const TPair<FName, FKeyMappingRow>& Pair : Profile->PlayerMappedKeys)
		{			
			DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::KeyMappingRow);
			DisplayDebugManager.DrawString(FString::Printf(TEXT("Mapping Row: '%s' \t \t has '%d' Mappings"), *Pair.Key.ToString(), Pair.Value.Mappings.Num()));

			for (const FPlayerKeyMapping& Mapping : Pair.Value.Mappings)
			{
				TStringBuilder<1024> Builder;		
				Builder.Appendf(TEXT("\t| Slot: %s   \t"), *GetSlotString(Mapping.GetSlot()));
				Builder.Appendf(TEXT("\t| Current: %s   \t"), *Mapping.GetCurrentKey().GetDisplayName().ToString());
				Builder.Appendf(TEXT("\t| Default: %s   \t"), *Mapping.GetDefaultKey().GetDisplayName().ToString());
				Builder.Appendf(TEXT("\t| Dirty: %s   "), Mapping.IsDirty() ? TEXT("true") : TEXT("false"));
			
				// Flip flop the color for each key mapping to make the debug screen easier to read
				DisplayDebugManager.SetDrawColor(UE::EnhancedInput::DebugColors::KeyMappingColor);
				DisplayDebugManager.DrawString(Builder.ToString());
			}
		}
	}

#endif	// ENABLE_DRAW_DEBUG
}

FPlayerMappableKeyQueryOptions::FPlayerMappableKeyQueryOptions()
	: MappingName(NAME_None)
	, KeyToMatch(EKeys::Invalid)
	, SlotToMatch(EPlayerMappableKeySlot::Unspecified)
	, bMatchBasicKeyTypes(true)
	, bMatchKeyAxisType(false)
	, RequiredDeviceType(EHardwareDevicePrimaryType::Unspecified)
	, RequiredDeviceFlags(EHardwareDeviceSupportedFeatures::Type::Unspecified)
{ }

#undef LOCTEXT_NAMESPACE
