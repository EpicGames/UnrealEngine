// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputAction.h"

#include "EnhancedActionKeyMapping.generated.h"

#define UE_API ENHANCEDINPUT_API

enum class EDataValidationResult : uint8;
class UInputModifier;
class UInputTrigger;
class UPlayerMappableKeySettings;

/**
* Defines which Player Mappable Key Setting to use for a Action Key Mapping.
*/
UENUM(BlueprintType)
enum class EPlayerMappableKeySettingBehaviors : uint8
{
	//Use the Settings specified in the Input Action.
	InheritSettingsFromAction,
	//Use the Settings specified in the Action Key Mapping overriding the ones specified in the Input action.
	OverrideSettings,
	//Don't use any Settings even if one is specified in the Input Action.
	IgnoreSettings
};

/**
 * Defines a mapping between a key activation and the resulting enhanced action
 * An key could be a button press, joystick axis movement, etc.
 * An enhanced action could be MoveForward, Jump, Fire, etc.
 *
**/
USTRUCT(BlueprintType)
struct FEnhancedActionKeyMapping
{
	friend class UInputMappingContext;
	friend class FEnhancedActionMappingCustomization;
	
	GENERATED_BODY()

	UE_API FEnhancedActionKeyMapping(const UInputAction* InAction = nullptr, const FKey InKey = EKeys::Invalid);
	
	/**
	* Returns the Player Mappable Key Settings owned by the Action Key Mapping or by the referenced Input Action, or nothing based of the Setting Behavior.
	*/
	UE_API UPlayerMappableKeySettings* GetPlayerMappableKeySettings() const;

	/**
	 * Returns the name of the mapping based on setting behavior used. If no name is found in the Mappable Key Settings it will return the name set in Player Mappable Options if bIsPlayerMappable is true.
	 */
	UE_API FName GetMappingName() const;

	/** The localized display name of this key mapping */
	UE_API const FText& GetDisplayName() const;

	/** The localized display category of this key mapping */
	UE_API const FText& GetDisplayCategory() const;

	/**
	* Returns true if this Action Key Mapping either holds a Player Mappable Key Settings or is set bIsPlayerMappable.
	*/
	UE_API bool IsPlayerMappable() const;

#if WITH_EDITOR
	UE_API EDataValidationResult IsDataValid(FDataValidationContext& Context) const;
#endif
	
	/** Identical comparison, including Triggers and Modifiers current inner values. */
	UE_API bool operator==(const FEnhancedActionKeyMapping& Other) const;
	
	/**
	* Trigger qualifiers. If any trigger qualifiers exist the mapping will not trigger unless:
	* If there are any Explicit triggers in this list at least one of them must be met.
	* All Implicit triggers in this list must be met.
	*/
	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, Category = "Input")
	TArray<TObjectPtr<UInputTrigger>> Triggers;

	/**
	* Modifiers applied to the raw key value.
	* These are applied sequentially in array order.
	* 
	* Note: Modifiers defined in individual key mappings will be applied before those defined in the Input Action asset.
	* Modifiers will not override any that are defined on the Input Action asset, they will be combined together during evaluation.
	*/
	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, Category = "Input")
	TArray<TObjectPtr<UInputModifier>> Modifiers;
	
	/** Action to be affected by the key  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input")
	TObjectPtr<const UInputAction> Action = nullptr;

	/** Key that affect the action. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input")
	FKey Key;

	/**
	 * If true, then this Key Mapping should be ignored. This is set to true if the key is down
	 * during a rebuild of it's owning PlayerInput ControlMappings.
	 * 
	 * @see IEnhancedInputSubsystemInterface::RebuildControlMappings
	 */
	UPROPERTY(Transient)
	uint8 bShouldBeIgnored : 1;

	/**
	 * True if any triggers on this mapping or its associated UInputAction
	 * are flagged as "always tick" triggers. This is only set when control
	 * mappings are rebuilt.
	 *
	 * @see UInputTrigger::bShouldAlwaysTick
	 * @see IEnhancedInputSubsystemInterface::ReorderMappings
	 */
	UPROPERTY(Transient)
	uint8 bHasAlwaysTickTrigger : 1;

protected:

	/**
	* Defines which Player Mappable Key Setting to use for a Action Key Mapping.
	*/
	UPROPERTY(EditAnywhere, Category = "Input|Settings")
	EPlayerMappableKeySettingBehaviors SettingBehavior = EPlayerMappableKeySettingBehaviors::InheritSettingsFromAction;

	/**
	* Used to expose this mapping or to opt-out of settings completely.
	*/
	UPROPERTY(EditAnywhere, Instanced, Category = "Input|Settings", meta = (EditCondition = "SettingBehavior == EPlayerMappableKeySettingBehaviors::OverrideSettings", DisplayAfter = "SettingBehavior"))
	TObjectPtr<UPlayerMappableKeySettings> PlayerMappableKeySettings = nullptr;

public:

	template<typename T = UPlayerMappableKeySettings> 
	T* GetPlayerMappableKeySettings() const
	{
		return Cast<T>(GetPlayerMappableKeySettings());
	}

	/**
	 * If the template bIgnoreModifierAndTriggerValues is true, compare to Other ignoring
	 * different trigger states, like pending activation time, but only accept
	 * both as equal if the Trigger types are the same and in the same order.
	 */
	template<bool bIgnoreModifierAndTriggerValues = true>
	bool Equals(const FEnhancedActionKeyMapping& Other) const
	{
		if constexpr (bIgnoreModifierAndTriggerValues)
		{
			return (Action == Other.Action &&
					Key == Other.Key &&
					CompareByObjectTypes(Modifiers, Other.Modifiers) &&
					CompareByObjectTypes(Triggers, Other.Triggers));
		}
		else
		{
			return *this == Other;
		}
	}
	
	/**
	 * Compares if two TArray of UObjects contain the same number and types of
	 * objects, but doesn't compare their values.
	 *
	 * This is needed because TArray::operator== returns false when the objects'
	 * inner values differ. And for keeping old Trigger states, we need their
	 * comparison to ignore their current values.
	 */
	template<typename T>
	static bool CompareByObjectTypes(const TArray<TObjectPtr<T>>& A, const TArray<TObjectPtr<T>>& B)
	{
		if (A.Num() != B.Num())
		{
			return false;
		}

		for (int32 Idx = 0; Idx < A.Num(); ++Idx)
		{
			const T* ObjectA = A[Idx];
			const T* ObjectB = B[Idx];

			if ((ObjectA == nullptr) != (ObjectB == nullptr))
			{
				// One is nullptr, the other isn't
				return false;
			}
			if (!ObjectA)
			{
				// Both are nullptr. Consider that as same type.
				continue;
			}
			
			const UClass* ClassA = ObjectA->GetClass();
			const UClass* ClassB = ObjectB->GetClass();
			
			if (ClassA != ClassB)
			{
				// If the classes differ then they are not the same
				return false;
			}
			
			if (ClassA->GetDefaultObject() != ClassB->GetDefaultObject())
			{
				// If the default objects of these two objects differ, then they should be treated differently
				return false;
			}
		}

		return true;
	}

};

#undef UE_API
