// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/InputDelegateBinding.h"
#include "InputTriggers.h"
#include "EnhancedInputActionDelegateBinding.generated.h"

#define UE_API ENHANCEDINPUT_API

class UInputComponent;

USTRUCT()
struct FBlueprintEnhancedInputActionBinding
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<const UInputAction> InputAction = nullptr;

	UPROPERTY()
	ETriggerEvent TriggerEvent = ETriggerEvent::None;

	UPROPERTY()
	FName FunctionNameToBind = NAME_None;

	// TODO: bDevelopmentOnly;	// This action delegate will not fire in shipped builds (debug/cheat actions)
};



UCLASS(MinimalAPI)
class UEnhancedInputActionDelegateBinding : public UInputDelegateBinding
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FBlueprintEnhancedInputActionBinding> InputActionDelegateBindings;

	//~ Begin UInputDelegateBinding Interface
	UE_API virtual void BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const override;
	//~ End UInputDelegateBinding Interface
};

UCLASS(MinimalAPI)
class UEnhancedInputActionValueBinding : public UInputDelegateBinding
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FBlueprintEnhancedInputActionBinding> InputActionValueBindings;

	//~ Begin UInputDelegateBinding Interface
	UE_API virtual void BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const override;
	//~ End UInputDelegateBinding Interface
};

#undef UE_API
