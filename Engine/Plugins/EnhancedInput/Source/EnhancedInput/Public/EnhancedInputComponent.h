// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/InputComponent.h"
#include "InputAction.h"

#include "EnhancedInputComponent.generated.h"

#define UE_API ENHANCEDINPUT_API

// TODO: Disable in test too?
#ifndef DEV_ONLY_KEY_BINDINGS_AVAILABLE
	#define DEV_ONLY_KEY_BINDINGS_AVAILABLE (!UE_BUILD_SHIPPING)
#endif 

enum class ETriggerEvent : uint8;

/** Delegate signature for debug key events. */
DECLARE_DELEGATE_TwoParams(FInputDebugKeyHandlerSignature, FKey, FInputActionValue);
DECLARE_DYNAMIC_DELEGATE_TwoParams(FInputDebugKeyHandlerDynamicSignature, FKey, Key, FInputActionValue, ActionValue);

/** Delegate signature for action events. */
DECLARE_DELEGATE(FEnhancedInputActionHandlerSignature);
DECLARE_DELEGATE_OneParam(FEnhancedInputActionHandlerValueSignature, const FInputActionValue&);
DECLARE_DELEGATE_OneParam(FEnhancedInputActionHandlerInstanceSignature, const FInputActionInstance&);	// Provides full access to value and timers
/**
 * @param ActionValue		The combined value of all inputs mapped to this action.
 * @param ElapsedTime		The amount of time in seconds that has elapsed since this input action has begun processing its input values. 
 * @param TriggeredTime		The amount of time in seconds that this input action has been in the "triggered" state.
 * @param SourceAction		The input action which caused this event to fire. 
 */
DECLARE_DYNAMIC_DELEGATE_FourParams(FEnhancedInputActionHandlerDynamicSignature, FInputActionValue, ActionValue, float, ElapsedTime, float, TriggeredTime, const UInputAction*, SourceAction);

// By default this behavior is on in the editor only. If you would like to turn this
// off, then add the following to your Build.cs file:
//		PublicDefinitions.Add("ENABLE_EDITOR_CALLABLE_INPUT_DELEGATES=0");
#ifndef ENABLE_EDITOR_CALLABLE_INPUT_DELEGATES
	#define ENABLE_EDITOR_CALLABLE_INPUT_DELEGATES	WITH_EDITOR
#elif !WITH_EDITOR
	#define ENABLE_EDITOR_CALLABLE_INPUT_DELEGATES	0
#endif

/** Unified storage for both native and dynamic delegates with any signature  */
template<typename TSignature>
struct TEnhancedInputUnifiedDelegate
{
protected:
	/** Holds the delegate to call. */
	TSharedPtr<TSignature> Delegate;

	/** Should this delegate fire with an Editor Script function guard? */
	bool bShouldFireWithEditorScriptGuard = false;

public:
	UObject* GetUObject() const
	{
		return IsBound() ? Delegate->GetUObject() : nullptr;
	}
	
	bool IsBound() const
	{
		return Delegate.IsValid() && Delegate->IsBound();
	}

	bool IsBoundToObject(FDelegateUserObjectConst Object) const
	{
		return IsBound() && Delegate->IsBoundToObject(Object);
	}

	void Unbind()
	{
		if (Delegate)
		{
			Delegate->Unbind();
		}
	}

	void SetShouldFireWithEditorScriptGuard(const bool bNewValue) { bShouldFireWithEditorScriptGuard = bNewValue; }
	
	bool ShouldFireWithEditorScriptGuard() const { return bShouldFireWithEditorScriptGuard; }

	/** Binds a native delegate, hidden for script delegates */
	template<	typename UserClass,
				typename TSig = TSignature,
				typename... TVars>
	void BindDelegate(UserClass* Object, typename TSig::template TMethodPtr<UserClass, TVars...> Func, TVars... Vars)
	{
		Unbind();
		Delegate = MakeShared<TSig>(TSig::CreateUObject(Object, Func, Vars...));
	}

	/** Binds a script delegate on an arbitrary UObject, hidden for native delegates */
	template<	typename TSig = TSignature,
				typename = typename TEnableIf<TIsDerivedFrom<TSig, FScriptDelegate>::IsDerived || TIsDerivedFrom<TSig, FMulticastScriptDelegate>::IsDerived>::Type>
	void BindDelegate(UObject* Object, const FName FuncName)
	{
		Unbind();
		Delegate = MakeShared<TSig>();
		Delegate->BindUFunction(Object, FuncName);
	}

	/** Binds a lambda expression */
	template<typename TSig = TSignature, typename FunctorType, typename... VarTypes>
	void BindLambda(FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		Unbind();
		Delegate = MakeShared<TSig>();
		Delegate->BindLambda(Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...);
	}

	template<typename TSig = TSignature>
	TSig& MakeDelegate()
	{
		Unbind();
		Delegate = MakeShared<TSig>();
		return *Delegate;
	}

	template<typename... TArgs>
	void Execute(TArgs... Args) const
	{
		if (IsBound())
		{
#if ENABLE_EDITOR_CALLABLE_INPUT_DELEGATES
			if (bShouldFireWithEditorScriptGuard)
			{
				FEditorScriptExecutionGuard ScriptGuard;
				Delegate->Execute(Args...);
			}
			else
			{
				Delegate->Execute(Args...);	
			}
#else
			Delegate->Execute(Args...);
#endif	// ENABLE_EDITOR_CALLABLE_INPUT_DELEGATES
			
		}
	}
};


// Used to force clone constructor calls only
enum class EInputBindingClone : uint8 { ForceClone };

/** A basic binding unique identifier */
struct FInputBindingHandle
{
private:
	uint32 Handle = 0;

protected:
	ENHANCEDINPUT_API FInputBindingHandle();	// Generates a handle
	FInputBindingHandle(const FInputBindingHandle& CloneFrom, EInputBindingClone) : Handle(CloneFrom.Handle) {}	// Clones a handle

public:
	virtual ~FInputBindingHandle() = default;

	bool operator==(const FInputBindingHandle& Rhs) const
	{
		return (GetHandle() == Rhs.GetHandle());
	}

	uint32 GetHandle() const { return Handle; }
};

/** A binding to an input action trigger event. */
struct FEnhancedInputActionEventBinding : public FInputBindingHandle
{
private:

	/** Action against which we are bound */
	TWeakObjectPtr<const UInputAction> Action;

	/** Trigger event that raises the delegate */
	ETriggerEvent TriggerEvent = ETriggerEvent::None;

	/**
	 * If true, then this event binding will stop any other listeners to the same UInputAction with a lower input component priority
	 * on the stack.
	 */
	uint8 bConsumes : 1 = false;

protected:
	// Clone constructor
	FEnhancedInputActionEventBinding(const FEnhancedInputActionEventBinding& CloneFrom, EInputBindingClone Clone) :
		FInputBindingHandle(CloneFrom, Clone),
		Action(CloneFrom.Action), 
		TriggerEvent(CloneFrom.TriggerEvent), 
		bConsumes(CloneFrom.bConsumes) {}

public:
	FEnhancedInputActionEventBinding() = default;
	FEnhancedInputActionEventBinding(
		const UInputAction* InAction,
		ETriggerEvent InTriggerEvent,
		const bool bRequestsConsumption = false)
	: Action(InAction), TriggerEvent(InTriggerEvent), bConsumes(bRequestsConsumption) {}

	const UInputAction* GetAction() const { return Action.Get(); }
	bool ShouldConsume() const { return bConsumes; } 
	ETriggerEvent GetTriggerEvent() const { return TriggerEvent; }

	void SetShouldConsume(const bool bNewConsumeValue) { bConsumes = bNewConsumeValue; }
	
	virtual void Execute(const FInputActionInstance& ActionData) const = 0;
	virtual TUniquePtr<FEnhancedInputActionEventBinding> Clone() const = 0;
	virtual void SetShouldFireWithEditorScriptGuard(const bool bNewValue) = 0;
	virtual bool IsBoundToObject(FDelegateUserObjectConst Object) const = 0;
	virtual UObject* GetUObject() const { return nullptr; }
};

/** Binds an action value for later reference. CurrentValue will be kept up to date with the value of the bound action */
struct FEnhancedInputActionValueBinding : public FInputBindingHandle
{
private:
	friend class UEnhancedPlayerInput;

	/** Action against which we are bound */
	TWeakObjectPtr<const UInputAction> Action;

	/** Copy of the current value of the action */
	mutable FInputActionValue CurrentValue;

public:
	FEnhancedInputActionValueBinding() = default;
	FEnhancedInputActionValueBinding(const UInputAction* InAction) : Action(InAction) {}

	const UInputAction* GetAction() const { return Action.Get(); }
	FInputActionValue GetValue() const { return CurrentValue; }
};

/** Binds a delegate to an event on a key chord. */
struct FInputDebugKeyBinding : public FInputBindingHandle
{
protected:
	// Clone constructor
	FInputDebugKeyBinding(const FInputDebugKeyBinding& CloneFrom, EInputBindingClone Clone)
		: FInputBindingHandle(CloneFrom, Clone)
		, KeyEvent(CloneFrom.KeyEvent)
		, bExecuteWhenPaused(CloneFrom.bExecuteWhenPaused)
		, Chord(CloneFrom.Chord)
	{ }

public:
	/** Key event to bind it to (e.g. pressed, released, double click) */
	TEnumAsByte<EInputEvent> KeyEvent = EInputEvent::IE_Pressed;

	bool bExecuteWhenPaused = false;

	/** Input Chord to bind to */
	FInputChord Chord;

	//ControllerId ControllerId;	// TODO: Controller id/player id (hybrid?) allowing binding e.g. multiple pads to a series of debug actions. Make this part of FInputChord?

	FInputDebugKeyBinding() = default;
	FInputDebugKeyBinding(const FInputChord InChord, const EInputEvent InKeyEvent, bool bInExecuteWhenPaused)
		: KeyEvent(InKeyEvent)
		, bExecuteWhenPaused(bInExecuteWhenPaused)
		, Chord(InChord)
	{ }

	virtual void Execute(const FInputActionValue& ActionValue) const = 0;
	virtual TUniquePtr<FInputDebugKeyBinding> Clone() const = 0;
};


/**
 * Binding wrapper structs.
 * You will need to create these to do manual binding.
 * They permit storage of delegates with differing signatures within a single array of bindings.
 */

template<typename TSignature>
struct FEnhancedInputActionEventDelegateBinding : FEnhancedInputActionEventBinding
{
private:
	FEnhancedInputActionEventDelegateBinding(const FEnhancedInputActionEventDelegateBinding<TSignature>& CloneFrom, EInputBindingClone Clone)
		: FEnhancedInputActionEventBinding(CloneFrom, Clone), Delegate(CloneFrom.Delegate)
	{
		Delegate.SetShouldFireWithEditorScriptGuard(CloneFrom.Delegate.ShouldFireWithEditorScriptGuard());
	}
public:
	FEnhancedInputActionEventDelegateBinding(const UInputAction* Action, ETriggerEvent InTriggerEvent, const bool bRequestsConsumption = false)
		: FEnhancedInputActionEventBinding(Action, InTriggerEvent, bRequestsConsumption)
	{}

	// Implemented below.
	virtual void Execute(const FInputActionInstance& ActionData) const override;
	virtual TUniquePtr<FEnhancedInputActionEventBinding> Clone() const override
	{
		return TUniquePtr<FEnhancedInputActionEventBinding>(new FEnhancedInputActionEventDelegateBinding<TSignature>(*this, EInputBindingClone::ForceClone));
	}

	virtual void SetShouldFireWithEditorScriptGuard(const bool bNewValue) override
	{
		Delegate.SetShouldFireWithEditorScriptGuard(bNewValue);
	}

	virtual bool IsBoundToObject(FDelegateUserObjectConst Object) const override
	{
		return Delegate.IsBoundToObject(Object);
	}

	virtual UObject* GetUObject() const override
	{
		return Delegate.GetUObject();
	}

	TEnhancedInputUnifiedDelegate<TSignature> Delegate;
};

template<typename TSignature>
struct FInputDebugKeyDelegateBinding : FInputDebugKeyBinding
{
private:
	FInputDebugKeyDelegateBinding(const FInputDebugKeyDelegateBinding<TSignature>& CloneFrom, EInputBindingClone Clone) : FInputDebugKeyBinding(CloneFrom, Clone), Delegate(CloneFrom.Delegate) {}
public:
	FInputDebugKeyDelegateBinding(const FInputChord Chord, const EInputEvent KeyEvent, bool bExecuteWhenPaused) : FInputDebugKeyBinding(Chord, KeyEvent, bExecuteWhenPaused) {}

	virtual void Execute(const FInputActionValue& ActionValue) const override
	{
		Delegate.Execute(Chord.Key, ActionValue);	// TODO: Remove FKey param? We don't support AnyKey, so it isn't terribly useful.
	}
	virtual TUniquePtr<FInputDebugKeyBinding> Clone() const override 
	{
		return TUniquePtr<FInputDebugKeyBinding>(new FInputDebugKeyDelegateBinding<TSignature>(*this, EInputBindingClone::ForceClone));
	}

	TEnhancedInputUnifiedDelegate<TSignature> Delegate;
};


// Action event delegate execution by signature.

template<>
inline void FEnhancedInputActionEventDelegateBinding<FEnhancedInputActionHandlerSignature>::Execute(const FInputActionInstance& ActionData) const
{
	Delegate.Execute();
}

template<>
inline void FEnhancedInputActionEventDelegateBinding<FEnhancedInputActionHandlerValueSignature>::Execute(const FInputActionInstance& ActionData) const
{
	Delegate.Execute(ActionData.GetValue());
}

template<>
inline void FEnhancedInputActionEventDelegateBinding<FEnhancedInputActionHandlerInstanceSignature>::Execute(const FInputActionInstance& ActionData) const
{
	Delegate.Execute(ActionData);
}

template<>
inline void FEnhancedInputActionEventDelegateBinding<FEnhancedInputActionHandlerDynamicSignature>::Execute(const FInputActionInstance& ActionData) const
{
	Delegate.Execute(ActionData.GetValue(), ActionData.GetElapsedTime(), ActionData.GetTriggeredTime(), ActionData.GetSourceAction());
}



/**
 * Implement an Actor component for input bindings.
 *
 * An Enhanced Input Component is a transient component that enables an Actor to bind enhanced actions to delegate functions, or monitor those actions.
 * Input components are processed from a stack managed by the PlayerController and processed by the PlayerInput.
 * These bindings will not consume input events, but this behaviour can be replicated using UInputMappingContext::Priority.
 *
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Input/index.html
 */
UCLASS(MinimalAPI, transient, config=Input, hidecategories=(Activation, "Components|Activation"))
class UEnhancedInputComponent
	: public UInputComponent
{
	GENERATED_UCLASS_BODY()

protected:

	/** The collection of action bindings. */
	TArray<TUniquePtr<FEnhancedInputActionEventBinding>> EnhancedActionEventBindings;

	/** The collection of action value bindings. These do not have delegates and are used to store a copy of the current action value only. */
	TArray<FEnhancedInputActionValueBinding> EnhancedActionValueBindings;	// TODO: TSortedMap?

	/** Debug key bindings, available in non-shipping builds only. */
	TArray<TUniquePtr<FInputDebugKeyBinding>> DebugKeyBindings;

	/**
	 * If true, then this input component's delegate bindings will fire with a FEditorScriptExecutionGuard around it.
	 * This means that the events bound to this component can be triggered outside of "-game" scenarios, such as in
	 * the level editor.
	 * NOTE: Enabling this can interfere with events processed on gameplay actors (UE-148305) 
	 */
	bool bShouldFireDelegatesInEditor = false;

	/** Whether to check when evaluating the input stack whether legacy inputs can consume keys  */
	bool bShouldCheckForLegacyKeyConsumption = false;

public:

	/**
	 * Sets whether this component supports checking for legacy input binds that might consume keys
	 *
	 */
	void SetShouldCheckForLegacyKeyConsumption(bool bNewValue) { bShouldCheckForLegacyKeyConsumption = bNewValue; }

	/**
	 * Whether this component supports checking for legacy input binds that might consume keys
	 *
	 */
	UE_API bool ShouldCheckForLegacyKeyConsumption() const;

	UE_API void SetShouldFireDelegatesInEditor(const bool bInNewValue);
	
	bool ShouldFireDelegatesInEditor() const { return bShouldFireDelegatesInEditor; }

	/**
	 * Checks whether this component has any input bindings.
	 *
	 * @return true if any bindings are set, false otherwise.
	 */
	UE_API virtual bool HasBindings() const override;

	/**
	 * Removes all action bindings.
	 *
	 */
	void ClearActionEventBindings() { EnhancedActionEventBindings.Reset(); }
	void ClearActionValueBindings() { EnhancedActionValueBindings.Reset(); }
	void ClearDebugKeyBindings() { DebugKeyBindings.Reset(); }

	/** Removes all action bindings. */
	UE_API virtual void ClearActionBindings() override;
	
	/** Clears any input callback delegates from the given UObject */
	UE_API virtual void ClearBindingsForObject(UObject* InOwner) override;

	/**
	 * Gets all action bindings of type
	 *
	 * @see BindAction, ClearActionEventBindings, GetNumActionEventBindings, RemoveActionEventBinding
	*/
	const TArray<TUniquePtr<FEnhancedInputActionEventBinding>>& GetActionEventBindings() const { return EnhancedActionEventBindings; }
	const TArray<FEnhancedInputActionValueBinding>& GetActionValueBindings() const { return EnhancedActionValueBindings; }
	const TArray<TUniquePtr<FInputDebugKeyBinding>>& GetDebugKeyBindings() const { return DebugKeyBindings; }

	/**
	 * Removes the action binding at the specified index.
	 *
	 * @param BindingIndex The index of the binding to remove.
	 * @return Whether the binding was successfully removed.
	 * @see BindAction, ClearActionEventBindings, GetActionEventBinding, GetNumActionEventBindings
	 */
	UE_API bool RemoveActionEventBinding(const int32 BindingIndex);
	UE_API bool RemoveDebugKeyBinding(const int32 BindingIndex);
	UE_API bool RemoveActionValueBinding(const int32 BindingIndex);

	/**
	 * Removes the binding with the specified handle. Binding handles are unique across all bindings.
	 *
	 * @param Handle The handle of the binding to remove.
	 * @return Whether the binding was successfully removed.
	 * @see BindAction, ClearActionEventBindings, GetActionEventBinding, GetNumActionEventBindings
	 */
	UE_API bool RemoveBindingByHandle(const uint32 Handle);

	/**
	 * Removes an arbitrary binding
	 *
	 * @param BindingToRemove The binding to remove, as returned by BindAction/BindActionValue/BindDebugKey
	 * @return Whether the binding was successfully removed.
	 * @see BindAction, BindDebugKey, ClearActionEventBindings, ClearDebugKeyBindings
	 */
	UE_API bool RemoveBinding(const FInputBindingHandle& BindingToRemove);

	/**
	 * Binds a delegate function matching any of the handler signatures to a UInputAction assigned via UInputMappingContext to the owner of this component.
	 */
#define DEFINE_BIND_ACTION(HANDLER_SIG)																																			\
	template<class UserClass, typename... VarTypes>																																					\
	FEnhancedInputActionEventBinding& BindAction(const UInputAction* Action, ETriggerEvent TriggerEvent, UserClass* Object, typename HANDLER_SIG::template TMethodPtr< UserClass, VarTypes... > Func, VarTypes... Vars) \
	{																																											\
		TUniquePtr<FEnhancedInputActionEventDelegateBinding<HANDLER_SIG>> AB = MakeUnique<FEnhancedInputActionEventDelegateBinding<HANDLER_SIG>>(Action, TriggerEvent);			\
		AB->Delegate.BindDelegate<UserClass>(Object, Func, Vars...);																														\
		AB->Delegate.SetShouldFireWithEditorScriptGuard(bShouldFireDelegatesInEditor);																							\
		return *EnhancedActionEventBindings.Add_GetRef(MoveTemp(AB));																											\
	}

	DEFINE_BIND_ACTION(FEnhancedInputActionHandlerSignature);
	DEFINE_BIND_ACTION(FEnhancedInputActionHandlerValueSignature);
	DEFINE_BIND_ACTION(FEnhancedInputActionHandlerInstanceSignature);

	/**
	 * Binds to an object UFUNCTION
	 */
	FEnhancedInputActionEventBinding& BindAction(const UInputAction* Action, ETriggerEvent TriggerEvent, UObject* Object, FName FunctionName)
	{
		TUniquePtr<FEnhancedInputActionEventDelegateBinding<FEnhancedInputActionHandlerDynamicSignature>> AB = MakeUnique<FEnhancedInputActionEventDelegateBinding<FEnhancedInputActionHandlerDynamicSignature>>(Action, TriggerEvent);
		AB->Delegate.BindDelegate(Object, FunctionName);
		AB->Delegate.SetShouldFireWithEditorScriptGuard(bShouldFireDelegatesInEditor);
		return *EnhancedActionEventBindings.Add_GetRef(MoveTemp(AB));
	}

	/**
	 * Binds a lambda expression to receive the input action value form an input event.
	 * Example:
	 *              InputComponent->BindActionValueLambda(
	 *					TestAction,
	 *					ETriggerEvent::Triggered,
	 *					[](const FInputActionValue& InputActionValue)
	 *					{
	 *						// Do you work here!
	 *					});
	 */
	template<typename FunctorType, typename... VarTypes>
	FEnhancedInputActionEventBinding& BindActionValueLambda(const UInputAction* Action, ETriggerEvent TriggerEvent, FunctorType&& InFunctor, VarTypes&&... Vars)
	{
		TUniquePtr<FEnhancedInputActionEventDelegateBinding<FEnhancedInputActionHandlerValueSignature>> AB = MakeUnique<FEnhancedInputActionEventDelegateBinding<FEnhancedInputActionHandlerValueSignature>>(Action, TriggerEvent);
	
		AB->Delegate.BindLambda(Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...);
	
		AB->Delegate.SetShouldFireWithEditorScriptGuard(bShouldFireDelegatesInEditor);
		return *EnhancedActionEventBindings.Add_GetRef(MoveTemp(AB));
	}

	/**
	 * Bind a lambda to receive the action instance data from an input event.
	 * Example:
	 *			InputComponent->BindActionInstanceLambda(
	 *					TestAction,
	 *					ETriggerEvent::Triggered,
	 *					[](const FInputActionInstance& ActionInstance)
	 *					{
	 *						// Do you work here!
	 *					});
	 */
	template<typename FunctorType, typename... VarTypes>
    FEnhancedInputActionEventBinding& BindActionInstanceLambda(const UInputAction* Action, ETriggerEvent TriggerEvent, FunctorType&& InFunctor, VarTypes&&... Vars)
    {
    	TUniquePtr<FEnhancedInputActionEventDelegateBinding<FEnhancedInputActionHandlerInstanceSignature>> AB = MakeUnique<FEnhancedInputActionEventDelegateBinding<FEnhancedInputActionHandlerInstanceSignature>>(Action, TriggerEvent);
    
    	AB->Delegate.BindLambda(Forward<FunctorType>(InFunctor), Forward<VarTypes>(Vars)...);
    
    	AB->Delegate.SetShouldFireWithEditorScriptGuard(bShouldFireDelegatesInEditor);
    	return *EnhancedActionEventBindings.Add_GetRef(MoveTemp(AB));
    }
	
	/**
	 * Binds a UInputAction assigned via UInputMappingContext to this component.
	 * No delegate will be called when this action triggers. The binding simply reflects the current value of the action.
	 */
	FEnhancedInputActionValueBinding& BindActionValue(const UInputAction* Action)
	{
		// Only one action value binding is required per action
		if (FEnhancedInputActionValueBinding* Existing = EnhancedActionValueBindings.FindByPredicate([Action](const FEnhancedInputActionValueBinding& TestBinding) { return TestBinding.GetAction() == Action; }))
		{
			return *Existing;
		}

		return EnhancedActionValueBindings.Add_GetRef(FEnhancedInputActionValueBinding(Action));
	}

	/**
	 * Binds a chord event to a delegate function in development builds only.
	 */
	template<class UserClass>
	FInputDebugKeyBinding& BindDebugKey(const FInputChord Chord, const EInputEvent KeyEvent, UserClass* Object, typename FInputDebugKeyHandlerSignature::TMethodPtr< UserClass > Func, bool bExecuteWhenPaused = true)
	{
#ifdef DEV_ONLY_KEY_BINDINGS_AVAILABLE
		TUniquePtr<FInputDebugKeyDelegateBinding<FInputDebugKeyHandlerSignature>> KB = MakeUnique<FInputDebugKeyDelegateBinding<FInputDebugKeyHandlerSignature>>(Chord, KeyEvent, bExecuteWhenPaused);
		KB->Delegate.BindDelegate(Object, Func);
		KB->Delegate.SetShouldFireWithEditorScriptGuard(bShouldFireDelegatesInEditor);
		return *DebugKeyBindings.Emplace_GetRef(MoveTemp(KB));
#endif
	}

	/**
	 * Binds to an object UFUNCTION
	 */
	FInputDebugKeyBinding& BindDebugKey(const FInputChord Chord, const EInputEvent KeyEvent, UObject* Object, FName FunctionName, bool bExecuteWhenPaused = true)
	{
#ifdef DEV_ONLY_KEY_BINDINGS_AVAILABLE
		TUniquePtr<FInputDebugKeyDelegateBinding<FInputDebugKeyHandlerDynamicSignature>> KB = MakeUnique<FInputDebugKeyDelegateBinding<FInputDebugKeyHandlerDynamicSignature>>(Chord, KeyEvent, bExecuteWhenPaused);
		KB->Delegate.BindDelegate(Object, FunctionName);
		KB->Delegate.SetShouldFireWithEditorScriptGuard(bShouldFireDelegatesInEditor);
		return *DebugKeyBindings.Emplace_GetRef(MoveTemp(KB));
#endif
	}


	/**
	 * Helper function to pull the action value for a bound action value.
	 */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", HideSelfPin = "true", HidePin = "Action"))
	UE_API FInputActionValue GetBoundActionValue(const UInputAction* Action) const;


	// While we do not encourage the binding of legacy action and axis input bindings on a UEnhancedInputComponent,
	// there is no functional issues with doing so. Set this define in your game's .build.cs file if you would like
	// allow for legacy binding to Enhanced Input Components. This maybe useful if you are upgrading an older project to
	// use Enhanced Input.
	//
	// You can set this in the .build.cs file like this:
	//		PublicDefinitions.Add("ENHANCED_INPUT_ALLOW_LEGACY_BINDING=1");
#ifndef ENHANCED_INPUT_ALLOW_LEGACY_BINDING
	#define ENHANCED_INPUT_ALLOW_LEGACY_BINDING 0
#endif

#if !ENHANCED_INPUT_ALLOW_LEGACY_BINDING
	// Delete all InputComponent binding helpers. Indicates intentions going forward and improves intellisense/VAX when working with EnhancedInputCompoennts.
	template<class UserClass>
	FInputActionBinding& BindAction(const FName ActionName, const EInputEvent KeyEvent, UserClass* Object, typename FInputActionHandlerSignature::TMethodPtr< UserClass > Func) = delete;
	template<class UserClass>
	FInputActionBinding& BindAction(const FName ActionName, const EInputEvent KeyEvent, UserClass* Object, typename FInputActionHandlerWithKeySignature::TMethodPtr< UserClass > Func) = delete;
	template< class DelegateType, class UserClass, typename... VarTypes >
	FInputActionBinding& BindAction(const FName ActionName, const EInputEvent KeyEvent, UserClass* Object, typename DelegateType::template TMethodPtr< UserClass > Func, VarTypes... Vars) = delete;
	template<class UserClass>
	FInputAxisBinding& BindAxis(const FName AxisName, UserClass* Object, typename FInputAxisHandlerSignature::TMethodPtr< UserClass > Func) = delete;
	FInputAxisBinding& BindAxis(const FName AxisName) = delete;
	template<class UserClass>
	FInputAxisKeyBinding& BindAxisKey(const FKey AxisKey, UserClass* Object, typename FInputAxisHandlerSignature::TMethodPtr< UserClass > Func) = delete;
	FInputAxisKeyBinding& BindAxisKey(const FKey AxisKey) = delete;
	template<class UserClass>
	FInputVectorAxisBinding& BindVectorAxis(const FKey AxisKey, UserClass* Object, typename FInputVectorAxisHandlerSignature::TMethodPtr< UserClass > Func) = delete;
	FInputVectorAxisBinding& BindVectorAxis(const FKey AxisKey) = delete;
	template<class UserClass>
	FInputKeyBinding& BindKey(const FInputChord Chord, const EInputEvent KeyEvent, UserClass* Object, typename FInputActionHandlerSignature::TMethodPtr< UserClass > Func) = delete;
	template<class UserClass>
	FInputKeyBinding& BindKey(const FKey Key, const EInputEvent KeyEvent, UserClass* Object, typename FInputActionHandlerSignature::TMethodPtr< UserClass > Func) = delete;
	template<class UserClass>
	FInputTouchBinding& BindTouch(const EInputEvent KeyEvent, UserClass* Object, typename FInputTouchHandlerSignature::TMethodPtr< UserClass > Func) = delete;
	template<class UserClass>
	FInputGestureBinding& BindGesture(const FKey GestureKey, UserClass* Object, typename FInputGestureHandlerSignature::TMethodPtr< UserClass > Func) = delete;
#endif	// !ENHANCED_INPUT_ALLOW_LEGACY_BINDING
};

#undef UE_API
