// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettingsBackedByCVars.h"
#include "Engine/PlatformSettings.h"
#include "GameplayTagContainer.h"

#include "EnhancedInputDeveloperSettings.generated.h"

#define UE_API ENHANCEDINPUT_API

class UInputMappingContext;
class UEnhancedInputUserSettings;
class UEnhancedPlayerMappableKeyProfile;
class UEnhancedPlayerInput;

/** Represents a single input mapping context and the priority that it should be applied with */
USTRUCT()
struct FDefaultContextSetting
{
	GENERATED_BODY()

	/** Input Mapping Context that should be Added to the EnhancedInputEditorSubsystem when it starts listening for input */
	UPROPERTY(EditAnywhere, Config, Category = "Input")
	TSoftObjectPtr<const UInputMappingContext> InputMappingContext = nullptr;

	/** The prioirty that should be given to this mapping context when it is added */
	UPROPERTY(EditAnywhere, Config, Category = "Input")
	int32 Priority = 0;

	/** If true, then this IMC will be applied immediately when the EI subsystem is ready */
	UPROPERTY(EditAnywhere, Config, Category = "Input")
	bool bAddImmediately = true;
	
	/** If true, then this IMC will be registered with the User Input Settings (if one is available) immediately when the Enhanced Input subsystem starts. */
	UPROPERTY(EditAnywhere, Config, Category = "Input")
	bool bRegisterWithUserSettings = false;
};

/** Developer settings for Enhanced Input */
UCLASS(MinimalAPI, config = Input, defaultconfig, meta = (DisplayName = "Enhanced Input"))
class UEnhancedInputDeveloperSettings : public UDeveloperSettingsBackedByCVars
{
	GENERATED_BODY()
public:
	
	UE_API UEnhancedInputDeveloperSettings(const FObjectInitializer& Initializer);

	/**
	 * Array of any input mapping contexts that you want to be applied by default to the Enhanced Input local player subsystem.
	 * NOTE: These mapping context's can only be from your game's root content directory, not plugins.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Enhanced Input",  meta = (editCondition = "bEnableDefaultMappingContexts"))
	TArray<FDefaultContextSetting> DefaultMappingContexts;
	
	/**
	 * Array of any input mapping contexts that you want to be applied by default to the Enhanced Input world subsystem.
	 * NOTE: These mapping context's can only be from your game's root content directory, not plugins.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Enhanced Input|World Subsystem", meta = (editCondition = "bEnableDefaultMappingContexts && bEnableWorldSubsystem"))
	TArray<FDefaultContextSetting> DefaultWorldSubsystemMappingContexts;

	/**
	 * Platform specific settings for Enhanced Input.
	 * @see UEnhancedInputPlatformSettings
	 */
	UPROPERTY(EditAnywhere, Category = "Enhanced Input")
	FPerPlatformSettings PlatformSettings;

	/**
	 * The class that should be used for the User Settings by each Enhanced Input subsystem.
	 * An instance of this class will be spawned by each Enhanced Input subsytem as a place to store
	 * user settings such as keymappings, accessibility settings, etc. Subclass this to add more custom
	 * options to your game.
	 */
	UPROPERTY(config, EditAnywhere, NoClear, Category = "Enhanced Input|User Settings", meta=(editCondition = "bEnableUserSettings"))
	TSoftClassPtr<UEnhancedInputUserSettings> UserSettingsClass;

	/**
	 * The default class for the player mappable key profile, used to store the key mappings set by the player in the user settings.
	 */
	UPROPERTY(config, EditAnywhere, NoClear, Category = "Enhanced Input|User Settings", meta=(editCondition = "bEnableUserSettings"))
	TSoftClassPtr<UEnhancedPlayerMappableKeyProfile> DefaultPlayerMappableKeyProfileClass;

	/**
	 * The default SaveGame slot name to use when loading/saving Enhanced Input user settings.
	 * 
	 * The file name will be this string with ".sav" appended to the end of it. So, on disk, the file would be
	 * "<InputSettingsSaveSlotName>.sav".
	 * 
	 * This file will be written to the "SaveGames" directory, which is "<Project Folder>/Saved/SaveGames" in the editor.
	 */
	UPROPERTY(config, EditAnywhere, NoClear, Category = "Enhanced Input|User Settings", meta=(editCondition = "bEnableUserSettings"))
	FString InputSettingsSaveSlotName = TEXT("EnhancedInputUserSettings");
	
	/** The default player input class that the Enhanced Input world subsystem will use. */
	UPROPERTY(config, EditAnywhere, NoClear, Category = "Enhanced Input|World Subsystem", meta=(editCondition = "bEnableWorldSubsystem"))
	TSoftClassPtr<UEnhancedPlayerInput> DefaultWorldInputClass;
	
	/**
	* If true, then any in progress Enhanced Input Actions will fire Cancelled and Triggered events 
	* when input is flushed (i.e. the viewport has lost focus, or UEnhancedPlayerInput::FlushPressedKeys has been called)
	* 
	* If false, then enhanced input actions may not fire their delegates when input is flushed and their key state would be retained.
	*/
	UPROPERTY(config, EditAnywhere, Category = "Enhanced Input")
	uint8 bSendTriggeredEventsWhenInputIsFlushed : 1;

	/**
	 * If true, then an instance of the User Settings Class will be created on each Enhanced Input subsystem.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Enhanced Input|User Settings", meta=(DisplayName="Enable User Settings", DisplayPriority = 1))
	uint8 bEnableUserSettings : 1;

	/** If true, then the DefaultMappingContexts will be applied to all Enhanced Input Subsystems. */
	UPROPERTY(config, EditAnywhere, Category = "Enhanced Input", meta = (ConsoleVariable = "EnhancedInput.EnableDefaultMappingContexts"))
	uint8 bEnableDefaultMappingContexts : 1;

	/**
	 * If true, then only the last action in a ChordedAction trigger will be fired.
	 * This means that only the action that has the ChordedTrigger on it will be fired, not the individual steps.
	 * 
	 * Default value is true.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Enhanced Input", meta=(ConsoleVariable="EnhancedInput.OnlyTriggerLastActionInChord"))
	uint8 bShouldOnlyTriggerLastActionInChord : 1;

	/**
	 * If true, then Enhanced Input will filter which Input Mapping Context mappings are processed
	 * based on the current value of the "Current Input Mode". If a mapping context's query
	 * does not match with the current input mode tag container, then its mappings will not be processed.
	 *
	 * Default Value is true.
	 * 
	 * @see IEnhancedInputSubsystemInterface::RebuildControlMappings
	 * @see UEnhancedPlayerInput::CurrentInputMode
	 */
	UPROPERTY(config, EditAnywhere, Category = "Enhanced Input|Input Modes")
	uint8 bEnableInputModeFiltering : 1;
	
	/** If true, then the world subsystem will be created. */
	UPROPERTY(config, EditAnywhere, Category = "Enhanced Input|World Subsystem", meta=(DisplayName="Enable World Subsystem (Experimental)", DisplayPriority = 1))
	uint8 bEnableWorldSubsystem : 1;
	
	/**
 	 * If true then the Enhanced Input world subsystem will log all input that is being processed by it (keypresses, analog values, etc)
 	 * Note: This can produce A LOT of logs, so only use this if you are debugging something. Does nothing in shipping builds
 	 */
	UE_DEPRECATED(5.6, "bShouldLogAllWorldSubsystemInputs is deprecated, use the log category on VeryVerbose for LogWorldSubsystemInput instead.")
	uint8 bShouldLogAllWorldSubsystemInputs : 1 = false;

	/**
	 * This value will be used to populate the default InputModeQuery on Input Mapping Contexts.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Enhanced Input|Input Modes", meta = (EditCondition="bEnableInputModeFiltering"))
	FGameplayTagQuery DefaultMappingContextInputModeQuery;

	/**
	 * The default input mode which will be set when a UEnhancedPlayerInput object is created.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Enhanced Input|Input Modes", meta = (EditCondition="bEnableInputModeFiltering"))
	FGameplayTagContainer DefaultInputMode;
};

#undef UE_API
