// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "EnhancedInputLibrary.generated.h"

#define UE_API ENHANCEDINPUT_API

class APlayerController;
enum class EInputActionValueType : uint8;
struct FEnhancedActionKeyMapping;
struct FInputActionValue;
class IEnhancedInputSubsystemInterface;
class UInputAction;
class UInputMappingContext;
class UPlayerMappableKeySettings;

UCLASS(MinimalAPI)
class UEnhancedInputLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Call SubsystemPredicate on each registered player and standalone enhanced input subsystem.
	 */
	static UE_API void ForEachSubsystem(TFunctionRef<void(IEnhancedInputSubsystemInterface*)> SubsystemPredicate);

	/**
	 * Flag all enhanced input subsystems making use of the mapping context for reapplication of all control mappings at the end of this frame.
	 * @param Context				Mappings will be rebuilt for all subsystems utilizing this context.
	 * @param bForceImmediately		The mapping changes will be applied synchronously, rather than at the end of the frame, making them available to the input system on the same frame.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input")
	static UE_API void RequestRebuildControlMappingsUsingContext(const UInputMappingContext* Context, bool bForceImmediately = false);


	/** Breaks an ActionValue into X, Y, Z. Axes not supported by value type will be 0. */
	UFUNCTION(BlueprintPure, Category = "Input", meta = (NativeBreakFunc))
	static UE_API void BreakInputActionValue(FInputActionValue InActionValue, double& X, double& Y, double& Z, EInputActionValueType& Type);

	/**
	 * Builds an ActionValue from X, Y, Z. Inherits type from an existing ActionValue. Ignores axis values unused by the provided value type.
	 * @note Intended for use in Input Modifier Modify Raw overloads to modify an existing Input Action Value.
	 */
	UFUNCTION(BlueprintPure, Category = "Input", meta = (Keywords = "construct build", NativeMakeFunc))
	static UE_API FInputActionValue MakeInputActionValueOfType(double X, double Y, double Z, const EInputActionValueType ValueType);
	
	/**
	* Returns the Player Mappable Key Settings owned by the Action Key Mapping or by the referenced Input Action, or nothing based of the Setting Behavior.
	*/
	UFUNCTION(BlueprintPure, Category = "Input", meta = (ReturnDisplayName = "Player Mappable Key Settings"))
	static UE_API UPlayerMappableKeySettings* GetPlayerMappableKeySettings(UPARAM(ref) const FEnhancedActionKeyMapping& ActionKeyMapping);

	/**
	* Returns the name of the mapping based on setting behavior used. If no name is found in the Mappable Key Settings it will return the name set in Player Mappable Options if bIsPlayerMappable is true.
	*/
	UFUNCTION(BlueprintPure, Category = "Input", meta = (ReturnDisplayName = "Mapping Name"))
	static UE_API FName GetMappingName(UPARAM(ref) const FEnhancedActionKeyMapping& ActionKeyMapping);

	/**
	 * Returns true if this Action Key Mapping either holds a Player Mappable Key Settings or is set bIsPlayerMappable.
	 */
	UFUNCTION(BlueprintPure, Category = "Input", meta = (ReturnDisplayName = "Is Player Mappable"))
	static UE_API bool IsActionKeyMappingPlayerMappable(UPARAM(ref) const FEnhancedActionKeyMapping& ActionKeyMapping);

	// Internal helper functionality

	// GetInputActionvalue internal accessor function for actions that have been bound to from a UEnhancedInputComponent
	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly, meta = (HidePin = "Action"))
	static UE_API FInputActionValue GetBoundActionValue(AActor* Actor, const UInputAction* Action);

	// FInputActionValue internal auto-converters.

	/** Interpret an InputActionValue as a boolean input */
	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly, meta = (BlueprintAutocast))
	static UE_API bool Conv_InputActionValueToBool(FInputActionValue InValue);

	/** Interpret an InputActionValue as a 1D axis (double) input */
	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly, meta = (BlueprintAutocast))
	static UE_API double Conv_InputActionValueToAxis1D(FInputActionValue InValue);

	/** Interpret an InputActionValue as a 2D axis (Vector2D) input */
	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly, meta = (BlueprintAutocast))
	static UE_API FVector2D Conv_InputActionValueToAxis2D(FInputActionValue InValue);

	/** Interpret an InputActionValue as a 3D axis (Vector) input */
	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly, meta = (BlueprintAutocast))
	static UE_API FVector Conv_InputActionValueToAxis3D(FInputActionValue ActionValue);

	/** Converts a FInputActionValue to a string */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (InputActionValue)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|String")
	static UE_API FString Conv_InputActionValueToString(FInputActionValue ActionValue);

	/** Converts an ETriggerEvent to a string */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (ETriggerEvent)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|String")
	static UE_API FString Conv_TriggerEventValueToString(const ETriggerEvent TriggerEvent);

	/**
	 * Flushes the player controller's pressed keys
	 * 
	 * @see APlayerController::FlushPressedKeys
	 */
	UFUNCTION(BlueprintCallable, Category="Input")
	static UE_API void FlushPlayerInput(APlayerController* PlayerController);
};

#undef UE_API
