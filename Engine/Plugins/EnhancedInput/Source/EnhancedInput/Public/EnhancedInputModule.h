// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Stats/Stats.h"

#define UE_API ENHANCEDINPUT_API

class UEnhancedInputLibrary;
struct FKey;

class APlayerController;

ENHANCEDINPUT_API DECLARE_LOG_CATEGORY_EXTERN(LogEnhancedInput, Log, All);

struct FEnhancedInputKeys
{
	// Combo FKey that serves as the key combo triggers are automatically mapped to - is not action bindable
	UE_DEPRECATED(5.8, "The combo trigger and key have been deprecated")
	static UE_API const FKey ComboKey;
};

/**
 * The public interface to this module
 */
class IEnhancedInputModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IEnhancedInputModule& Get()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_IEnhancedInputModule_Get);
		static IEnhancedInputModule& Singleton = FModuleManager::LoadModuleChecked< IEnhancedInputModule >("EnhancedInput");
		return Singleton;
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_IEnhancedInputModule_IsAvailable);
		return FModuleManager::Get().IsModuleLoaded("EnhancedInput");
	}

	virtual UEnhancedInputLibrary* GetLibrary() = 0;

};

#undef UE_API
