// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/PlatformSettings.h"
#include "Engine/PlatformSettingsManager.h"
#include "EnhancedInputPlatformSettings.generated.h"

#define UE_API ENHANCEDINPUT_API

class UInputMappingContext;
struct FStreamableHandle;

/**
 * A base class that can be used to store platform specific data for Enhanced Input.
 * 
 * Make a subclass of this to add some additional options for per-platform settings 
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, ClassGroup = Input, meta = (Category = "EnhancedInput"))
class UEnhancedInputPlatformData : public UObject
{
	GENERATED_BODY()

public:

	//~ UObject interface
#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR
	//~ End UObject interface
	
	/** Returns a pointer to the desired redirect mapping context. If there isn't one, then this returns InContext. */
	UFUNCTION(BlueprintCallable, Category = "EnhancedInput")
	UE_API const UInputMappingContext* GetContextRedirect(UInputMappingContext* InContext) const;

	/** Return the map of all Context Redirects in this platform data. */
	const TMap<TObjectPtr<const UInputMappingContext>, TObjectPtr<const UInputMappingContext>>& GetMappingContextRedirects() const { return MappingContextRedirects; }
	
protected:
	
	/**
	 * Maps one Input Mapping Context to another. This can be used to replace
	 * specific Input Mapping Contexts with another on a per-platform basis. 
	 */
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category = "EnhancedInput")
	TMap<TObjectPtr<const UInputMappingContext>, TObjectPtr<const UInputMappingContext>> MappingContextRedirects;
};

/** Settings for Enhanced Input that can be set per-platform */
UCLASS(MinimalAPI, config = Input, defaultconfig)
class UEnhancedInputPlatformSettings : public UPlatformSettings
{
	GENERATED_BODY()

public:

	static UEnhancedInputPlatformSettings* Get()
	{
		return UPlatformSettingsManager::Get().GetSettingsForPlatform<UEnhancedInputPlatformSettings>();
	}

	/** Get the current Enhanced Input platform data */
	const TArray<TSoftClassPtr<UEnhancedInputPlatformData>>& GetInputData() const { return InputData; }

	/** Populates the given map with all mapping context redirects in the current platform's InputData. */
	UE_API void GetAllMappingContextRedirects(OUT TMap<TObjectPtr<const UInputMappingContext>, TObjectPtr<const UInputMappingContext>>& OutRedirects);

	/** Iterates each valid UEnhancedInputPlatformData that is on this platform. */
	UE_API void ForEachInputData(TFunctionRef<void(const UEnhancedInputPlatformData&)> Predicate);

	/** Returns true if this platform has specified that it should log what mapping context redirects occur */
	bool ShouldLogMappingContextRedirects() const { return bShouldLogMappingContextRedirects; }
	
protected:

	/** Load the input data subclasses and cache them for later use in the InputDataClasses array. */
	UE_DEPRECATED(5.7, "LoadInputDataClasses is now deprecated")
	UE_API void LoadInputDataClasses();
	
	/** Input data that can be populated with Enhanced Input Platform Data blueprints */
	UPROPERTY(config, EditAnywhere, Category = "Default")
	TArray<TSoftClassPtr<UEnhancedInputPlatformData>> InputData;

	/**
	 * A transient array of the subclasses for the Enhanced Input Platform data. This will prevent us from
	 * from having to load the class' default object during game time.
	 */
	UE_DEPRECATED(5.7, "InputDataClasses is now deprecated")
	UPROPERTY(Transient)
	TArray<TSubclassOf<UEnhancedInputPlatformData>> InputDataClasses;

	/** If true, then Enhanced Input will log which mapping contexts have been redirected when building the control mappings. */
	UPROPERTY(Config, EditAnywhere, Category = "Default")
	bool bShouldLogMappingContextRedirects = false;
};

#undef UE_API
