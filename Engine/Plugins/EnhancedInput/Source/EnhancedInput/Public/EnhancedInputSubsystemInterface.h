// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "EnhancedPlayerInput.h"

#include "EnhancedInputSubsystemInterface.generated.h"

#define UE_API ENHANCEDINPUT_API

enum class EMappingQueryIssue : uint8;
enum class EMappingQueryResult : uint8;
struct FMappingQueryIssue;

class APlayerController;
class UCanvas;
class UInputMappingContext;
class UInputAction;
class UEnhancedPlayerInput;
class UInputModifier;
class UInputTrigger;
class UWorldSubsystem;
class UEnhancedInputUserSettings;
class UEnhancedPlayerMappableKeyProfile;

// Subsystem interface
UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint, CannotGenerateMessageNodes=true))
class UEnhancedInputSubsystemInterface : public UInterface
{
	GENERATED_BODY()
};

UENUM()
enum class EInputMappingRebuildType : uint8
{
	// No rebuild required.
	None,
	// Standard mapping rebuild. Retains existing triggers and modifiers for actions that were previously mapped.
	Rebuild,
	// If you have made changes to the triggers/modifiers associated with a UInputAction that was previously mapped a flush is required to reset the tracked data for that action.
	RebuildWithFlush,
};

/** Passed in as params for Adding/Remove input contexts */
USTRUCT(BlueprintType)
struct FModifyContextOptions
{
	GENERATED_BODY()
	
	FModifyContextOptions()
		: bIgnoreAllPressedKeysUntilRelease(true)
		, bForceImmediately(false)
		, bNotifyUserSettings(false)
	{}

	/**
	 * If true, then any keys that are "down" or "pressed" during the rebuild of control mappings will
	 * not be processed by Enhanced Input until after they are "released". 
	 * 
	 * For example, if you are adding a mapping context with a key mapping to "X",
	 * and the player is holding down "X" while that IMC is added, 
	 * there will not be a "Triggered" event until the player releases "X" and presses it again.
	 * 
	 * If this is set to false for the above example, then the "Triggered" would fire immediately 
	 * as soon as the IMC is finished being added.
	 *
	 * Default: True
	 * 
	 * Note: This will only do something for keys bound to boolean Input Action types.
	 * Note: This includes all keys that the player has pressed, not just the keys that are previously mapped in Enhanced Input before
	 * the call to RebuildControlMappings.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input")
	uint8 bIgnoreAllPressedKeysUntilRelease : 1;

	/**
	 * The mapping changes will be applied synchronously, rather than at the end of the frame,
	 * making them available to the input system on the same frame.
	 * 
	 * This is not recommended to be set to true if you are adding multiple mapping contexts 
	 * as it will have poor performance.
	 *
	 * Default: False
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input")
	uint8 bForceImmediately : 1;
	
	/**
	 * If true, then this Mapping Context will be registered or unregistered with the
	 * Enhanced Input User Settings on this subsystem, if they exist.
	 *
	 * Default: False
	 *
	 * Note: You need to enable and configure your UEnhancedInputUserSettings class in the project
	 * settings for this to do anything.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input")
	uint8 bNotifyUserSettings : 1;
};

// Includes native functionality shared between all subsystems
class IEnhancedInputSubsystemInterface
{
	friend class FEnhancedInputModule;

	GENERATED_BODY()

public:

	virtual UEnhancedPlayerInput* GetPlayerInput() const = 0;

	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta=(DisplayName="Get Enhanced Input User Settings"))
	UE_API virtual UEnhancedInputUserSettings* GetUserSettings() const;

	/**
	 * Getter for the CVar EnhancedInput.TrackKeysForAllActionMappings which enables Enhanced Input to respect Key consumption.
	 */
	static bool TrackingKeysForAllActionMappings();

protected:
	
	/**
	 * Create a new user settings object if it is enabled in the EI developer settings.
	 *
	 * Not every enhanced input subsystem needs user settings, so this is an optional feature.
	 */
	UE_API virtual void InitalizeUserSettings();
	
	/** Binds to any delegates of interest on the UEnhancedInputUserSettings if they are enabled in the developer settings. */
	UE_API virtual void BindUserSettingDelegates();
	
	/** Callback for when any Enhanced Input user settings have been changed (a new key mapping for example) */
	UFUNCTION()
	UE_API virtual void OnUserSettingsChanged(UEnhancedInputUserSettings* Settings);

	/** A callback for when the user has applied a new mappable key profile. */
	UFUNCTION()
	UE_API virtual void OnUserKeyProfileChanged(const UEnhancedPlayerMappableKeyProfile* InNewProfile);

public:
	
	/**
	 * Input simulation via injection. Runs modifiers and triggers delegates as if the input had come through the underlying input system as FKeys.
	 * Applies action modifiers and triggers on top.
	 *
	 * @param Action		The Input Action to set inject input for
	 * @param RawValue		The value to set the action to
	 * @param Modifiers		The modifiers to apply to the injected input.
	 * @param Triggers		The triggers to apply to the injected input.
	 */
	UFUNCTION(BlueprintCallable, Category="Input", meta=(AutoCreateRefTerm="Modifiers,Triggers"))
	UE_API virtual void InjectInputForAction(const UInputAction* Action, FInputActionValue RawValue, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers);

	/**
	 * Input simulation via injection. Runs modifiers and triggers delegates as if the input had come through the underlying input system as FKeys.
	 * Applies action modifiers and triggers on top.
	 *
	 * @param Action		The Input Action to set inject input for
	 * @param Value			The value to set the action to (the type will be controlled by the Action)
	 * @param Modifiers		The modifiers to apply to the injected input.
	 * @param Triggers		The triggers to apply to the injected input.
	 */
	UFUNCTION(BlueprintCallable, Category="Input", meta=(AutoCreateRefTerm="Modifiers,Triggers"))
	UE_API virtual void InjectInputVectorForAction(const UInputAction* Action, FVector Value, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers);

	/**
	 * Input simulation via injection. Runs modifiers and triggers delegates as if the input had come through the underlying input system as FKeys.
	 * Applies action modifiers and triggers on top.
	 *
	 * @param MappingName		The name of the player mapping that can be used for look up an associated UInputAction object.
	 * @param RawValue			The value to set the action to
	 * @param Modifiers			The modifiers to apply to the injected input.
	 * @param Triggers			The triggers to apply to the injected input.
	 */
	UFUNCTION(BlueprintCallable, Category="Input", meta=(AutoCreateRefTerm="Modifiers,Triggers"))
	UE_API virtual void InjectInputForPlayerMapping(UPARAM(Meta=(GetOptions="EnhancedInput.PlayerMappableKeySettings.GetKnownMappingNames")) const FName MappingName, FInputActionValue RawValue, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers);
	
	/**
	 * Input simulation via injection. Runs modifiers and triggers delegates as if the input had come through the underlying input system as FKeys.
	 * Applies action modifiers and triggers on top.
	 *
	 * @param MappingName		The name of the player mapping that can be used for look up an associated UInputAction object.
	 * @param Value				The value to set the action to (the type will be controlled by the Action)
	 * @param Modifiers			The modifiers to apply to the injected input.
	 * @param Triggers			The triggers to apply to the injected input.
	 */
	UFUNCTION(BlueprintCallable, Category="Input", meta=(AutoCreateRefTerm="Modifiers,Triggers"))
	UE_API virtual void InjectInputVectorForPlayerMapping(UPARAM(Meta=(GetOptions="EnhancedInput.PlayerMappableKeySettings.GetKnownMappingNames")) const FName MappingName, FVector Value, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers);

	/**
	 * Starts simulation of input via injection. This injects the given input every tick until it is stopped with StopContinuousInputInjectionForAction.
	 *
	 * @param Action		The Input Action to set inject input for
	 * @param RawValue		The value to set the action to (the type will be controlled by the Action)
	 * @param Modifiers		The modifiers to apply to the injected input.
	 * @param Triggers		The triggers to apply to the injected input.
	 */
	UFUNCTION(BlueprintCallable, Category="Input", meta=(AutoCreateRefTerm="Modifiers,Triggers"))
	UE_API virtual void StartContinuousInputInjectionForAction(const UInputAction* Action, FInputActionValue RawValue, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers);

	/**
	 * Starts simulation of input via injection. This injects the given input every tick until it is stopped with StopContinuousInputInjectionForAction.
	 *
	 * @param MappingName		The name of the player mapping that can be used for look up an associated UInputAction object.
	 * @param RawValue			The value to set the action to (the type will be controlled by the Action)
	 * @param Modifiers			The modifiers to apply to the injected input.
	 * @param Triggers			The triggers to apply to the injected input.
	 */
	UFUNCTION(BlueprintCallable, Category="Input", meta=(AutoCreateRefTerm="Modifiers,Triggers"))
	UE_API virtual void StartContinuousInputInjectionForPlayerMapping(UPARAM(Meta=(GetOptions="EnhancedInput.PlayerMappableKeySettings.GetKnownMappingNames")) const FName MappingName, FInputActionValue RawValue, const TArray<UInputModifier*>& Modifiers, const TArray<UInputTrigger*>& Triggers);

	/**
	 * Update the value of a continuous input injection, preserving the state of triggers and modifiers.
	 *
	 * @param Action	The Input Action to set inject input for
	 * @param RawValue	The value to set the action to (the type will be controlled by the Action)
	 */
	UFUNCTION(BlueprintCallable, Category="Input")
	UE_API virtual void UpdateValueOfContinuousInputInjectionForAction(const UInputAction* Action, FInputActionValue RawValue);

	/**
	 * Update the value of a continuous input injection for the given player mapping name, preserving the state of triggers and modifiers.
	 *
	 * @param MappingName	The name of the player mapping that can be used for look up an associated UInputAction object.
	 * @param RawValue		The value to set the action to (the type will be controlled by the Action)
	 */
	UFUNCTION(BlueprintCallable, Category="Input")
	UE_API virtual void UpdateValueOfContinuousInputInjectionForPlayerMapping(UPARAM(Meta=(GetOptions="EnhancedInput.PlayerMappableKeySettings.GetKnownMappingNames")) const FName MappingName, FInputActionValue RawValue);

	/**
	 * Stops continuous input injection for the given action.
	 *
	 * @param Action		The action to stop injecting input for
	 */
	UFUNCTION(BlueprintCallable, Category="Input")
	UE_API virtual void StopContinuousInputInjectionForAction(const UInputAction* Action);

	/**
	 * Stops continuous input injection for the given player mapping name.
	 *
	 * @param MappingName		The name of the player mapping that can be used for look up an associated UInputAction object.
	 */
	UFUNCTION(BlueprintCallable, Category="Input")
	UE_API virtual void StopContinuousInputInjectionForPlayerMapping(UPARAM(Meta=(GetOptions="EnhancedInput.PlayerMappableKeySettings.GetKnownMappingNames")) const FName MappingName);
	
	/**
	 * Determines if there is a continuous input injection for the given action.
	 */
	UE_API bool HasContinuousInputInjectionForAction(const UInputAction* Action) const;
	
	/**
	 * Remove all applied mapping contexts.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input")
	UE_API virtual void ClearAllMappings();
	
	/**
	 * Add a control mapping context.
	 * @param MappingContext		A set of key to action mappings to apply to this player
	 * @param Priority				Higher priority mappings will be applied first and, if they consume input, will block lower priority mappings.
	 * @param Options				Options to consider when adding this mapping context.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input", meta=(AutoCreateRefTerm = "Options"))
	UE_API virtual void AddMappingContext(const UInputMappingContext* MappingContext, int32 Priority, const FModifyContextOptions& Options = FModifyContextOptions());

	/**
	* Remove a specific control context. 
	* This is safe to call even if the context is not applied.
	* @param MappingContext		Context to remove from the player
	* @param Options			Options to consider when removing this input mapping context
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input", meta=(AutoCreateRefTerm = "Options"))
	UE_API virtual void RemoveMappingContext(const UInputMappingContext* MappingContext, const FModifyContextOptions& Options = FModifyContextOptions());

	/**
	 * Returns the current input mode set on Enhanced Input. This mode will control what mappings are 
	 * currently being processed by the player. If the Input Mapping Context's InputModeQuery is
	 * not empty and does not match this tag container, then it's mappings will not be processed.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input")
	UE_API virtual FGameplayTagContainer GetInputMode() const;

	/**
	 * Sets the current input mode on the player and triggers a rebuild of control mappings.
	 * 
	 * @param NewMode The new input mode tag container to use
	 * @param Options Options for how to treat the control mappings rebuild.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input", meta=(AutoCreateRefTerm = "Options", AdvancedDisplay = "Options"))
	UE_API virtual void SetInputMode(const FGameplayTagContainer& NewMode, const FModifyContextOptions& Options = FModifyContextOptions());

	/**
	 * Appends the given "TagsToAdd" to the current input mode tag container and triggers a rebuild of control mappings.
	 *  
	 * @param TagsToAdd	The gameplay tag container to append 
	 * @param Options	Options for how to treat the control mappings rebuild
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input", meta=(AutoCreateRefTerm = "Options", AdvancedDisplay = "Options"))
	UE_API virtual void AppendTagsToInputMode(const FGameplayTagContainer& TagsToAdd, const FModifyContextOptions& Options = FModifyContextOptions());

	/**
	 * Adds the given "TagToAdd" to the current input mode tag container and triggers a rebuild of control mappings.
	 * 
	 * @param TagToAdd	The gameplay tag to add to the current input mode tag container
	 * @param Options	Options for how to treat the control mappings rebuild
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input", meta=(AutoCreateRefTerm = "Options", AdvancedDisplay = "Options"))
	UE_API virtual void AddTagToInputMode(const FGameplayTag& TagToAdd, const FModifyContextOptions& Options = FModifyContextOptions());
	
	/**
	 * Removes tags from the current input mode and triggers a rebuild of control mappings.
	 * 
	 * @param TagsToRemove Tags to remove from the current input mode tag container
	 * @param Options		Options for how to treat the control mappings rebuild
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input", meta=(AutoCreateRefTerm = "Options", AdvancedDisplay = "Options"))
	UE_API virtual void RemoveTagsFromInputMode(const FGameplayTagContainer& TagsToRemove, const FModifyContextOptions& Options = FModifyContextOptions());

	/**
	 * Removes the given tag from the current input mode and triggers a rebuild of control mappings.
	 * 
	 * @param TagToRemove	The tag to remove from the current input mode 
	 * @param Options		Options for how to treat the control mappings rebuild 
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input", meta=(AutoCreateRefTerm = "Options", AdvancedDisplay = "Options"))
	UE_API virtual void RemoveTagFromInputMode(const FGameplayTag& TagToRemove, const FModifyContextOptions& Options = FModifyContextOptions());
	
	/**
	* Flag player for reapplication of all mapping contexts at the end of this frame.
	* This is called automatically when adding or removing mappings contexts.
	*
	* @param Options		Options to consider when removing this input mapping context
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input", meta=(AutoCreateRefTerm = "Options"))
	UE_API virtual void RequestRebuildControlMappings(const FModifyContextOptions& Options = FModifyContextOptions(), EInputMappingRebuildType RebuildType = EInputMappingRebuildType::Rebuild);

	/**
	 * Check if a key mapping is safe to add to a given mapping context within the set of active contexts currently applied to the player controller.
	 * @param InputContext		Mapping context to which the action/key mapping is intended to be added
	 * @param Action			Action that can be triggered by the key
	 * @param Key				Key that will provide input values towards triggering the action
	 * @param OutIssues			Issues that may cause this mapping to be invalid (at your discretion). Any potential issues will be recorded, even if not present in FatalIssues.
	 * @param BlockingIssues	All issues that should be considered fatal as a bitset.
	 * @return					Summary of resulting issues.
	 * @see QueryMapKeyInContextSet
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input|Mapping Queries")
	UE_API virtual EMappingQueryResult QueryMapKeyInActiveContextSet(const UInputMappingContext* InputContext, const UInputAction* Action, FKey Key, TArray<FMappingQueryIssue>& OutIssues, EMappingQueryIssue BlockingIssues/* = DefaultMappingIssues::StandardFatal*/);

	/**
	 * Check if a key mapping is safe to add to a collection of mapping contexts
	 * @param PrioritizedActiveContexts	Set of mapping contexts to test against ordered by priority such that earlier entries take precedence over later ones.
	 * @param InputContext		Mapping context to which the action/key mapping is intended to be applied. NOTE: This context must be present in PrioritizedActiveContexts.
	 * @param Action			Action that is triggered by the key
	 * @param Key				Key that will provide input values towards triggering the action
	 * @param OutIssues			Issues that may cause this mapping to be invalid (at your discretion). Any potential issues will be recorded, even if not present in FatalIssues.
	 * @param BlockingIssues	All issues that should be considered fatal as a bitset.
	 * @return					Summary of resulting issues.
	 * @see QueryMapKeyInActiveContextSet
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input|Mapping Queries")
	UE_API virtual EMappingQueryResult QueryMapKeyInContextSet(const TArray<UInputMappingContext*>& PrioritizedActiveContexts, const UInputMappingContext* InputContext, const UInputAction* Action, FKey Key, TArray<FMappingQueryIssue>& OutIssues, EMappingQueryIssue BlockingIssues/* = DefaultMappingIssues::StandardFatal*/);

	/**
	 * Check if a mapping context is applied to this subsystem's owner.
	 */
	UE_API virtual bool HasMappingContext(const UInputMappingContext* MappingContext) const;

	/**
	 * Check if a mapping context is applied to this subsystem's owner.
	 *
	 * @param MappingContext		The mapping context to search for on the subsystem's owner.
	 * @param OutFoundPriority		The priority of the mapping context if it is applied. -1 if the context is not applied	
	 * @return	True if the mapping context is applied
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input|Mapping Queries")
	UE_API virtual bool HasMappingContext(const UInputMappingContext* MappingContext, int32& OutFoundPriority) const;

	/**
	 * Returns the keys mapped to the given action in the active input mapping contexts.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input|Mapping Queries")
	UE_API virtual TArray<FKey> QueryKeysMappedToAction(const UInputAction* Action) const;

	/**
	 * Get an array of the currently applied key mappings that are marked as Player Mappable.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Input|Mapping Queries")
	UE_API virtual TArray<FEnhancedActionKeyMapping> GetAllPlayerMappableActionKeyMappings() const;

	/** Logs warnings for any currently applied Input Mapping Contexts that have a tracked RegistrationTrackingMode to detect leaks */
	UE_API void ValidateTrackedMappingContextsAreUnregistered() const;
	
protected:

	/**
	 * Ticks any forced/injected input to the subsystem's player input. If you want support for ApplyForcedInput
	 * in a subsytem implementing this interface, then you must call this function.
	 */
	UE_API void TickForcedInput(float DeltaTime);

	/**
	 * Called each tick from the Enhanced Input module.
	 * 
	 * If bControlMappingsRebuiltThisTick is true this tick, then call the ControlMappingsRebuiltThisFrame
	 * function and reset the bControlMappingsRebuiltThisTick flag to false.
	 * 
	 * This gives an implementor of this interface an oppurtunity to add in a nice blueprint useable delegate.
	 */
	UE_API void HandleControlMappingRebuildDelegate();

	/** Function that will be called when Control Mappings have been rebuilt this tick. */
	virtual void ControlMappingsRebuiltThisFrame() {}

	// helper function to display debug about mapping context info
	UE_API void ShowMappingContextDebugInfo(UCanvas* Canvas, const UEnhancedPlayerInput* PlayerInput);

	/**
	 * Pure-virtual getter for the map of inputs that should be injected every frame. These inputs will be injected when 
	 * ForcedInput is ticked. Any classes that implement this interface should have this function return a managed map to
	 * avoid GC and unreachibility issues.
	 */
	virtual TMap<TObjectPtr<const UInputAction>, FInjectedInput>& GetContinuouslyInjectedInputs() = 0;
	
private:

	// Forced actions/keys for debug. These will be applied each tick once set even if zeroed, until removed. 
	UE_API void ApplyForcedInput(const UInputAction* Action, FInputActionValue Value);
	UE_API void ApplyForcedInput(FKey Key, FInputActionValue Value);
	UE_API void RemoveForcedInput(const UInputAction* Action);
	UE_API void RemoveForcedInput(FKey Key);

	UE_API void InjectChordBlockers(const TArray<int32>& ChordedMappings);
	UE_API bool HasTriggerWith(TFunctionRef<bool(const class UInputTrigger*)> TestFn, const TArray<class UInputTrigger*>& Triggers);	

	/**
	 * Reorder the given UnordedMappings such that chording mappings > chorded mappings > everything else.
	 * This is used to ensure mappings within a single context are evaluated in the correct order to support chording.
	 * Populate the DependentChordActions array with any chorded triggers so that we can detect which ones should be triggered
	 * later. 
	 */
	UE_API TArray<FEnhancedActionKeyMapping> ReorderMappings(const TArray<FEnhancedActionKeyMapping>& UnorderedMappings, TArray<UEnhancedPlayerInput::FDependentChordTracker>& OUT DependentChordActions);
	
	/**
	 * Reapply all control mappings to players pending a rebuild
	 */
	UE_API void RebuildControlMappings();

	/** Convert input settings axis config to modifiers for a given mapping */
	UE_API void ApplyAxisPropertyModifiers(UEnhancedPlayerInput* PlayerInput, struct FEnhancedActionKeyMapping& Mapping) const;

	TMap<TWeakObjectPtr<const UInputAction>, FInputActionValue> ForcedActions;

	/** 
	* Data storing the state of input values we inject for specific FKeys
	*/
	struct FInjectedKeyData
	{
		FInjectedKeyData() = default;
		
		FInjectedKeyData(const FInputActionValue& Value)
			: InputValue(Value)
			, LastInjectedValue(FInputActionValue(FVector::ZeroVector))
		{}
		
		/** 
		* The desired input action value to inject when we tick.
		*/
		FInputActionValue InputValue = FInputActionValue(FVector::ZeroVector);

		/**
		 * The last input value which was injected for this key. This is
		 * set on tick after we have injected the input for this key at least once.
		 */
		FInputActionValue LastInjectedValue = FInputActionValue(FVector::ZeroVector);
	};

	/**
	 * Map of FKey's to any input data that we are injecting to the player right now.
	 */
	TMap<FKey, FInjectedKeyData> ForcedKeys;

	/**
	 * A map of input actions with a Chorded trigger, mapped to the action they are dependent on.
	 * The Key is the Input Action with the Chorded Trigger, and the value is the action it is dependant on.
	 */
	TMap<TObjectPtr<const UInputAction>, TObjectPtr<const UInputAction>> ChordedActionDependencies;

	/**
	 * A map of the currently applied mapping context redirects. This is populated in RebuildControlMappings
	 * with any InputMappingContexts that have been redirected on this local player from the platform settings.
	 */
	TMap<TObjectPtr<const UInputMappingContext>, TObjectPtr<const UInputMappingContext>> AppliedContextRedirects;

	EInputMappingRebuildType MappingRebuildPending = EInputMappingRebuildType::None;

	/**
	 * A flag that will be set when adding/removing a mapping context.
	 *
	 * If this is true, then any keys that are pressed when control mappings are rebuilt will be ignored 
	 * by the new Input context after being until the key is lifted
	 */
	bool bIgnoreAllPressedKeysUntilReleaseOnRebuild = true;

	bool bMappingRebuildPending = false;

	/**
	 * If true then the control mappings have been rebuilt on this frame.
	 * This is reset to false every tick in the EnhancedInputModule
	 */
	bool bControlMappingsRebuiltThisTick = false;

	// Debug visualization for enhanced input local player subsystem
	UE_API virtual void ShowDebugInfo(UCanvas* Canvas);
	
	// Gather any UInputModifiers on the given Player Input and Instance data that need to be visualized for debugging
	static UE_API void GetAllRelevantInputModifiersForDebug(const UEnhancedPlayerInput* PlayerInput, const FInputActionInstance* InstanceData, OUT TArray<UInputModifier*>& OutModifiers);

	UE_API void ShowDebugActionModifiers(UCanvas* Canvas, const UInputAction* Action);
	static UE_API void PurgeDebugVisualizations();
};

#undef UE_API
