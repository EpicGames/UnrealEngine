// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EnhancedInputSubsystemInterface.h"
#include "Subsystems/LocalPlayerSubsystem.h"
#include "Subsystems/WorldSubsystem.h"

#include "EnhancedInputSubsystems.generated.h"

#define UE_API ENHANCEDINPUT_API

class FEnhancedInputWorldProcessor;
enum class ETickableTickType : uint8;
struct FInputKeyParams;

DECLARE_LOG_CATEGORY_EXTERN(LogWorldSubsystemInput, Log, All);

// Per local player input subsystem
UCLASS(MinimalAPI)
class UEnhancedInputLocalPlayerSubsystem : public ULocalPlayerSubsystem, public IEnhancedInputSubsystemInterface
{
	GENERATED_BODY()

public:

	// Begin ULocalPlayerSubsystem
	UE_API virtual void Deinitialize() override;
	UE_API virtual void PlayerControllerChanged(APlayerController* NewPlayerController) override;
	// End ULocalPlayerSubsystem
	
	// Begin IEnhancedInputSubsystemInterface
	UE_API virtual UEnhancedPlayerInput* GetPlayerInput() const override;
	UE_API virtual UEnhancedInputUserSettings* GetUserSettings() const override;
	UE_API virtual void InitalizeUserSettings() override;
	UE_API virtual void ControlMappingsRebuiltThisFrame() override;
	UE_API virtual void AddMappingContext(const UInputMappingContext* MappingContext, int32 Priority, const FModifyContextOptions& Options = FModifyContextOptions()) override;
	UE_API virtual void RemoveMappingContext(const UInputMappingContext* MappingContext, const FModifyContextOptions& Options = FModifyContextOptions()) override;
protected:
	virtual TMap<TObjectPtr<const UInputAction>, FInjectedInput>& GetContinuouslyInjectedInputs() override { return ContinuouslyInjectedInputs; }
	// End IEnhancedInputSubsystemInterface
	
public:

	template<class UserSettingClass = UEnhancedInputUserSettings>
	inline UserSettingClass* GetUserSettings() const
	{
		return Cast<UserSettingClass>(GetUserSettings());
	}

	/** A delegate that will be called when control mappings have been rebuilt this frame. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnControlMappingsRebuilt);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMappingContextAdded, const UInputMappingContext*, MappingContext);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMappingContextRemoved, const UInputMappingContext*, MappingContext);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnUserSettingsPostInitialized, const UEnhancedInputUserSettings*, Settings);

	/**
	 * Blueprint Event that is called at the end of any frame that Control Mappings have been rebuilt.
	 */
	UPROPERTY(BlueprintAssignable, DisplayName=OnControlMappingsRebuilt, Category = "Input")
	FOnControlMappingsRebuilt ControlMappingsRebuiltDelegate;

	/*
	* A callback fired when a mapping context is added (AddMappingContext is called on this subsystem)
	* 
	* Note:  This does not make any guarantee that the control mappings will have been rebuilt. 
	* If you need that, then listen to the ControlMappingsRebuiltDelegate instead.
	*/
	UPROPERTY(BlueprintAssignable, Category = "Input")
	FOnMappingContextAdded OnMappingContextAdded;

	/*
	* A callback fired when a mapping context is removed (RemoveMappingContext is called on this subsystem)
	* 
	* Note:  This does not make any guarantee that the control mappings will have been rebuilt. 
	* If you need that, then listen to the ControlMappingsRebuiltDelegate instead.
	*/
	UPROPERTY(BlueprintAssignable, Category = "Input")
	FOnMappingContextRemoved OnMappingContextRemoved;

	/**
	 * Delegate that is fired after this local player's User Settings object has been created and loaded for the first time.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Input")
	FOnUserSettingsPostInitialized OnPostUserSettingsInitialized;

protected:
    	
	/** The user settings for this subsystem used to store each user's input related settings */
	UPROPERTY()
	TObjectPtr<UEnhancedInputUserSettings> UserSettings;

	// Map of inputs that should be injected every frame. These inputs will be injected when ForcedInput is ticked. 
	UPROPERTY(Transient) 
	TMap<TObjectPtr<const UInputAction>, FInjectedInput> ContinuouslyInjectedInputs;
	
};

/**
 * Per world input subsystem that allows you to bind input delegates to actors without an owning Player Controller. 
 * This should be used when an actor needs to receive input delegates but will never have an owning Player Controller.
 * For example, you can add input delegates to unlock a door when the user has a certain set of keys pressed.
 * Be sure to enable input on the actor, or else the input delegates won't fire!
 * 
 * Note: if you do have an actor with an owning Player Controller use the local player input subsystem instead.
 */
UCLASS(MinimalAPI, DisplayName="Enhanced Input World Subsystem (Experimental)")
class UEnhancedInputWorldSubsystem : public UWorldSubsystem, public IEnhancedInputSubsystemInterface
{

// The Enhanced Input Module ticks the player input on this subsystem
friend class FEnhancedInputModule;
// The input processor tells us about what keys are pressed
friend class FEnhancedInputWorldProcessor;

	GENERATED_BODY()

public:

	//~ Begin UWorldSubsystem interface
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
protected:
	UE_API virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const;
	//~ End UWorldSubsystem interface

public:	
	//~ Begin IEnhancedInputSubsystemInterface
	UE_API virtual UEnhancedPlayerInput* GetPlayerInput() const override;
	UE_API virtual void ShowDebugInfo(UCanvas* Canvas) override;
protected:
	virtual TMap<TObjectPtr<const UInputAction>, FInjectedInput>& GetContinuouslyInjectedInputs() override { return ContinuouslyInjectedInputs; }
	//~ End IEnhancedInputSubsystemInterface

public:
	/** Adds this Actor's input component onto the stack to be processed by this subsystem's tick function */
	UFUNCTION(BlueprintCallable, Category = "Input|World", meta=(DefaultToSelf = "Actor"))
	UE_API void AddActorInputComponent(AActor* Actor);

	/** Removes this Actor's input component from the stack to be processed by this subsystem's tick function */
	UFUNCTION(BlueprintCallable, Category = "Input|World", meta = (DefaultToSelf = "Actor"))
	UE_API bool RemoveActorInputComponent(AActor* Actor);
	
protected:

	/** 
	* Inputs a key on this subsystem's player input which can then be processed as normal during Tick.
	* 
	* This should only be called by the FEnhancedInputWorldProcessor 
	*/
	UE_API bool InputKey(const FInputKeyEventArgs& Params);

	/** 
	* Builds the current input stack and ticks the world subsystem's player input.
	* 
	* Called from the Enhanced Input Module Tick.
	* 
	* The Enhanced Input local player subsystem will have their Player Input's ticked by their owning 
	* Player Controller in APlayerController::TickPlayerInput, but because the world subsystem has no 
	* owning controller we need to tick it elsewhere.
	*/
	UE_API void TickPlayerInput(float DeltaTime);

	/** Adds all the default mapping contexts */
	UE_API void AddDefaultMappingContexts();

	/** Removes all the default mapping contexts */
	UE_API void RemoveDefaultMappingContexts();

	/** The player input that is processing the input within this subsystem */
	UPROPERTY()
	TObjectPtr<UEnhancedPlayerInput> PlayerInput = nullptr;

	/**
	 * Input processor that is created on Initalize.
	 */
	TSharedPtr<FEnhancedInputWorldProcessor> InputPreprocessor = nullptr;	
	
	/** Internal. This is the current stack of InputComponents that is being processed by the PlayerInput. */
	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<UInputComponent>> CurrentInputStack;

	// Map of inputs that should be injected every frame. These inputs will be injected when ForcedInput is ticked. 
	UPROPERTY(Transient) 
	TMap<TObjectPtr<const UInputAction>, FInjectedInput> ContinuouslyInjectedInputs;
};

#undef UE_API
