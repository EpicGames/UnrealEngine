// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/PlayerInput.h"
#include "InputAction.h"
#include "GameplayTagContainer.h"
#include "NativeGameplayTags.h"

#include "EnhancedPlayerInput.generated.h"

#define UE_API ENHANCEDINPUT_API

class UInputModifier;
class UInputTrigger;
enum class ETriggerEvent : uint8;
enum class ETriggerState : uint8;
struct FEnhancedActionKeyMapping;
class UEnhancedInputUserSettings;

// Internal representation containing event variants
enum class ETriggerEventInternal : uint8;
enum class EKeyEvent : uint8;
class UInputMappingContext;

namespace UE::EnhancedInput
{
	/**
	 * The default input mode of Enhanced Input. Every Input Mapping Context will it's default filtering query set to
	 * check for this exact tag. 
	 */
	ENHANCEDINPUT_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(InputMode_Default);
};

// ContinuouslyInjectedInputs Map is not managed.
// Continuous input injections seem to be getting garbage collected and
// crashing in UObject::ProcessEvent when calling ModifyRaw.
// Band-aid fix: Making these managed references. Also check modifications to
// IEnhancedInputSubsystemInterface::Start/StopContinuousInputInjectionForAction.
USTRUCT()
struct FInjectedInput
{
	GENERATED_BODY()

	FInputActionValue RawValue;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UInputTrigger>> Triggers;
	UPROPERTY(Transient)
	TArray<TObjectPtr<UInputModifier>> Modifiers;
};

USTRUCT()
struct FKeyConsumptionOptions
{
	GENERATED_BODY()
	
	/** Keys that should be consumed if the trigger state is reached */
	TArray<FKey> KeysToConsume;
		
	/** A bitmask of trigger events that when reached, should cause the key to be marked as consumed. */
	ETriggerEvent EventsToCauseConsumption = ETriggerEvent::None;
};

USTRUCT()
struct FInjectedInputArray
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	TArray<FInjectedInput> Injected;
};

USTRUCT()
struct FAppliedInputContextData
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Priority = 0;

	/**
	 * Used when RegistrationTrackingMode is set to CountRegistrations
	 * This is how many times the IMC has been added minus how many times it has been removed. The IMC is unregistered when this hits 0.
	 */
	UPROPERTY()
	int32 RegistrationCount = 0;
};

/**
* UEnhancedPlayerInput : UPlayerInput extensions for enhanced player input system
*/
UCLASS(MinimalAPI, config = Input, transient)
class UEnhancedPlayerInput : public UPlayerInput
{
	friend class IEnhancedInputSubsystemInterface;
	friend class UEnhancedInputLibrary;
	friend struct FInputTestHelper;

	GENERATED_BODY()

public:

	UE_API UEnhancedPlayerInput();

	//~ Begin UPlayerInput interface
	UE_API virtual void FlushPressedKeys() override;
	//~ End UPlayerInput interface

	/**
	* Returns the action instance data for the given input action if there is any. Returns nullptr if the action is not available.
	*/
	const FInputActionInstance* FindActionInstanceData(TObjectPtr<const UInputAction> ForAction) const { return ActionInstanceData.Find(ForAction); }

	/** Retrieve the current value of an action for this player.
	* Note: If the action is not currently triggering this will return a zero value of the appropriate value type, ignoring any ongoing inputs.
	*/
	UE_API FInputActionValue GetActionValue(TObjectPtr<const UInputAction> ForAction) const;

	// Input simulation via injection. Runs modifiers and triggers delegates as if the input had come through the underlying input system as FKeys. Applies action modifiers and triggers on top.
	UE_API void InjectInputForAction(TObjectPtr<const UInputAction> Action, FInputActionValue RawValue, const TArray<UInputModifier*>& Modifiers = {}, const TArray<UInputTrigger*>& Triggers = {});

	UE_API virtual bool InputKey(const FInputKeyEventArgs& Params) override;

	/** Returns the Time Dilation value that is currently effecting this input. */
	UE_API float GetEffectiveTimeDilation() const;

	/**
	 * Returns a const ref to the current input mode. 
	 */
	UE_API const FGameplayTagContainer& GetCurrentInputMode() const;

	/**
	 * @return The name which should be used to save the input settings to.
	 * By default, this will return InputSettingsSaveSlotName from the developer settings.
	 */
	UE_API virtual FString GetUserSettingsSaveFileName() const;

	/**
	 * Returns a const array view of the current enhanced input action mappings.
	 * 
	 * Note: Do NOT hold onto any references to these mappings or their input triggers/modifiers, as they will get
	 * re-instanced every time the control mappings are rebuilt. This is for read-only access to view the current mappings
	 * which may be useful for systems which want to query specific data about the current trigger states of specific mappings.
	 */
	UE_API TConstArrayView<const FEnhancedActionKeyMapping> GetEnhancedActionMappingsView() const;
	
protected:

	/**
	 * Sets the current input mode to be the given NewMode.
	 * 
	 * This should only be called by the Enhanced Input subsystem interface, and followed up by a call to RequestRebuildControlMappings.
	 */
	UE_API void SetCurrentInputMode(const FGameplayTagContainer& NewMode);

	/**
	 * Returns a mutable reference to the current input mode
	 *
	 * This should only be called by the Enhanced Input subsystem interface, and followed up by a call to RequestRebuildControlMappings.
	 */
	UE_API FGameplayTagContainer& GetCurrentInputMode();

	UE_API virtual void EvaluateKeyMapState(const float DeltaTime, const bool bGamePaused, OUT TArray<TPair<FKey, FKeyState*>>& KeysWithEvents) override;
	UE_API virtual void EvaluateInputDelegates(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused, const TArray<TPair<FKey, FKeyState*>>& KeysWithEvents) override;
	UE_API virtual void PrepareInputDelegatesForEvaluation(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused, const TArray<TPair<FKey, FKeyState*>>& KeysWithEvents) override;
	UE_API virtual bool EvaluateInputComponentDelegates(UInputComponent* const IC, const TArray<TPair<FKey, FKeyState*>>& KeysWithEvents, const float DeltaTime, const bool bGamePaused) override;
	UE_API virtual void EvaluateBlockedInputComponent(UInputComponent* InputComponent) override;
	
	// Causes key to be consumed if it is affecting an action.
	UE_API virtual bool IsKeyHandledByAction(FKey Key) const override;
	
	/** Note: Source reference only. Use GetEnhancedActionMappings() for the actual mappings (with properly instanced triggers/modifiers) */
	const TMap<TObjectPtr<const UInputMappingContext>, FAppliedInputContextData>& GetAppliedInputContextData() const { return AppliedInputContextData; }
	UE_DEPRECATED(5.6, "GetAppliedInputContexts() is deprecated, use GetAppliedInputContextData() instead")
	const TMap<TObjectPtr<const UInputMappingContext>, int32>& GetAppliedInputContexts() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return AppliedInputContexts;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** This player's version of the Action Mappings */
	const TArray<FEnhancedActionKeyMapping>& GetEnhancedActionMappings() const { return EnhancedActionMappings; }

	/**
	* Notifies that the given Input Actions are no longer mapped to any keys, and should have their state reset.
	*/
	UE_API void NotifyInputActionsUnmapped(const TSet<const UInputAction*>& RemovedInputActions);

	/** Array of data that represents what keys should be consumed if an enhanced input action is in a specific triggered state */
	UPROPERTY()
	TMap<TObjectPtr<const UInputAction>, FKeyConsumptionOptions> KeyConsumptionData;
	
	/** Tracked action values. Queryable. */
	UPROPERTY(Transient)
	mutable TMap<TObjectPtr<const UInputAction>, FInputActionInstance> ActionInstanceData;

	/** 
	* Set of consumed input actions this frame. 
	* Populated during the evaluation of input delegates and cleared at the end of the evaluation 
	*/
	UPROPERTY(Transient)
	TSet<TObjectPtr<const UInputAction>> ConsumedInputActions;
	
private:

	/** Add a player specific action mapping.
	* Returns index into EnhancedActionMappings array.
	*/
	UE_API int32 AddMapping(const FEnhancedActionKeyMapping& Mapping);
	UE_API void ClearAllMappings();

	UE_API virtual void ConditionalBuildKeyMappings_Internal() const override;

	// Perform a first pass run of modifiers on an action instance
	UE_API void InitializeMappingActionModifiers(const FEnhancedActionKeyMapping& Mapping);

	UE_API FInputActionValue ApplyModifiers(const TArray<UInputModifier*>& Modifiers, FInputActionValue RawValue, float DeltaTime) const;						// Pre-modified (raw) value
	UE_API ETriggerEventInternal GetTriggerStateChangeEvent(ETriggerState LastTriggerState, ETriggerState NewTriggerState) const;
	UE_API ETriggerEvent ConvertInternalTriggerEvent(ETriggerEventInternal Event) const;	// Collapse a detailed internal trigger event into a friendly representation

	UE_API void ProcessActionMappingEvent(
		TObjectPtr<const UInputAction> Action,
		float DeltaTime,
		bool bGamePaused,
		FInputActionValue RawValue,
		EKeyEvent KeyEvent,
		const TArray<UInputModifier*>& Modifiers,
		const TArray<UInputTrigger*>& Triggers,
		const bool bHasAlwaysTickTrigger = false);

	UE_API FInputActionInstance& FindOrAddActionEventData(TObjectPtr<const UInputAction> Action) const;

	template<typename T>
	void GatherActionEventDataForActionMap(const T& ActionMap, TMap<TObjectPtr<const UInputAction>, FInputActionInstance>& FoundActionEventData) const;

	/**
	 * Currently applied key mappings
	 * Note: Source reference only. Use EnhancedActionMappings for the actual mappings (with properly instanced triggers/modifiers)
	 *
	 * These mapping contexts will only have their mappings processed if the current input mode
	 * matches the query set on them.
	*/
	UPROPERTY(Transient)
	TMap<TObjectPtr<const UInputMappingContext>, FAppliedInputContextData> AppliedInputContextData;
	UE_DEPRECATED(5.6, "AppliedInputContexts is deprecated, use AppliedInputContextData instead")
	UPROPERTY(Transient)
	TMap<TObjectPtr<const UInputMappingContext>, int32> AppliedInputContexts;

	/** This player's version of the Action Mappings */
	UPROPERTY(Transient)
	TArray<FEnhancedActionKeyMapping> EnhancedActionMappings;

	/**
	 * The current input mode that is active on this player. If Input Mapping contexts have requirements
	 * which this container does not meet, then their mappings will not be applied.
	 */
	UPROPERTY(Transient)
	FGameplayTagContainer CurrentInputMode;

	// Number of active binds by key
	TMap<FKey, int32> EnhancedKeyBinds;

	/** Actions which had actuated events at the last call to ProcessInputStack (held/pressed/released) */
	TSet<TObjectPtr<const UInputAction>> ActionsWithEventsThisTick;

	/** Actions that have been triggered this tick and have a delegate that may be fired */
	TSet<TObjectPtr<const UInputAction>> TriggeredActionsThisTick;

	/** A set of input actions that have been removed from the player's input mappings in a previous rebuild of the key mappings. */
	TSet<TObjectPtr<const UInputAction>> ActionsThatHaveBeenRemovedFromMappings;

	/**
	 * A map of Keys to the amount they were depressed this frame. This is reset with each call to ProcessInputStack
	 * and is populated within UEnhancedPlayerInput::InputKey.
	 */
	UPROPERTY(Transient)
	TMap<FKey, FVector> KeysPressedThisTick;

	/** Inputs injected since the last call to ProcessInputStack */
	UPROPERTY(Transient)
	TMap<TObjectPtr<const UInputAction>, FInjectedInputArray> InputsInjectedThisTick;

	/** Last frame's injected inputs */
	UPROPERTY(Transient)
	TSet<TObjectPtr<const UInputAction>> LastInjectedActions;

	/** Used to keep track of Input Actions that have UInputTriggerChordAction triggers on them */
	struct FDependentChordTracker
	{
		/** The Input Action that has the UInputTriggerChordAction on it */
		TObjectPtr<const UInputAction> SourceAction;
		
		/** The action that is referenced by the SourceAction's Chord trigger */
		TObjectPtr<const UInputAction> DependantAction;
	};
	
	/**
	 * Array of all dependant Input Action's with Chord triggers on them.
	 * Populated by IEnhancedInputSubsystemInterface::ReorderMappings
	 */
	TArray<FDependentChordTracker> DependentChordActions;

protected:

	// We need to grab the down states of all keys before calling Super::ProcessInputStack as it will leave bDownPrevious in the same state as bDown (i.e. this frame, not last).
	TMap<FKey, bool> KeyDownPrevious;
	
	/** 
	* If true, then FlushPressedKeys has been called and the input key state map has been flushed.
	* 
	* This will be set to true in UEnhancedPlayerInput::FlushPressedKeys, and reset to false at the end of
	* UEnhancedPlayerInput::ProcessInputStack
	*/
	uint8 bIsFlushingInputThisFrame : 1;

	/**
	* If there is a key mapping to EKeys::AnyKey, we will keep track of what key was used when we first found a "Pressed"
	* event. That way we can use the same key when we wait for a "Released" event.
	*/
	FName CurrentlyInUseAnyKeySubstitute;

private:

	/** The last time of the last frame that was processed in ProcessPlayerInput */
	float LastFrameTime = 0.0f;

	/** Delta seconds between frames calculated with UWorld::GetRealTimeSeconds */
	float RealTimeDeltaSeconds = 0.0f;
};

#undef UE_API
