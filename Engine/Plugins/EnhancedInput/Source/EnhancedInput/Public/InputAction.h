// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "InputModifiers.h"
#include "InputTriggers.h"

#include "InputAction.generated.h"

#define UE_API ENHANCEDINPUT_API

struct FPropertyChangedEvent;

class UPlayerMappableKeySettings;

enum class ETriggerEventInternal : uint8;

/** 
* This is an advanced setting that allows you to change how the value of an Input Action is calculated when there are 
* multiple mappings to the same Input Action. The default behavior is to accept highest absolute value.
*/
UENUM()
enum class EInputActionAccumulationBehavior : uint8
{
	/** 
	* Take the value from the mapping with the highest Absolute Value.
	* 
	* For example, given a value of -0.3 and 0.5, the input action's value would be 0.5. 
	*/
	TakeHighestAbsoluteValue,

	/** 
	* Cumulatively adds the key values for each mapping.
	* 
	* For example, a value of -0.7 and +0.75 on the same input action would result in a value of 0.05.
	* 
	* A practical example of when to use this would be for something like WASD movement, if you want pressing W and S to cancel each other out.
	*/
	Cumulative,
};

/**
* An Input Action is a logical representation of something the user can do, such as "Jump" or "Crouch".
* These are what your gameplay code binds to in order to listen for input state changes. For most scenarios 
* your gameplay code should be listening for the "Triggered" event on an input action. This will allow
* for the most scalable and customizable input configuration because you can add different triggers 
* for each key mapping in the Input Mapping Context. 
* 
* They are the conceptual equivalent to "Action" and "Axis" mapping names from the Legacy Input System.
* 
* Note: These are instanced per player (via FInputActionInstance)
*/
UCLASS(MinimalAPI, BlueprintType)
class UInputAction : public UDataAsset
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	// Track actions that have had their ValueType changed to update blueprints referencing them.
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

	static UE_API TSet<const UInputAction*> ActionsWithModifiedValueTypes;
	static UE_API TSet<const UInputAction*> ActionsWithModifiedTriggers;
	
	/**
	 * Returns a bitmask of supported trigger events that is built from each UInputTrigger on this Action.
	 */
	UE_API ETriggerEventsSupported GetSupportedTriggerEvents() const;

	DECLARE_MULTICAST_DELEGATE(FTriggersChanged);
	DECLARE_MULTICAST_DELEGATE(FModifiersChanged);

	FTriggersChanged OnTriggersChanged;
	FModifiersChanged OnModifiersChanged;
#endif // WITH_EDITOR
	
	/**
	* Returns the Player Mappable Key Settings for this Input Action.
	*/
	const TObjectPtr<UPlayerMappableKeySettings>& GetPlayerMappableKeySettings() const { return PlayerMappableKeySettings; }

	// A localized descriptor of this input action
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Description")
	FText ActionDescription = FText::GetEmpty();

	// Should this action be able to trigger whilst the game is paused - Replaces bExecuteWhenPaused
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Action)
    bool bTriggerWhenPaused = false;
	
	// Should this action swallow any inputs bound to it or allow them to pass through to affect lower priority bound actions?
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input Consumption", meta=(DisplayName="Consume Lower Priority Enhanced Input Mappings"))
	bool bConsumeInput = true;

	/**
	 * Should this Input Action consume any legacy Action and Axis key mappings?
	 * If true, then any key mapping to this input action will consume(aka block) the legacy key
	 * mapping from firing delegates.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input Consumption")
	bool bConsumesActionAndAxisMappings = false;

	// This action's mappings are not intended to be automatically overridden by higher priority context mappings. Users must explicitly remove the mapping first. NOTE: It is the responsibility of the author of the mapping code to enforce this!
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Action)
	bool bReserveAllMappings = false;	// TODO: Need something more complex than this?

	/** A bitmask of trigger events that, when reached, will consume any FKeys mapped to this input action. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input Consumption", meta=(EditCondition = "bConsumesActionAndAxisMappings", Bitmask, BitmaskEnum="/Script/EnhancedInput.ETriggerEvent"))
	int32 TriggerEventsThatConsumeLegacyKeys;
	
	// The type that this action returns from a GetActionValue query or action event
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Action, AssetRegistrySearchable)
	EInputActionValueType ValueType = EInputActionValueType::Boolean;

	/**
	* This defines how the value of this input action will be calcuated in the case that there are multiple key mappings to the same input action.
	* 
	* When TakeHighestAbsoluteValue is selected, then the key mapping with the highest absolutle value will be utilized. (Default)
	* When Cumulative is selected, then each key mapping will be added together to get the key value. 
	* 
	* @see UEnhancedPlayerInput::ProcessActionMappingEvent, where this property is read from. 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Action, AdvancedDisplay)
	EInputActionAccumulationBehavior AccumulationBehavior = EInputActionAccumulationBehavior::TakeHighestAbsoluteValue;
	
	/**
	* Trigger qualifiers. If any trigger qualifiers exist the action will not trigger unless:
	* At least one Explicit trigger in this list has been met.
	* All Implicit triggers in this list are met.
	*/
	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, Category = Action)
	TArray<TObjectPtr<UInputTrigger>> Triggers;

	/**
	* Modifiers are applied to the final action value.
	* These are applied sequentially in array order.
	* They are applied on top of any FEnhancedActionKeyMapping modifiers that drove the initial input
	* 
	* Note: Modifiers defined in the Input Action asset will be applied AFTER any modifiers defined in individual key mappings in the Input Mapping Context asset.
	*/
	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, Category = Action, meta=(DisplayAfter="Triggers"))
	TArray<TObjectPtr<UInputModifier>> Modifiers;

protected:

	/**
	* Holds setting information about this Action Input for setting screen and save purposes.
	*/
	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, Category = "User Settings", meta=(AllowPrivateAccess))
	TObjectPtr<UPlayerMappableKeySettings> PlayerMappableKeySettings;
};

// Calculate a collective representation of trigger state from evaluations of all triggers in one or more trigger groups.
struct FTriggerStateTracker
{
	// Trigger rules by evaluated trigger count:
	// Implicits == 0, Explicits == 0	- Always fire, unless value is 0.
	// Implicits == 0, Explicits  > 0	- At least one explict has fired.
	// Implicits  > 0, Explicits == 0	- All implicits have fired.
	// Implicits  > 0, Explicits  > 0	- All implicits and at least one explicit have fired.
	// Blockers   > 0					- Override all other triggers to force trigger failure.

	// Add a group of triggers to the evaluated state, returning the new trigger state.
	ETriggerState EvaluateTriggers(const UEnhancedPlayerInput* PlayerInput, const TArray<UInputTrigger*>& Triggers, FInputActionValue ModifiedValue, float DeltaTime);

	ETriggerState GetState() const;

	void SetMappingTriggerApplied(bool bNewVal) { bMappingTriggerApplied = bNewVal; }
	bool GetMappingTriggerApplied() const { return bMappingTriggerApplied; }

	bool operator>=(const FTriggerStateTracker& Other) const
	{
		ETriggerState State = GetState();
		ETriggerState OtherState = Other.GetState();
		return Tie(State, bEvaluatedTriggers) >= Tie(OtherState, Other.bEvaluatedTriggers);
	}
	bool operator< (const FTriggerStateTracker& Other) const
	{
		ETriggerState State = GetState();
		ETriggerState OtherState = Other.GetState();
		return Tie(State, bEvaluatedTriggers) < Tie(OtherState, Other.bEvaluatedTriggers);
	}

	// TODO: Hacky. This is the state we should return if we have evaluated no valid triggers. Set during action evaluation based on final ModifiedValue.
	void SetStateForNoTriggers(ETriggerState State);

private:
	ETriggerState NoTriggerState = ETriggerState::None;

	bool bEvaluatedInput = false;		// Non-zero input value was provided at some point in the evaluation
	bool bEvaluatedTriggers = false;	// At least one valid trigger was evaluated
	bool bFoundActiveTrigger = false;	// If any trigger is in an ongoing or triggered state the final state must be at least ongoing (with the exception of blocking triggers!)
	bool bAnyExplictTriggered = false;
	bool bFoundExplicit = false;		// If no explicits are found the trigger may fire through implicit testing only. If explicits exist at least one must be met.
	bool bAllImplicitsTriggered = true;
	bool bBlocking = false;				// If any trigger is blocking, we can't fire.
	bool bMappingTriggerApplied = false; // Set to true when an actionmapping is processed and triggers were found
};


// Run time queryable action instance
// Generated from UInputAction templates above
USTRUCT(BlueprintType)
struct FInputActionInstance
{
	friend class UEnhancedPlayerInput;

	GENERATED_BODY()

private:

	// The source action that this instance is created from
	UPROPERTY(BlueprintReadOnly, Category = "Input", Meta = (AllowPrivateAccess))
	TObjectPtr<const UInputAction> SourceAction = nullptr;

	// Internal trigger states
	ETriggerState LastTriggerState = ETriggerState::None;
	FTriggerStateTracker TriggerStateTracker;
	ETriggerEventInternal TriggerEventInternal = ETriggerEventInternal(0);

protected:

	// Trigger state
	UPROPERTY(BlueprintReadOnly, Transient, Category = Action)
	ETriggerEvent TriggerEvent = ETriggerEvent::None;

	// The last time that this evaluated to a Triggered State
	UPROPERTY(BlueprintReadOnly, Transient, Category = Action)
	float LastTriggeredWorldTime = 0.0f;

	UPROPERTY(Instanced, BlueprintReadOnly, Category = Config)
	TArray<TObjectPtr<UInputTrigger>> Triggers;

	UPROPERTY(Instanced, BlueprintReadOnly, Category = Config)
	TArray<TObjectPtr<UInputModifier>> Modifiers;

	// Combined value of all inputs mapped to this action
	FInputActionValue Value;

	// Total trigger processing/evaluation time (How long this action has been in event Started, Ongoing, or Triggered
	UPROPERTY(BlueprintReadOnly, Category = Action)
	float ElapsedProcessedTime = 0.f;

	// Triggered time (How long this action has been in event Triggered only)
	UPROPERTY(BlueprintReadOnly, Category = Action)
	float ElapsedTriggeredTime = 0.f;

public:
	FInputActionInstance() = default;
	UE_API FInputActionInstance(const UInputAction* InSourceAction);

	// Current trigger event
	ETriggerEvent GetTriggerEvent() const { return TriggerEvent; }

	// Current action value - Will be zero if the current trigger event is not ETriggerEvent::Triggered!
	UE_API FInputActionValue GetValue() const;

	// Returns the evaluated trigger state
	ETriggerState GetEvaluatedActionTriggerState() const;

	// Total time the action has been evaluating triggering (Ongoing & Triggered)
	float GetElapsedTime() const { return ElapsedProcessedTime; }

	// Time the action has been actively triggered (Triggered only)
	float GetTriggeredTime() const { return ElapsedTriggeredTime; }

	// Time that this action was last actively triggered
	float GetLastTriggeredWorldTime() const { return LastTriggeredWorldTime; }

	const TArray<TObjectPtr<UInputTrigger>>& GetTriggers() const { return Triggers; }
	const TArray<TObjectPtr<UInputModifier>>& GetModifiers() const { return Modifiers; }

	// The source action that this instance is created from
	TObjectPtr<const UInputAction> GetSourceAction() const { return SourceAction; }

	void AddInputTrigger(UInputTrigger* InputTrigger) { Triggers.Add(InputTrigger); }
	void RemoveInputTrigger(UInputTrigger* InputTrigger) { Triggers.Remove(InputTrigger); }
};

#undef UE_API
