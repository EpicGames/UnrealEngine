// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputCoreTypes.h"

#include "InputActionValue.generated.h"

UENUM(BlueprintType)
enum class EInputActionValueType : uint8
{
	// Value types in increasing size order (used for type promotion)
	// Name these Digital/Analog?

	Boolean				UMETA(DisplayName = "Digital (bool)"),
	Axis1D				UMETA(DisplayName = "Axis1D (float)"),
	Axis2D				UMETA(DisplayName = "Axis2D (Vector2D)"),
	Axis3D				UMETA(DisplayName = "Axis3D (Vector)"),
};


USTRUCT(BlueprintType, meta = (HasNativeMake = "/Script/EnhancedInput.EnhancedInputLibrary.MakeInputActionValueOfType", HasNativeBreak = "/Script/EnhancedInput.EnhancedInputLibrary.BreakInputActionValue"))
struct FInputActionValue
{
	GENERATED_BODY()

public:
	//using Button = bool;
	using Axis1D = float;
	using Axis2D = FVector2D;
	using Axis3D = FVector;

	// Support all relevant default constructors (FInputActionValue isn't movable)
	FInputActionValue() = default;
	FInputActionValue(const FInputActionValue&) = default;
	FInputActionValue& operator= (const FInputActionValue&) = default;

	// Construct default value for a key
	FInputActionValue(FKey KeyDefault) : Value(0.f), ValueType(GetValueTypeFromKey(KeyDefault)) {}

	// Specialized constructors for supported types
	// Converting a value to a different type (e.g. Val = FVector(1, 1, 1); Val = true;) zeroes out any unused components to ensure getters continue to function correctly.
	explicit FInputActionValue(bool bInValue) :	Value(bInValue ? 1.f : 0.f, 0.f, 0.f),	ValueType(EInputActionValueType::Boolean) {}
	FInputActionValue(Axis1D InValue) : Value(InValue, 0.f, 0.f),						ValueType(EInputActionValueType::Axis1D) {}
	FInputActionValue(Axis2D InValue) : Value(InValue, 0.f),							ValueType(EInputActionValueType::Axis2D) {}
	FInputActionValue(Axis3D InValue) : Value(InValue),									ValueType(EInputActionValueType::Axis3D) {}

	// Build a specific type with an arbitrary Axis3D value
	FInputActionValue(EInputActionValueType InValueType, Axis3D InValue) : Value(InValue), ValueType(InValueType)
	{
		// Clear out value components to match type
		switch (ValueType)
		{
		case EInputActionValueType::Boolean:
		case EInputActionValueType::Axis1D:
			Value.Y = 0.f;
			//[[fallthrough]];
		case EInputActionValueType::Axis2D:
			Value.Z = 0.f;
			//[[fallthrough]];
		case EInputActionValueType::Axis3D:
		default:
			return;
		}
	}

	// Resets Value without affecting ValueType
	void Reset()
	{
		Value = FVector::ZeroVector;
	}

	FInputActionValue& operator+=(const FInputActionValue& Rhs)
	{
		Value += Rhs.Value;
		// Promote value type to largest number of bits.
		ValueType = FMath::Max(ValueType, Rhs.ValueType);
		return *this;
	}

	friend FInputActionValue operator+(const FInputActionValue& Lhs, const FInputActionValue& Rhs)
	{
		FInputActionValue Result(Lhs);
		Result += Rhs;
		return Result;
	}

	// Scalar operators
	FInputActionValue& operator*=(float Scalar)
	{
		Value *= Scalar;
		return *this;
	}

	friend FInputActionValue operator*(const FInputActionValue& Lhs, const float Rhs)
	{
		FInputActionValue Result(Lhs);
		Result *= Rhs;
		return Result;
	}

	// TODO: Would prefer a value type checked version here but this is complicated by the ability to change action value types within the editor.
	// TODO: Binding helpers to limit key types that can be bound? Stop players binding an Axis2D input device to a bool action value.

	template<typename T>
	inline T Get() const { static_assert(sizeof(T) == 0, "Unsupported conversion for type"); }

	// Read only index based value accessor, doesn't care about type. Expect 0 when accessing unused components.
	float operator[](int32 Index) const { return Value[Index]; }

	bool IsNonZero(float Tolerance = KINDA_SMALL_NUMBER) const { return Value.SizeSquared() >= Tolerance * Tolerance; }

	// In-place type conversion
	FInputActionValue& ConvertToType(EInputActionValueType Type)
	{
		if (ValueType != Type)
		{
			*this = FInputActionValue(Type, Value);
		}
		return *this;
	}
	FInputActionValue& ConvertToType(const FInputActionValue& Other) { return ConvertToType(Other.GetValueType()); }

	EInputActionValueType GetValueType() const { return ValueType; }
	static EInputActionValueType GetValueTypeFromKey(FKey Key)
	{
		if (Key.IsAxis3D())
		{
			return EInputActionValueType::Axis3D;
		}
		else if (Key.IsAxis2D())
		{
			return EInputActionValueType::Axis2D;
		}
		else if (Key.IsAxis1D())
		{
			return EInputActionValueType::Axis1D;
		}
		return EInputActionValueType::Boolean;
	}

	float GetMagnitudeSq() const
	{
		switch (GetValueType())
		{
		case EInputActionValueType::Boolean:
		case EInputActionValueType::Axis1D:
			return Value.X * Value.X;
		case EInputActionValueType::Axis2D:
			return Value.SizeSquared2D();
		case EInputActionValueType::Axis3D:
			return Value.SizeSquared();
		}

		checkf(false, TEXT("Unsupported value type for input action value!"));
		return 0.f;
	}

	float GetMagnitude() const
	{
		switch (GetValueType())
		{
		case EInputActionValueType::Boolean:
		case EInputActionValueType::Axis1D:
			return Value.X;
		case EInputActionValueType::Axis2D:
			return Value.Size2D();
		case EInputActionValueType::Axis3D:
			return Value.Size();
		}

		checkf(false, TEXT("Unsupported value type for input action value!"));
		return 0.f;
	}

	// Type sensitive debug stringify
	inline FString ToString() const
	{
		switch (GetValueType())
		{
		case EInputActionValueType::Boolean:
			return FString(IsNonZero() ? TEXT("true") : TEXT("false"));
		case EInputActionValueType::Axis1D:
			return FString::Printf(TEXT("%3.3f"), Value.X);
		case EInputActionValueType::Axis2D:
			return FString::Printf(TEXT("X=%3.3f Y=%3.3f"), Value.X, Value.Y);
		case EInputActionValueType::Axis3D:
			return FString::Printf(TEXT("X=%3.3f Y=%3.3f Z=%3.3f"), Value.X, Value.Y, Value.Z);
		}

		checkf(false, TEXT("Unsupported value type for input action value!"));
		return FString{};
	}

protected:

	FVector Value = FVector::ZeroVector;

	EInputActionValueType ValueType = EInputActionValueType::Boolean;

};

// Supported getter specializations
template<>
inline bool FInputActionValue::Get() const
{
	// True if any component is non-zero
	return IsNonZero();
}

template<>
inline FInputActionValue::Axis1D FInputActionValue::Get() const
{
	return Value.X;
}

template<>
inline FInputActionValue::Axis2D FInputActionValue::Get() const
{
	return Axis2D(Value.X, Value.Y);
}

template<>
inline FInputActionValue::Axis3D FInputActionValue::Get() const
{
	return Value;
}
