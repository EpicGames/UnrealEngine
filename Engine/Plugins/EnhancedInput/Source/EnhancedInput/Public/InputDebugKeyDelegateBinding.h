// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "Engine/InputDelegateBinding.h"
#include "Framework/Commands/InputChord.h"
#include "InputDebugKeyDelegateBinding.generated.h"

#define UE_API ENHANCEDINPUT_API

class UInputComponent;

USTRUCT()
struct FBlueprintInputDebugKeyDelegateBinding
{
	GENERATED_BODY()

	UPROPERTY()
	FInputChord InputChord;

	UPROPERTY()
	TEnumAsByte<EInputEvent> InputKeyEvent = IE_Pressed;

	UPROPERTY()
	FName FunctionNameToBind = NAME_None;

	UPROPERTY()
	bool bExecuteWhenPaused = false;

	// TODO: bConsumeInput?
};

UCLASS(MinimalAPI)
class UInputDebugKeyDelegateBinding : public UInputDelegateBinding
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<FBlueprintInputDebugKeyDelegateBinding> InputDebugKeyDelegateBindings;

	//~ Begin UInputDelegateBinding Interface
	UE_API virtual void BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const override;
	//~ End UInputDelegateBinding Interface
};

#undef UE_API
