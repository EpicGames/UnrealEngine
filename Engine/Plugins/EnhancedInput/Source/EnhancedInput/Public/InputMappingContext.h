// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EnhancedActionKeyMapping.h"
#include "GameplayTagContainer.h"

#include "InputMappingContext.generated.h"

#define UE_API ENHANCEDINPUT_API

struct FKey;

class UInputAction;

/**
 * Options for an input mapping context being filtered based on the current input mode of the player.
 */
UENUM()
enum class EMappingContextInputModeFilterOptions : uint8
{
	/**
	 * This mapping context should use the project's default input mode query.
	 * 
	 * @see UEnhancedInputDeveloperSettings::DefaultMappingContextInputModeQuery
	 * 
	 * (Project Settings -> Engine -> Enhanced Input -> Input Modes)
	 */
	UseProjectDefaultQuery,

	/**
	 * This mapping context should use a custom input mode query instead of the project default.
	 */
	UseCustomQuery,

	/**
	 * This Input mapping context should not be filtered based on the current mode, effectively ignoring
	 * the current mode.
	 */
	DoNotFilter
};

/**
 * Options for how multiple registrations of an input mapping context should be tracked.
 */
UENUM()
enum class EMappingContextRegistrationTrackingMode : uint8
{
	/**
	 * This is the default behavior.
	 * Registrations of the Input Mapping Context are not tracked. The mapping context will be unregistered when removing it the first time, no matter how many times it has been added.
	 */
	Untracked,

	/**
	 * Track how many times the IMC is added and keeps the IMC applied until the IMC is removed the same number of times.
	 * This allows multiple systems to use the same Input Mapping Context without needing to check if any other systems are still using the same Input Mapping Context.
	 *
	 * Warnings will be logged if Input Mapping Contexts with this tracking mode are still applied at deinitialization, the expectation is that there will be a RemoveMappingContext() call for every call to AddMappingContext() when using this mode.
	 * 
	 * @see IEnhancedInputSubsystemInterface::ValidateTrackedMappingContextsAreUnregistered
	 */
	CountRegistrations
};

USTRUCT(BlueprintType)
struct FInputMappingContextMappingData
{
	GENERATED_BODY()
	
	/**
	 * List of Input Actions and which keys they are mapped to.
	 */
	UPROPERTY(config, BlueprintReadOnly, EditAnywhere, Category = "Mappings")
	TArray<FEnhancedActionKeyMapping> Mappings;
};

/**
* UInputMappingContext : A collection of key to action mappings for a specific input context
* Could be used to:
*	Store predefined controller mappings (allow switching between controller config variants). TODO: Build a system allowing redirects of UInputMappingContexts to handle this.
*	Define per-vehicle control mappings
*	Define context specific mappings (e.g. I switch from a gun (shoot action) to a grappling hook (reel in, reel out, disconnect actions).
*	Define overlay mappings to be applied on top of existing control mappings (e.g. Hero specific action mappings in a MOBA)
*/
UCLASS(MinimalAPI, BlueprintType, config = Input)
class UInputMappingContext : public UDataAsset
{
	GENERATED_BODY()

protected:
	// List of key to action mappings.
	UE_DEPRECATED(5.7, "Use the DefaultKeyMappings struct instead.")
	UPROPERTY(config, BlueprintReadOnly, Category = "Mappings", meta = (DeprecatedProperty="Note", DeprecationMessage = "Use the DefaultKeyMappings struct instead."))
	TArray<FEnhancedActionKeyMapping> Mappings;

	/**
	 * List of Input Actions to which FKeys they are mapped to.
	 */
	UPROPERTY(config, BlueprintReadOnly, EditAnywhere, Category = "Mappings")
	FInputMappingContextMappingData DefaultKeyMappings;
	
	/**
	 * Key mappings that should be used INSTEAD of the default "Mappings" array above then the
	 * provided Remappable Key Profile of the given name is currently active.
	 *
	 * If there are no override mappings specified for a set for a specific mapping profile, then the default mappings will be used. 
	 */
	UPROPERTY(config, BlueprintReadOnly, EditAnywhere, Category = "Mappings")
	TMap<FString, FInputMappingContextMappingData> MappingProfileOverrides; 

public:

#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif
	
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	
protected:
	
	/**
     * Defines how this input mapping context should be filtered based on the current input mode. 
     *
     * Default is Use Project Default Query.
     * 
     * @Note: bEnableInputModeFiltering must be enabled in the UEnhancedInputDeveloperSettings for this to be considered.
     */
    UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Input Modes", meta = (EditCondition="ShouldShowInputModeQuery()"))
	EMappingContextInputModeFilterOptions InputModeFilterOptions = EMappingContextInputModeFilterOptions::UseProjectDefaultQuery;
	
	/**
	 * Tag Query which will be matched against the current Enhanced Input Subsystem's input mode if InputModeFilterOptions is set to UseCustomQuery. 
	 *
	 * If this tag query does not match with the current input mode tag container, then the mappings
	 * will not be processed.
	 *
	 * @Note: bEnableInputModeFiltering must be enabled in the UEnhancedInputDeveloperSettings for this to be considered.
	 */
	UPROPERTY(config, BlueprintReadOnly, EditAnywhere, Category = "Input Modes", meta = (EditConditionHides, EditCondition="ShouldShowInputModeQuery() && InputModeFilterOptions == EMappingContextInputModeFilterOptions::UseCustomQuery"))
	FGameplayTagQuery InputModeQueryOverride;

	/**
	 * Select the behaviour when multiple AddMappingContext() calls are made for this Input Mapping Context
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Registration")
	EMappingContextRegistrationTrackingMode RegistrationTrackingMode = EMappingContextRegistrationTrackingMode::Untracked;

public:
	
	/**
	 * Executes the given lambda on every default and override key mapping specified on this mapping context.
	 * 
	 * @param Func Lambda function to execute on a given key mapping
	 */
	UE_API void ForEachKeyMapping(TFunctionRef<void(const FEnhancedActionKeyMapping&)>&& Func) const;

	/**
	* @return True if this mapping context should be filtered based on the current input mode.
	*/
	UE_API bool ShouldFilterMappingByInputMode() const;

	/**
	 * @return The tag query which should be used when deciding whether this mapping context should be filtered
	 * out based on the current input mode or not. 
	 */
	UE_API FGameplayTagQuery GetInputModeQuery() const;

	/**
	 * @return The registration tracking mode that this IMC is using. 
	 */
	EMappingContextRegistrationTrackingMode GetRegistrationTrackingMode() const { return RegistrationTrackingMode; }

	/** @return The input mode filter options set on this mapping context. */
	EMappingContextInputModeFilterOptions GetInputModeFilterOptions() const { return InputModeFilterOptions; }
	
	// Localized context descriptor
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Description", DisplayName = "Description")
	FText ContextDescription;

	friend class FInputContextDetails;
	friend class FActionMappingsNodeBuilderEx;

	/** UFUNCTION helper to be used as en edit condition for displaying input mode query releated properties. */
	UFUNCTION()
	static UE_API bool ShouldShowInputModeQuery();

	/**
	 * @param ProfileId The Player Mappable Key Profile ID
	 * @return True if this Input Mapping Context has some specific override mappings for the given profile. 
	 */
	UE_API bool HasMappingsForProfile(const FString& ProfileId) const;

	/**
	 * @return An array of profile identifiers which this IMC specifies overrides for.
	 */
	UE_API TArray<FString> GetProfilesWithOverridenMappings() const;

	/**
	 * @param ProfileId The Player Mappable Key Profile ID
	 * @return Reference to the key mappings to use for the given profile. If there is no override mapping specified for the given
	 *		   profile, then the default mappings will be returned.
	 */
	UE_API const TArray<FEnhancedActionKeyMapping>& GetMappingsForProfile(const FString& ProfileId) const;

	/**
	 * Returns true if this Input Mapping Context contains a reference to the given Input Action in its default or profile override mappings.
	 */
	UE_API bool HasMappingForInputAction(const UInputAction* Action) const;

	
	// TODO_BH: Deprecate all of these functions below. They are only used in the editor and for tests! Then we can also get rid of the friend decls above
	
	/**
	* Mapping accessors.
	* Note: Use UEnhancedInputLibrary::RequestRebuildControlMappingsForContext to invoke changes made to an FEnhancedActionKeyMapping
	*/
	const TArray<FEnhancedActionKeyMapping>& GetMappings() const { return DefaultKeyMappings.Mappings; }
	FEnhancedActionKeyMapping& GetMapping(TArray<FEnhancedActionKeyMapping>::SizeType Index) { return DefaultKeyMappings.Mappings[Index]; }

	// TODO: Don't want to encourage Map/Unmap calls here where context switches would be desirable. These are intended for use in the config/binding screen only.

	/**
	* Map a key to an action within the mapping context.
	*/
	UFUNCTION(BlueprintCallable, Category = "Mapping")
	UE_API FEnhancedActionKeyMapping& MapKey(const UInputAction* Action, FKey ToKey);

	/**
	* Unmap a key from an action within the mapping context.
	*/
	UFUNCTION(BlueprintCallable, Category = "Mapping")
	UE_API void UnmapKey(const UInputAction* Action, FKey Key);
	
	/**
	* Unmap all key maps to an action within the mapping context.
	*/
	UFUNCTION(BlueprintCallable, Category = "Mapping")
	UE_API void UnmapAllKeysFromAction(const UInputAction* Action);

	/**
	* Unmap everything within the mapping context.
	*/
	UFUNCTION(BlueprintCallable, Category = "Mapping")
	UE_API void UnmapAll();
};

#undef UE_API
