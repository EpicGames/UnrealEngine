// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "InputMappingQuery.generated.h"

// Result summary from a QueryMapKeyIn... call
UENUM(BlueprintType)
enum class EMappingQueryResult : uint8
{
	// Query failed because the player controller being queried is not configured to support enhanced input (PlayerInput is not Enhanced).
	Error_EnhancedInputNotEnabled,

	// Query failed because the input context being queried against is not part of the active context list.
	Error_InputContextNotInActiveContexts,

	// Query failed because the action being queried against is None/null
	Error_InvalidAction,

	// Mapping cannot be applied due to blocking issues. Check OutIssues for details.
	NotMappable,

	// Mapping will not affect any existing mappings and is safe to apply.
	MappingAvailable,
};

// Mapping issues arising from a QueryMapKeyIn... call
UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true", ScriptName = "MappingQueryIssueFlag"))
enum class EMappingQueryIssue : uint8
{
	// Mapping will not affect any existing mappings and is safe to apply.
	NoIssue									= 0x00,

	// Mapping has been reserved for the exclusive use of another action. The new mapping should be refused.
	ReservedByAction						= 0x01,

	// Mapping will cause an existing mapping to be hidden and/or need remapping.
	HidesExistingMapping					= 0x02,

	// Mapping will not be functional, due to an existing mapping blocking it.
	HiddenByExistingMapping					= 0x04,

	// Mapping will be functional, but a collision with another mapping within this context may cause issues.
	CollisionWithMappingInSameContext		= 0x08,

	// Trying to bind an FKey with a less complex type than the bound action expects (e.g. Keyboard key bound to a 2D Gamepad axis. May not be desirable). Note: bool -> Axis1D promotions are never considered forced.
	ForcesTypePromotion						= 0x10,

	// Trying to bind an FKey with a more complex type than the bound action supports (this could behave oddly e.g. 2D Gamepad axis bound to a 1D axis will ignore Y axis)
	ForcesTypeDemotion						= 0x20,
};
ENUM_CLASS_FLAGS(EMappingQueryIssue);

// Useful mapping query issue collections.
namespace DefaultMappingIssues
{
inline constexpr EMappingQueryIssue NoCollisions = EMappingQueryIssue::HidesExistingMapping | EMappingQueryIssue::HiddenByExistingMapping | EMappingQueryIssue::CollisionWithMappingInSameContext;
inline constexpr EMappingQueryIssue TypeMismatch = EMappingQueryIssue::ForcesTypePromotion | EMappingQueryIssue::ForcesTypeDemotion;
inline constexpr EMappingQueryIssue StandardFatal = EMappingQueryIssue::ReservedByAction | NoCollisions | TypeMismatch;	// Default fatal value for QueryMapKeyIn... function calls.
}

class UInputMappingContext;
class UInputAction;

// Potential issue raised with a mapping request
USTRUCT(BlueprintType)
struct FMappingQueryIssue
{
	GENERATED_BODY()

	FMappingQueryIssue() = default;
	FMappingQueryIssue(EMappingQueryIssue InIssue) : Issue(InIssue) {};

	UPROPERTY(BlueprintReadOnly, Category = Result)
	EMappingQueryIssue Issue = EMappingQueryIssue::NoIssue;

	// Input context that contains a blocking action bound to the queried key
	UPROPERTY(BlueprintReadOnly, Category = Result)
	TObjectPtr<const UInputMappingContext> BlockingContext = nullptr;

	// Action within the input context that caused the blockage
	UPROPERTY(BlueprintReadOnly, Category = Result)
	TObjectPtr<const UInputAction> BlockingAction = nullptr;
};

// ************************************************************************************************
// ************************************************************************************************
// ************************************************************************************************
