// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputActionValue.h"
#include "InputModifiers.generated.h"

#define UE_API ENHANCEDINPUT_API

class UEnhancedPlayerInput;

/**
Base class for building modifiers.
*/
UCLASS(MinimalAPI, Abstract, Blueprintable, EditInlineNew, CollapseCategories, Config = Input, defaultconfig, configdonotcheckdefaults)
class UInputModifier : public UObject
{
	GENERATED_BODY()

protected:

	/** ModifyRaw implementation. Override this to alter input values in native code.
	 * @param CurrentValue - The modified value returned by the previous modifier in the chain, or the base input device value if this is the first modifier in the chain.
	 * @param DeltaTime - Elapsed time since last input tick.
	 * @return Modified value. Note that whilst the returned value can be of any FInputActionValueType it will be reset to the value type of the associated action before any further processing.
	 */
	virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime)
	{
		return CurrentValue;
	}

	virtual FLinearColor GetVisualizationColor_Implementation(FInputActionValue SampleValue, FInputActionValue FinalValue) const
	{
		const float Intensity = FMath::Min(1.f, FinalValue.GetMagnitude());	// TODO: 3D visualization!
		return FLinearColor(Intensity, Intensity, Intensity);
	}

public:

	/**
	 * ModifyRaw
	 * Will be called by each modifier in the modifier chain
	 * @param CurrentValue - The modified value returned by the previous modifier in the chain, or the base raw value if this is the first modifier in the chain.
	 */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Modifier")
	UE_API FInputActionValue ModifyRaw(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) const;

	/**
	 * Helper to allow debug visualization of the modifier.
	 * @param SampleValue - The base input action value pre-modification (ranging -1 -> 1 across all applicable axes).
	 * @param FinalValue - The post-modification input action value for the provided SampleValue.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Modifier")
	UE_API FLinearColor GetVisualizationColor(FInputActionValue SampleValue, FInputActionValue FinalValue) const;

	/**
	 * Called during RebuildControlMappings when this modifier instance replaces an older instance
	 * of the same class. Transfers runtime state from the old modifier so that stateful modifiers
	 * like Smooth don't lose their accumulated state.
	 *
	 * The old modifier is guaranteed to be the same class (verified by FEnhancedActionKeyMapping::Equals).
	 *
	 * This function dispatches to ReceiveModifierReinstanced for subclass state.
	 */
	UE_API void OnModifierReinstanced(const UInputModifier* OldModifier);

protected:

	/**
	 * Override to transfer custom runtime state from a previous modifier instance of the same class.
	 * Called by OnModifierReinstanced.
	 * Subclasses should call Super to ensure parent state is transferred.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Modifier")
	UE_API void ReceiveModifierReinstanced(const UInputModifier* OldModifier);
	virtual void ReceiveModifierReinstanced_Implementation(const UInputModifier* OldModifier) {}
};

/** Behavior options for how the value is calculated in UInputModifierSmoothDelta. */
UENUM()
enum class ENormalizeInputSmoothingType : uint8
{
	None UMETA(Hidden),
	Lerp,
	Interp_To,
	Interp_Constant_To,
	Interp_Circular_In, 
	Interp_Circular_Out, 
	Interp_Circular_In_Out,
	Interp_Ease_In, 
	Interp_Ease_Out, 
	Interp_Ease_In_Out,
	Interp_Expo_In,
	Interp_Expo_Out,
	Interp_Expo_In_Out,
	Interp_Sin_In,
	Interp_Sin_Out,
	Interp_Sin_In_Out
};

/**
* Normalized Smooth Delta
* 
* Produces a smoothed normalized delta of the current(new) and last(old) input value.
* Boolean input values will be returned as is.
*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Smooth Delta"))
class UInputModifierSmoothDelta : public UInputModifier
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, Config)
	ENormalizeInputSmoothingType SmoothingMethod = ENormalizeInputSmoothingType::Lerp;

	// Speed, or Alpha. If the speed given is 0, then jump to the target.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, Config)
	float Speed = 0.5f;

	/**
	 * For ease functions, this controls the degree of the curve.
	 * 
	 * This only affects the Interp_Ease_In, Interp_Ease_Out, and Interp_Ease_In_Out smoothing methods
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, Config, meta=(EditCondition="SmoothingMethod == ENormalizeInputSmoothingType::Interp_Ease_In ||SmoothingMethod == ENormalizeInputSmoothingType::Interp_Ease_Out || SmoothingMethod == ENormalizeInputSmoothingType::Interp_Ease_In_Out"))
	float EasingExponent = 2.0f;

protected:
	virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) override;
	UE_API virtual void ReceiveModifierReinstanced_Implementation(const UInputModifier* OldModifier) override;

	FVector OldValue = FVector::ZeroVector;
    
    FVector Delta = FVector::ZeroVector;
};


UENUM()
enum class EDeadZoneType : uint8
{
	// Apply dead zone to axes individually. This will result in input being chamfered at the corners for
	// 2d/3d axis inputs, and matches the original UE4 deadzone logic.
	Axial,
	
	// Apply dead zone logic to all axes simultaneously. This gives smooth input (circular/spherical coverage).
	// On a 1d axis input this works identically to Axial.
	// 
	// For most games, this will give the smoothest feeling analog values. The input is smoothed to avoid
	// "jumpiness" when you are moving the analog axis.
	Radial UMETA(DisplayName="Smoothed Radial"),

	// Apply dead zone logic to all axes simultaneously without any smooth input
	// which the normal "Radial" deadzone applies.
	//
	// The behavior of this deadzone type is as follows:
	// If the magnitude of the input is less then the lower threshold, ignore it.
	// Clamp the magnitude of the input to the upper threshold value.
	//
	// For some games, this may result in feeling "jumpy", because the value goes from 0.0 to
	// the lower threshold immediately instead of being smoothed, like the normal "Radial" deadzone option.
	UnscaledRadial
};

/** Dead Zone
    *  Input values within the range LowerThreshold -> UpperThreshold will be remapped from 0 -> 1.
	*  Values outside this range will be clamped.
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Dead Zone"))
class UInputModifierDeadZone : public UInputModifier
{
	GENERATED_BODY()

public:

	/** 
	* Threshold below which input is ignored.
	*
	* This value should always be less than the UpperThreshold.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings, Config, meta=(ClampMin=0, ClampMax=1))
	float LowerThreshold = 0.2f;

	/** 
	* Threshold above which input is clamped to 1.0.
	* 
	* This value should always be greater than the "Lower Threshold" value. 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings, Config, meta=(ClampMin=0, ClampMax=1))
	float UpperThreshold = 1.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings, Config)
	EDeadZoneType Type = EDeadZoneType::Radial;

protected:

#if WITH_EDITOR
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) override;

	// Visualize as black when unmodified. Red when blocked (with differing intensities to indicate axes)
	// Mirrors visualization in https://www.gamasutra.com/blogs/JoshSutphin/20130416/190541/Doing_Thumbstick_Dead_Zones_Right.php.
	virtual FLinearColor GetVisualizationColor_Implementation(FInputActionValue SampleValue, FInputActionValue FinalValue) const override;
};

/** Scalar
	*  Scales input by a set factor per axis
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Scalar"))
class UInputModifierScalar : public UInputModifier
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif

	/**
	 * The scalar that will be applied to the input value.
	 *
	 * For example, if you have a scalar of (2.0, 2.0, 2.0), each input axis will be multiplied by 2.0.
	 *  
	 * Note: This will do nothing on boolean input action types, as they can only be true or false.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	FVector Scalar = FVector::OneVector;

protected:
	virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) override;
};

/** Scalar
	*  Multiplies the input value by Delta Time this frame
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Scale By Delta Time"))
class UInputModifierScaleByDeltaTime : public UInputModifier
{
	GENERATED_BODY()
	
protected:
	virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) override;
};

/** Negate
	*  Inverts input per axis
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Negate"))
class UInputModifierNegate : public UInputModifier
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bX = true;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bY = true;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bZ = true;

protected:
	virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) override;
	virtual FLinearColor GetVisualizationColor_Implementation(FInputActionValue SampleValue, FInputActionValue FinalValue) const override;
};

/** Smooth
	*  Smooth inputs out over multiple frames
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Smooth"))
class UInputModifierSmooth : public UInputModifier
{
	GENERATED_BODY()

	void ClearSmoothedAxis();

protected:

	// TODO: Smoothing variants. Configuration options. e.g. smooth over a set time/frame count.

	/** How long input has been zero. */
	float ZeroTime = 0.f;

	/** Current average input/sample */
	FInputActionValue AverageValue;

	/** Number of samples since input  has been zero */
	int32 Samples = 0;

#define SMOOTH_TOTAL_SAMPLE_TIME_DEFAULT (0.0083f)
	/** Input sampling total time. */
	float TotalSampleTime = SMOOTH_TOTAL_SAMPLE_TIME_DEFAULT;

	virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) override;
	UE_API virtual void ReceiveModifierReinstanced_Implementation(const UInputModifier* OldModifier) override;
};

/** Response Curve Exponential
	*  Apply a simple exponential response curve to input values, per axis
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Response Curve - Exponential"))
class UInputModifierResponseCurveExponential : public UInputModifier
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, Config)
	FVector CurveExponent = FVector::OneVector;

protected:
	virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) override;
};

class UCurveFloat;

/** Response Curve User Defined
	*  Apply a custom response curve to input values, per axis
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Response Curve - User Defined"))
class UInputModifierResponseCurveUser : public	UInputModifier
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (DisplayThumbnail = "false"))
	TObjectPtr<UCurveFloat> ResponseX;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (DisplayThumbnail = "false"))
	TObjectPtr<UCurveFloat> ResponseY;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (DisplayThumbnail = "false"))
	TObjectPtr<UCurveFloat> ResponseZ;

protected:
	virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) override;
};

UENUM()
enum class EFOVScalingType : uint8
{
	// FOV scaling to apply scaled movement deltas to inputs dependent upon the player's selected FOV
	Standard,

	// FOV scaling was incorrectly calculated in UE4's UPlayerInput::MassageAxisInput. This implementation is intended to aid backwards compatibility, but should not be used by new projects.
	UE4_BackCompat,
};

/** FOV Scaling
	* Apply FOV dependent scaling to input values, per axis
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "FOV Scaling"))
class UInputModifierFOVScaling : public	UInputModifier
{
	GENERATED_BODY()

public:

	// Extra scalar applied on top of basic FOV scaling.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, Config)
	float FOVScale = 1.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, Config)
	EFOVScalingType FOVScalingType = EFOVScalingType::Standard;	// TODO: UE4_BackCompat by default?

protected:
	virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) override;
};


/** Input space to World space conversion
	* Auto-converts axes within the Input Action Value into world space	allowing the result to be directly plugged into functions that take world space values.
	* E.g. For a 2D input axis up/down is mapped to world X (forward), whilst axis left/right is mapped to world Y (right).
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "To World Space"))
class UInputModifierToWorldSpace : public	UInputModifier
{
	GENERATED_BODY()

protected:
	virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) override;
	virtual FLinearColor GetVisualizationColor_Implementation(FInputActionValue SampleValue, FInputActionValue FinalValue) const override;
};


UENUM()
enum class EInputAxisSwizzle : uint8
{
	// Swap X and Y axis. Useful for binding 1D inputs to the Y axis for 2D actions.
	YXZ,

	// Swap X and Z axis
	ZYX,

	// Swap Y and Z axis
	XZY,

	// Reorder all axes, Y first
	YZX,

	// Reorder all axes, Z first
	ZXY,
};

/** Swizzle axis components of an input value.
	* Useful to map a 1D input onto the Y axis of a 2D action.
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Swizzle Input Axis Values"))
class UInputModifierSwizzleAxis : public UInputModifier
{
	GENERATED_BODY()

public:

	// Default to XY swap, useful for binding 1D inputs to the Y axis.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EInputAxisSwizzle Order = EInputAxisSwizzle::YXZ;

protected:
	virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) override;
	virtual FLinearColor GetVisualizationColor_Implementation(FInputActionValue SampleValue, FInputActionValue FinalValue) const override;
};

#undef UE_API
