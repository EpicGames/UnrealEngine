// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputActionValue.h"

#include "InputTriggers.generated.h"

#define UE_API ENHANCEDINPUT_API

class UEnhancedPlayerInput;
class UInputAction;

/**
* Trigger states are a light weight interpretation of the provided input values used in trigger UpdateState responses.
*/
UENUM()
enum class ETriggerState : uint8
{
	// No inputs
	None,

	// Triggering is being monitored, but not yet been confirmed (e.g. a time based trigger that requires the trigger state to be maintained over several frames)
	Ongoing,

	// The trigger state has been met
	Triggered,
};

/**
* Trigger events are the Action's interpretation of all Trigger State transitions that occurred for the action in the last tick
*/
UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class ETriggerEvent : uint8
{
	// No significant trigger state changes occurred and there are no active device inputs
	None		= (0x0)		UMETA(Hidden),
	// Triggering occurred after one or more processing ticks
	Triggered	= (1 << 0),	// ETriggerState (None -> Triggered, Ongoing -> Triggered, Triggered -> Triggered)
	
	// An event has occurred that has begun Trigger evaluation. Note: Triggered may also occur this frame, but this event will always be fired first.
	Started		= (1 << 1),	// ETriggerState (None -> Ongoing, None -> Triggered)

	// Triggering is still being processed. For example, an action with a "Press and Hold" trigger
	// will be "Ongoing" while the user is holding down the key but the time threshold has not been met yet. 
	Ongoing		= (1 << 2),	// ETriggerState (Ongoing -> Ongoing)

	// Triggering has been canceled. For example,  the user has let go of a key before the "Press and Hold" time threshold.
	// The action has started to be evaluated, but never completed. 
	Canceled	= (1 << 3),	// ETriggerState (Ongoing -> None)

	// The trigger state has transitioned from Triggered to None this frame, i.e. Triggering has finished.
	// Note: Using this event restricts you to one set of triggers for Started/Completed events. You may prefer two actions, each with its own trigger rules.
	// Completed will not fire if any trigger reports Ongoing on the same frame, but both should fire. e.g. Tick 2 of Hold (= Ongoing) + Pressed (= None) combo will raise Ongoing event only.
	Completed	= (1 << 4),	// ETriggerState (Triggered -> None)
};
ENUM_CLASS_FLAGS(ETriggerEvent)

namespace UE::Input
{
	// Returns an FString that represents the given ETriggerEvent
	ENHANCEDINPUT_API FString LexToString(const ETriggerEvent TriggerEvent);
};

/**
* Trigger type determine how the trigger contributes to an action's overall trigger event the behavior of the trigger
*/
UENUM()
enum class ETriggerType : uint8
{
	// Input may trigger if any explicit trigger is triggered.
	Explicit,

	// Input may trigger only if all implicit triggers are triggered.
	Implicit,

	// Inverted trigger that will block all other triggers when IsBlocking returns true
	Blocker
};

/**
 * Represents what ETriggerEvent types can be triggered off of a give UInputTrigger.
 * Used to determine if a warning should be placed on a exec pin that would never get called
 * on a K2Node_EnhancedInputAction node
 */
UENUM()
enum class ETriggerEventsSupported : uint8
{
	// This trigger supports no trigger events. 
	None				= (0x0),
	
	// This trigger supports just the Triggered event type. This include ETriggerEvent::Triggered
	Instant				= (1 << 0),

	// This trigger cannot be canceled, once it is started it cannot be stopped from being triggered
	// This includes the ETriggerEvents of Started, Ongoing, and Triggered 
	Uninterruptible		= (1 << 1),

	// This represents a trigger than can be held down and have a duration, and be canceled. This includes
	// the ETriggerEvents of Started, Ongoing, Canceled, and Triggered. 
	Ongoing				= (1 << 2),

	// This trigger supports all trigger events
	All					= (Instant | Uninterruptible | Ongoing), 
};
ENUM_CLASS_FLAGS(ETriggerEventsSupported)

/**
Base class for building triggers.
Transitions to Triggered state once the input meets or exceeds the actuation threshold.
*/
UCLASS(MinimalAPI, Abstract, Blueprintable, EditInlineNew, CollapseCategories, Config = Input, defaultconfig, configdonotcheckdefaults)
class UInputTrigger : public UObject
{
	GENERATED_BODY()


protected:

	// Default implementations of overridable functionality.
	// C++ trigger implementations should override these directly.
	virtual ETriggerType GetTriggerType_Implementation() const { return ETriggerType::Explicit; }

	// Triggers on actuation.
	UE_API virtual ETriggerState UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime);

public:
	// Point at which this trigger fires
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category = "Trigger Settings")
	float ActuationThreshold = 0.5f;

	/* Decides whether this trigger ticks every frame or not.
	 * This WILL affect performance and should only be used in specific custom triggers.
	 */
	UPROPERTY(Config, BlueprintReadOnly, Category = "Trigger Settings")
	bool bShouldAlwaysTick = false;

	/*
	* Is the value passed in sufficiently large to be of interest to the trigger.
	* This is a helper function that implements the most obvious (>=) interpretation of the actuation threshold.
	*/
	UFUNCTION(BlueprintCallable, Category="Trigger")
	bool IsActuated(const FInputActionValue& ForValue) const { return ForValue.GetMagnitudeSq() >= ActuationThreshold * ActuationThreshold; }

	// Value passed to UpdateState on the previous tick. This will be updated automatically after the trigger is updated.
	UPROPERTY(BlueprintReadOnly, Category = "Trigger Settings")
	FInputActionValue LastValue;	// TODO: Potential issues with this being of bool type on first tick.

	/*
	Changes the way this trigger affects an action with multiple triggers:
		All implicit triggers must be triggering to trigger the action.
		If there are any explicit triggers at least one must be triggering to trigger the action.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Trigger")
	UE_API ETriggerType GetTriggerType() const;

	/*
	This function checks if the requisite conditions have been met for the trigger to fire.
	 Returns Trigger State None		 - No trigger conditions have been met. Trigger is inactive.
			 Trigger State Ongoing	 - Some trigger conditions have been met. Trigger is processing but not yet active.
			 Trigger State Triggered - All trigger conditions have been met to fire. Trigger is active.
	*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Trigger")
	UE_API ETriggerState UpdateState(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime);

	/** Determines what kind of trigger events can happen from the behavior of this trigger. */
	virtual ETriggerEventsSupported GetSupportedTriggerEvents() const { return ETriggerEventsSupported::Instant; }

	/**
	 * Returns true if the the ETriggerEvent is can be triggered based off of the ETriggerEventsSupported.
	 */
	static UE_API bool IsSupportedTriggerEvent(const ETriggerEventsSupported SupportedEvents, const ETriggerEvent Event);

	/**
	 * Called during RebuildControlMappings when this trigger instance replaces an older instance
	 * of the same class. Transfers runtime state from the old trigger so that state-dependent
	 * triggers like Pressed don't see false transitions, and timed triggers don't restart.
	 *
	 * The old trigger is guaranteed to be the same class (verified by FEnhancedActionKeyMapping::Equals).
	 *
	 * This function copies LastValue, then dispatches to ReceiveTriggerReinstanced for subclass state.
	 */
	UE_API void OnTriggerReinstanced(const UInputTrigger* OldTrigger);

protected:

	/**
	 * Override to transfer custom runtime state from a previous trigger instance of the same class.
	 * Called by OnTriggerReinstanced after LastValue has already been copied.
	 * Subclasses should call Super to ensure parent state is transferred.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Trigger")
	UE_API void ReceiveTriggerReinstanced(const UInputTrigger* OldTrigger);
	virtual void ReceiveTriggerReinstanced_Implementation(const UInputTrigger* OldTrigger) {}

public:

	/** Returns true when all other triggers should be blocked, i.e. chords block when not triggered. */
	virtual bool IsBlocking(const ETriggerState State) const { return false; }

	// Provide debug output for use with ShowDebug EnhancedInput. Return an empty string to disable display.
	virtual FString GetDebugState() const { return FString(); }
};


/**
Base class for building triggers that have firing conditions governed by elapsed time.
This class transitions state to Ongoing once input is actuated, and will track Ongoing input time until input is released.
Inheriting classes should provide the logic for Triggered transitions.
*/
UCLASS(MinimalApi, Abstract, Config = Input)	// TODO: Parent CLASS_Config flag is passed down to all UInputTrigger descendants, but ClassConfigName is only inherited by immediate children causing UHT to complain about a missing config file for any triggers based on this.
class UInputTriggerTimedBase : public UInputTrigger
{
	GENERATED_BODY()

protected:

	// How long have we been actuating this trigger?
	UPROPERTY(BlueprintReadWrite, Category = "Trigger Settings")
	float HeldDuration = 0.0f;			 // TODO: Annoying given the action mapping is already tracking this.

	// Transitions to Ongoing on actuation. Never triggers.
	UE_API virtual ETriggerState UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime) override;

	// Calculates the new held duration given the current player input and delta time
	UE_API float CalculateHeldDuration(const UEnhancedPlayerInput* const PlayerInput, const float DeltaTime) const;

	UE_API virtual void ReceiveTriggerReinstanced_Implementation(const UInputTrigger* OldTrigger) override;

public:

	virtual ETriggerEventsSupported GetSupportedTriggerEvents() const override { return ETriggerEventsSupported::Ongoing; }
	
	/**
	 * Should global time dilation be applied to the held duration?
	 * Default is set to false.
	 * 
	 * If this is set to true, then the owning Player Controller's actor time dilation
	 * will be used when calculating the HeldDuration.
	 * 
	 * @see UInputTriggerTimedBase::CalculateHeldDuration
	 * @see AWorldSettings::GetEffectiveTimeDilation
	 */
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category = "Trigger Settings")
	bool bAffectedByTimeDilation = false;

	virtual FString GetDebugState() const override { return HeldDuration ? FString::Printf(TEXT("Held:%.2f"), HeldDuration) : FString(); }
};



// Default native triggers


// Default behavior for no triggers is Down

/** UInputTriggerDown
	Trigger fires when the input exceeds the actuation threshold.
	Note: When no triggers are bound Down (with an actuation threshold of > 0) is the default behavior.
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Down"))
class UInputTriggerDown final : public UInputTrigger
{
	GENERATED_BODY()

public:
	virtual ETriggerEventsSupported GetSupportedTriggerEvents() const override { return ETriggerEventsSupported::Instant; }

protected:

	virtual ETriggerState UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime) override;
};

/** UInputTriggerPressed
	Trigger fires once only when input exceeds the actuation threshold.
	Holding the input will not cause further triggers.
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta=(DisplayName="Pressed"))
class UInputTriggerPressed final : public UInputTrigger
{
	GENERATED_BODY()

public:
	virtual ETriggerEventsSupported GetSupportedTriggerEvents() const override { return ETriggerEventsSupported::Instant; }

protected:

	virtual ETriggerState UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime) override;
	virtual FString GetDebugState() const { return IsActuated(LastValue) ? FString(TEXT("Pressed:Held")) : FString(); }
};


/** UInputTriggerReleased
	Trigger returns Ongoing whilst input exceeds the actuation threshold.
	Trigger fires once only when input drops back below actuation threshold.
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Released"))
class UInputTriggerReleased final : public UInputTrigger
{
	GENERATED_BODY()

public:
	virtual ETriggerEventsSupported GetSupportedTriggerEvents() const override { return ETriggerEventsSupported::Instant; }

protected:

	virtual ETriggerState UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime) override;
	virtual FString GetDebugState() const { return IsActuated(LastValue) ? FString(TEXT("Released:Held")) : FString(); }
};



/** UInputTriggerHold
	Trigger fires once input has remained actuated for HoldTimeThreshold seconds.
	Trigger may optionally fire once, or repeatedly fire.
*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Hold"))
class UInputTriggerHold final : public UInputTriggerTimedBase
{
	GENERATED_BODY()

	bool bTriggered = false;

protected:

	virtual ETriggerState UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime) override;
	UE_API virtual void ReceiveTriggerReinstanced_Implementation(const UInputTrigger* OldTrigger) override;

public:
	virtual ETriggerEventsSupported GetSupportedTriggerEvents() const override { return ETriggerEventsSupported::Ongoing; }
	
	// How long does the input have to be held to cause trigger?
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category = "Trigger Settings", meta = (ClampMin = "0"))
	float HoldTimeThreshold = 1.0f;

	// Should this trigger fire only once, or fire every frame once the hold time threshold is met?
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category = "Trigger Settings")
	bool bIsOneShot = false;

	virtual FString GetDebugState() const override { return HeldDuration ? FString::Printf(TEXT("Hold:%.2f/%.2f"), HeldDuration, HoldTimeThreshold) : FString(); }
};

/** UInputTriggerHoldAndRelease
	Trigger fires when input is released after having been actuated for at least HoldTimeThreshold seconds.
*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Hold And Release"))
class UInputTriggerHoldAndRelease final : public UInputTriggerTimedBase
{
	GENERATED_BODY()

protected:

	virtual ETriggerState UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime) override;

public:
	// How long does the input have to be held to cause trigger?
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category = "Trigger Settings", meta = (ClampMin = "0"))
	float HoldTimeThreshold = 0.5f;
};

/** UInputTriggerTap
	Input must be actuated then released within TapReleaseTimeThreshold seconds to trigger.
*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Tap"))
class UInputTriggerTap final : public UInputTriggerTimedBase
{
	GENERATED_BODY()

protected:

	virtual ETriggerState UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime) override;

public:
	virtual ETriggerEventsSupported GetSupportedTriggerEvents() const override { return ETriggerEventsSupported::Instant; }
	
	// Release within this time-frame to trigger a tap
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category = "Trigger Settings", meta = (ClampMin = "0"))
	float TapReleaseTimeThreshold = 0.2f;
};

/**
 * Fires the "Triggered" event upon a repeated taps of a key.
 * 
 * Note: You can use this trigger to easily make a "Double Tap" effect.
 */
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Repeated Tap"))
class UInputTriggerRepeatedTap : public UInputTriggerTimedBase
{
	GENERATED_BODY()

protected:
	virtual ETriggerState UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime) override;
	virtual FString GetDebugState() const override;

public:
	UE_API UInputTriggerRepeatedTap();

	/**
	 * The max amount of time which can pass in between single taps that can pass and still be considered
	 * a repeat tap. For example:
	 *
	 * -> Single Key Tap (the first tap)
	 * --
	 * -- ... Some time passes
	 * --
	 * -> Another single key tap (the second tap)
	 *
	 * -- if (time passed <= RepeatDelay) then trigger repeat tap
	 */
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category="Trigger Settings", meta = (ClampMin = "0"))
	double RepeatDelay = 0.5;

	/**
	 * The number of taps in a row it takes for this to be considered triggered.
	 *
	 * Note: To create a "Double tap" trigger, this value should be 2.
	 */
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category="Trigger Settings", meta = (ClampMin = "1"))
	int32 NumberOfTapsWhichTriggerRepeat = 2;

	// Release within this time-frame to trigger a tap
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category = "Trigger Settings", meta = (ClampMin = "0"))
	float TapReleaseTimeThreshold = 0.2f;

protected:
	/**
	 * This is the time at which we need to be <= on the next detection of a
	 * tap in order to be considered a double tap.
	 *
	 * This is set to (FPlatform::Seconds() + RepeatDelay) whenever we detect a tap
	 */
	UPROPERTY()
	double RepeatTime = 0.0;

	/**
	 * Tracks the number of individual taps that have occured since the last time we met
	 */
	UPROPERTY()
	int32 NumberOfTapsSinceLastTrigger = 0;

	/**
	 * True once we've detected a single tap and are still within the repeat window, waiting for
	 * the next tap. Keeps the trigger state Ongoing on idle frames between taps (the first tap
	 * doesn't increment NumberOfTapsSinceLastTrigger, so we need a separate flag to represent
	 * "a repeat sequence is in progress"). Cleared when the sequence triggers or is cancelled.
	 */
	UPROPERTY()
	bool bAwaitingRepeatTap = false;
};

/** UInputTriggerPulse
	Trigger that fires at an Interval, in seconds, while input is actuated. 
	Note:	Completed only fires when the repeat limit is reached or when input is released immediately after being triggered.
			Otherwise, Canceled is fired when input is released.
	*/
UCLASS(NotBlueprintable, MinimalAPI, meta = (DisplayName = "Pulse"))
class UInputTriggerPulse final : public UInputTriggerTimedBase
{
	GENERATED_BODY()

private:

	int32 TriggerCount = 0;

protected:

	virtual ETriggerState UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime) override;
	UE_API virtual void ReceiveTriggerReinstanced_Implementation(const UInputTrigger* OldTrigger) override;

public:
	// Whether to trigger when the input first exceeds the actuation threshold or wait for the first interval?
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category = "Trigger Settings")
	bool bTriggerOnStart = true;

	// How long between each trigger fire while input is held, in seconds?
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category = "Trigger Settings", meta = (ClampMin = "0"))
	float Interval = 1.0f;

	// How many times can the trigger fire while input is held? (0 = no limit)
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category = "Trigger Settings", meta = (ClampMin = "0"))
	int32 TriggerLimit = 0;

	virtual FString GetDebugState() const override { return HeldDuration ? FString::Printf(TEXT("Triggers:%d/%d, Interval:%.2f/%.2f"), TriggerCount, TriggerLimit, (HeldDuration/(Interval*(TriggerCount+1))), Interval) : FString(); }
};


// Chorded actions

/**
 * UInputTriggerChordAction
 * Applies a chord action that must be triggering for this trigger's action to trigger
*/
UCLASS(MinimalAPI, NotBlueprintable, meta = (DisplayName = "Chorded Action", NotInputConfigurable = "true"))
class UInputTriggerChordAction : public UInputTrigger
{
	GENERATED_BODY()
public:
#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif

protected:
	// Implicit, so action cannot fire unless this is firing.
	virtual ETriggerType GetTriggerType_Implementation() const override { return ETriggerType::Implicit; }

	UE_API virtual ETriggerState UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime) override;
	virtual bool IsBlocking(const ETriggerState State) const override { return State != ETriggerState::Triggered; }

public:

	// The action that must be triggering for this trigger's action to trigger
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trigger Settings", meta = (DisplayThumbnail = "false"))
	TObjectPtr<const UInputAction> ChordAction = nullptr;
};

/** UInputTriggerChordBlocker
	Automatically instantiated  to block mappings that are masked by a UInputTriggerChordAction chord from firing whilst the chording key is active.
	NOTE: Do not attempt to add these manually.
*/
UCLASS(NotBlueprintable, MinimalAPI, HideDropdown)
class UInputTriggerChordBlocker final : public UInputTriggerChordAction
{
	GENERATED_BODY()
protected:
	virtual ETriggerType GetTriggerType_Implementation() const override { return ETriggerType::Blocker; }
	virtual bool IsBlocking(const ETriggerState State) const override { return State == ETriggerState::Triggered; }
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.8, "The Input combo trigger and related types are deprecated") FInputComboStepData
{
	GENERATED_BODY()

	// The action that must be completed (according to Combo Step Completion States) to progress the combo
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Trigger Settings", meta = (DisplayThumbnail = "false"))
	TObjectPtr<const UInputAction> ComboStepAction = nullptr;

	// Trigger events that will complete this step - what events from this action should progress the combo
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trigger Settings", config, meta = (Bitmask, BitmaskEnum="/Script/EnhancedInput.ETriggerEvent"))
	uint8 ComboStepCompletionStates = static_cast<uint8>(ETriggerEvent::Triggered);
	
	/**
	 * Time to press the key before combo is cancelled - starts once the previous step in the combo is completed
	 * Note: This can be safely ignored for the first action in the combo
	 */
	UPROPERTY(EditAnywhere, Config, BlueprintReadWrite, Category = "Trigger Settings")
	float TimeToPressKey = 0.5f;
};

USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.8, "The Input combo trigger and related types are deprecated") FInputCancelAction
{
	GENERATED_BODY()

	// The action that must be completed (according to Cancellation States) to cancel the combo
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Trigger Settings", meta = (DisplayThumbnail = "false"))
	TObjectPtr<const UInputAction> CancelAction = nullptr;

	// Trigger events for this action that will cancel the combo - what events from this action should cancel the combo
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trigger Settings", config, meta = (Bitmask, BitmaskEnum="/Script/EnhancedInput.ETriggerEvent"))
	uint8 CancellationStates = static_cast<uint8>(ETriggerEvent::Triggered);
};

/**
 * UInputTriggerCombo
 * All actions in the combo array must be completed (based on combo completion event specified - triggered, completed, etc.) to trigger the action this trigger is on.
 * Actions must also be completed in the order specified by the combo actions array (starting at index 0).
 * Note: This will only trigger for one frame before resetting the combo trigger's progress 
 */
UCLASS(MinimalAPI, NotBlueprintable, meta = (DisplayName = "Combo (Beta)", NotInputConfigurable = "true"))
class UE_DEPRECATED(5.8, "UInputTriggerCombo has been deprecated.") UInputTriggerCombo final : public UInputTrigger
{
	GENERATED_BODY()

	UE_API UInputTriggerCombo();

protected:
	// Implicit, so action cannot fire unless this is firing.
	virtual ETriggerType GetTriggerType_Implementation() const override { return ETriggerType::Implicit; }

	UE_API virtual ETriggerState UpdateState_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue ModifiedValue, float DeltaTime) override;

	// Keeps track of what action we're currently at in the combo
	UPROPERTY(BlueprintReadOnly, Category = "Trigger Settings")
	int32 CurrentComboStepIndex = 0;
	
	// Time elapsed between last combo InputAction trigger and current time
	UPROPERTY(BlueprintReadOnly, Category = "Trigger Settings")
	float CurrentTimeBetweenComboSteps = 0.0f;

public:

#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	
	/**
	 * List of input actions that need to be completed (according to Combo Step Completion States) to trigger this action.
	 * Input actions must be triggered in order (starting at index 0) to count towards the triggering of the combo.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trigger Settings", meta = (DisplayThumbnail = "false", TitleProperty = "ComboStepAction"))
	TArray<FInputComboStepData> ComboActions;

	// Actions that will cancel the combo if they are completed (according to Cancellation States)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trigger Settings", meta = (DisplayThumbnail = "false", DisplayName = "Cancel Actions"))
	TArray<FInputCancelAction> InputCancelActions;

PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** Determines what kind of trigger events can happen from the behavior of this trigger. */
	virtual ETriggerEventsSupported GetSupportedTriggerEvents() const override { return ETriggerEventsSupported::All; }
};

#undef UE_API
