// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "PlayerMappableInputConfig.generated.h"

class UInputAction;
struct FEnhancedActionKeyMapping;

class UInputMappingContext;

/**
 * UPlayerMappableInputConfig
 * 
 * This represents one set of Player Mappable controller/keymappings. You can use this input config to create
 * the default mappings for your player to start with in your game. It provides an easy way to get only the player
 * mappable key actions, and it can be used to add multiple UInputMappingContext's at once to the player.
 *
 * Populate this data asset with Input Mapping Contexts that have player mappable actions in them. 
 */
UCLASS(BlueprintType, Meta = (DisplayName = "Player Mappable Input Config", ShortTooltip = "Data asset used to define a set of player mappable controller/keyboard mappings."))
class UE_DEPRECATED(5.3, "UPlayerMappableInputConfig has been deprecated. Please use UEnhancedInputUserSettings instead.") ENHANCEDINPUT_API UPlayerMappableInputConfig : public UPrimaryDataAsset
{
	GENERATED_BODY()

	UPlayerMappableInputConfig(const FObjectInitializer& ObjectInitializer);

public:

#if WITH_EDITOR
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

	/** Iterate all the player mappable keys in this config in the default input mapping contexts */
	void ForEachDefaultPlayerMappableKey(TFunctionRef<void(const FEnhancedActionKeyMapping&)> Operation) const;

	/** Get all the player mappable keys in this config. */
	UFUNCTION(BlueprintCallable, Category = "Input|PlayerMappable")
	TArray<FEnhancedActionKeyMapping> GetPlayerMappableKeys() const;

	/** Returns the action key mapping for the mapping that matches the given name */
	UFUNCTION(BlueprintCallable, Category = "Input|PlayerMappable")
	FEnhancedActionKeyMapping GetMappingByName(const FName MappingName) const;

	/** Returns all the keys mapped to a specific Input Action in this mapping config. */
	UFUNCTION(BlueprintCallable, Category = "Input|PlayerMappable")
	TArray<FEnhancedActionKeyMapping> GetKeysBoundToAction(const UInputAction* InAction) const;

	/** Resets this mappable config to use the keys  */
	UFUNCTION(BlueprintCallable, Category = "Input|PlayerMappable")
	void ResetToDefault();
	
protected:
	
	/** The unique name of this config that can be used when saving it */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input|PlayerMappable")
	FName ConfigName;

	/** The display name that can be used  */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input|PlayerMappable")
	FText ConfigDisplayName;

	/** A flag that can be used to mark this Input Config as deprecated to your player/designers. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input|PlayerMappable")
	bool bIsDeprecated = false;

	/** Metadata that can used to store any other related items to your key mapping such as icons, ability assets, etc. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input|PlayerMappable")
	TObjectPtr<UObject> Metadata = nullptr;

	/** Mapping contexts that make up this Input Config with their associated priority. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input|PlayerMappable")
	TMap<TObjectPtr<UInputMappingContext>, int32> Contexts;

public:

	/** Return all the Input Mapping contexts that  */
	UFUNCTION(BlueprintCallable, Category = "Input|PlayerMappable")
	const TMap<TObjectPtr<UInputMappingContext>, int32>& GetMappingContexts() const { return Contexts; }

	UFUNCTION(BlueprintCallable, Category = "Input|PlayerMappable")
	const FName GetConfigName() const { return ConfigName; }

	UFUNCTION(BlueprintCallable, Category = "Input|PlayerMappable")
	const FText& GetDisplayName() const { return ConfigDisplayName; }

	UFUNCTION(BlueprintCallable, Category = "Input|PlayerMappable")
	const bool IsDeprecated() const { return bIsDeprecated; }

	/** Get all the player mappable keys in this config. */
	UFUNCTION(BlueprintCallable, Category = "Input|PlayerMappable")
	UObject* GetMetadata() const { return Metadata; }
};
