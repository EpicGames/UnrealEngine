// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"

#include "PlayerMappableKeySettings.generated.h"

#define UE_API ENHANCEDINPUT_API

struct FEnhancedActionKeyMapping;

/**
* Hold setting information of an Action Input or a Action Key Mapping for setting screen and save purposes.
*/
UCLASS(MinimalAPI, DefaultToInstanced, EditInlineNew, DisplayName="Player Mappable Key Settings")
class UPlayerMappableKeySettings : public UObject
{
	GENERATED_BODY()

public:

	virtual FName MakeMappingName(const FEnhancedActionKeyMapping* OwningActionKeyMapping) const { return GetMappingName(); }
	virtual FName GetMappingName() const { return Name; }

#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
	UE_API virtual void PostLoad() override;

	/**
	 * Get the known mapping names that are current in use. This is a helper function if you want to use a "GetOptions" metadata on a UPROPERTY.
	 * For example, the following will display a little drop down menu to select from all current mapping names:
	 *
	 *  UPROPERTY(EditAnywhere, BlueprintReadOnly, meta=(GetOptions="EnhancedInput.PlayerMappableKeySettings.GetKnownMappingNames"))
	 *  FName MappingName;
	 */
	UFUNCTION()
	static UE_API const TArray<FName>& GetKnownMappingNames();	
#endif // WITH_EDITOR

	/** Metadata that can used to store any other related items to this key mapping such as icons, ability assets, etc. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	TObjectPtr<UObject> Metadata = nullptr;

	/** A unique name for this player mapping to be saved with. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	FName Name;

	/** The localized display name of this key mapping. Use this when displaying the mappings to a user. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	FText DisplayName;

	/** The category that this player mapping is in */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	FText DisplayCategory = FText::GetEmpty();

	/** 
	* If this key mapping should only be added when a specific key profile is equipped, then set those here.
	* 
	* If this is empty, then the key mapping will not be filtered out based on the current profile.
	*/
	UE_DEPRECATED(5.6, "Use SupportedKeyProfileIds instead")
	UPROPERTY()
	FGameplayTagContainer SupportedKeyProfiles;

	/** 
	* If this key mapping should only be added when a specific key profile is equipped, then set those here.
	* 
	* If this is empty, then the key mapping will not be filtered out based on the current profile.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	TArray<FString> SupportedKeyProfileIds;
};

#undef UE_API
