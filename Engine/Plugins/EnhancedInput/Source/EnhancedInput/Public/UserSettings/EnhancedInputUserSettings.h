// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/InputSettings.h"
#include "GameFramework/SaveGame.h"
#include "GameplayTagContainer.h"
#include "InputCoreTypes.h"
#include "Misc/EnumRange.h"
#include "UObject/Object.h"

#include "EnhancedInputUserSettings.generated.h"

#define UE_API ENHANCEDINPUT_API

class UCanvas;
class UEnhancedPlayerInput;
class UInputAction;
class UInputMappingContext;
class ULocalPlayer;
struct FEnhancedActionKeyMapping;

/**
 * The "Slot" that a player mappable key is in.
 * Used by UI to allow for multiple keys to be bound by the player for a single player mapping
 * 
 * | <Mapping Name>  | Slot 1 | Slot 2 | Slot 3 | Slot.... N |
 */
UENUM(BlueprintType)
enum class EPlayerMappableKeySlot : uint8 
{
	// The first key slot
	First = 0,

	// The second mappable key slot. This is the default max in the project settings
	Second,
	
	Third,
	Fourth,
	Fifth,
	Sixth,
	Seventh,
	
	// A key that isn't in any slot
	Unspecified,
	Max
};

ENUM_RANGE_BY_FIRST_AND_LAST(EPlayerMappableKeySlot, EPlayerMappableKeySlot::First, EPlayerMappableKeySlot::Seventh);

/** Arguments that can be used when mapping a player key */
USTRUCT(BlueprintType)
struct FMapPlayerKeyArgs final
{
	GENERATED_BODY()

	UE_API PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMapPlayerKeyArgs();	
	~FMapPlayerKeyArgs() = default;
	FMapPlayerKeyArgs(const FMapPlayerKeyArgs&) = default;
	FMapPlayerKeyArgs(FMapPlayerKeyArgs&&) = default;
	FMapPlayerKeyArgs& operator=(const FMapPlayerKeyArgs&) = default;
	FMapPlayerKeyArgs& operator=(FMapPlayerKeyArgs&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * The name of the mapping for this key. This is either the default mapping name from an Input Action asset, or one
	 * that is overridden in the Input Mapping Context.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Enhanced Input|User Settings")
	FName MappingName;

	/** What slot this key mapping is for */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Enhanced Input|User Settings")
	EPlayerMappableKeySlot Slot;

	/** The new Key that this should be mapped to */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Enhanced Input|User Settings")
	FKey NewKey;
	
	/** An OPTIONAL specifier about what kind of hardware this mapping is for. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Enhanced Input|User Settings", meta = (AllowPrivateAccess = true, GetOptions = "Engine.InputPlatformSettings.GetAllHardwareDeviceNames"))
	FName HardwareDeviceId;

	/** The Key Mapping Profile identifier that this mapping should be set on. If this is empty, then the currently equipped profile will be used. */
	UE_DEPRECATED(5.6, "Use ProfileIdString instead.")	
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use ProfileIdString instead."))
	FGameplayTag ProfileId;

	/**
	 * The Key Mapping Profile identifier that this mapping should be set on.
	 *
	 * If this is empty, then the currently equipped profile will be used.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Enhanced Input|User Settings")
	FString ProfileIdString;

	/** If there is not a player mapping already with the same Slot and Hardware Device ID, then create a new mapping for this slot. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Enhanced Input|User Settings")
	uint8 bCreateMatchingSlotIfNeeded : 1;

	/** Defers setting changed delegates until the next frame if set to true. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Enhanced Input|User Settings")
	uint8 bDeferOnSettingsChangedBroadcast : 1;
};

/** Represents a single key mapping that is set by the player */
USTRUCT(BlueprintType)
struct FPlayerKeyMapping final
{
	GENERATED_BODY()
	friend class UEnhancedInputUserSettings;
	
public:
	
	UE_API FPlayerKeyMapping();
	
	UE_API FPlayerKeyMapping(
		const FEnhancedActionKeyMapping& OriginalMapping,
		EPlayerMappableKeySlot InSlot = EPlayerMappableKeySlot::Unspecified,
		const FHardwareDeviceIdentifier& InHardwareDevice = FHardwareDeviceIdentifier::Invalid);

	/** A static invalid player key mapping to be used for easy comparisons in blueprint */
	static UE_API FPlayerKeyMapping InvalidMapping;

	/** Returns true if this mapping has been customized by the player, and false if it has not been. */
	UE_API bool IsCustomized() const;

	/** Returns true if this player mapping is valid */
	UE_API bool IsValid() const;

	/**
	 * Returns the key that the player has mapped. If the player has not mapped one yet, then this returns the
	 * default key mapping from the input mapping context.
	 */
	UE_API const FKey& GetCurrentKey() const;

	/** Returns the default key that this mapping is to */
	UE_API const FKey& GetDefaultKey() const;

	/** Print out some debug information about this player key mappings */
	UE_API FString ToString() const;

	/**
	 * The unique FName associated with this mapping. This is defined by this mappings owning Input Action
	 * or the individual Enhanced Action Key Mapping if it is overridden
	 */
	UE_API const FName GetMappingName() const;

	/** The localized display name to use for this mapping */
	UE_API const FText& GetDisplayName() const;

	/** The localized display name for the category to use for this mapping */
	UE_API const FText& GetDisplayCategory() const;

	/** Returns what player mappable slot this mapping is in */
	UE_API EPlayerMappableKeySlot GetSlot() const;

	/** Returns the optional hardware device ID that this mapping is specific to */
	UE_API const FHardwareDeviceIdentifier& GetHardwareDeviceId() const;

	/** Gets the primary device type associated with this key mapping. Taken from the HardwareDeviceId. */
	UE_API EHardwareDevicePrimaryType GetPrimaryDeviceType() const;

	/** Gets the supported hardware device feature flags associated with this key mapping. Taken from the HardwareDeviceId. */
	UE_API EHardwareDeviceSupportedFeatures::Type GetHardwareDeviceSupportedFeatures() const;

	/** Returns the input action asset associated with this player key mapping */
	UE_API const UInputAction* GetAssociatedInputAction() const;

	/** Resets the current mapping to the default one */
	UE_API void ResetToDefault();

	/** Sets the value of the current key to the one given */
	UE_API void SetCurrentKey(const FKey& NewKey);

	/** Sets the value of the hardware device ID that this custom key mapping is associated with. This can be used to filter key mappings per input device */
	UE_API void SetHardwareDeviceId(const FHardwareDeviceIdentifier& InDeviceId);

	/**
	 * Sets the Default Key on this mapping to that of the given ActionKeyMapping.
	 * Checks for customization of this key. This is for use during player key registration.
	 */
	UE_API void UpdateDefaultKeyFromActionKeyMapping(const FEnhancedActionKeyMapping& OriginalMapping);

	/**
	 * Updates the metadata properties on this player mapped key based on the given
	 * enhanced action mapping. This will populate the fields on this struct that are not editable
	 * by the player such as the localized display name and default key.  
	 */
	UE_API void UpdateMetadataFromActionKeyMapping(const FEnhancedActionKeyMapping& OriginalMapping);
	
	ENHANCEDINPUT_API friend uint32 GetTypeHash(const FPlayerKeyMapping& InMapping);
	UE_API bool operator==(const FPlayerKeyMapping& Other) const;
	UE_API bool operator!=(const FPlayerKeyMapping& Other) const;
	UE_API FPlayerKeyMapping& operator=(FPlayerKeyMapping& Other);
	UE_API FPlayerKeyMapping& operator=(const FPlayerKeyMapping& Other);
	UE_API FPlayerKeyMapping(const FPlayerKeyMapping& Other);

	/** Returns true if this mapping has been modified since it was registered from an IMC */
	UE_API const bool IsDirty() const;
	
protected:
	
	/** The name of the mapping for this key */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Enhanced Input|User Settings")
	FName MappingName;
	
	/** Localized display name of this mapping */
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Enhanced Input|User Settings")
	FText DisplayName;

	/** Localized display category of this mapping */
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Enhanced Input|User Settings")
	FText DisplayCategory;

	/** What slot this key is mapped to */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Enhanced Input|User Settings")
	EPlayerMappableKeySlot Slot;
	
	/** True if this key mapping is dirty (i.e. has been changed by the player) */
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Enhanced Input|User Settings")
	uint8 bIsDirty : 1;
	
	/** The default key that this mapping was set to in its input mapping context */
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Enhanced Input|User Settings", meta = (AllowPrivateAccess = "True"))
	FKey DefaultKey;
	
	/** The key that the player has mapped to */
	UPROPERTY(VisibleAnywhere,  BlueprintReadOnly, Category="Enhanced Input|User Settings", meta = (AllowPrivateAccess = "True"))
	FKey CurrentKey;

	/** An optional Hardware Device specifier for this mapping */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Enhanced Input|User Settings")
	FHardwareDeviceIdentifier HardwareDeviceId;

	UE_DEPRECATED(5.6, "This property is deprecated, use AssociatedInputActionSoft instead.")
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Enhanced Input|User Settings", meta=(DeprecatedProperty, DeprecationMessage = "This property is deprecated, use AssociatedInputActionSoft instead."))
	TObjectPtr<const UInputAction> AssociatedInputAction; 

	/** The input action associated with this player key mapping */
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Enhanced Input|User Settings")
	TSoftObjectPtr<const UInputAction> AssociatedInputActionSoft;
};

/**
 * Stores all mappings bound to a single mapping name.
 *
 * Since a single mapping can have multiple bindings to it and this system should be Blueprint friendly,
 * this needs to be a struct (blueprint don't support nested containers).
 */
USTRUCT(BlueprintType)
struct FKeyMappingRow final
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Enhanced Input|User Settings")
	TSet<FPlayerKeyMapping> Mappings;

	/** Returns true if this row has any mappings in it */
	UE_API bool HasAnyMappings() const;
};

/** 
* Options when querying what keys are mapped to a specific action on the player mappable
* key profile. 
*/
USTRUCT(BlueprintType)
struct FPlayerMappableKeyQueryOptions final
{
	GENERATED_BODY()

	UE_API FPlayerMappableKeyQueryOptions();

	/** The mapping name to search for */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Query Options")
	FName MappingName;
	
	/** If specified, then this key will be used to match against when checking if the key types and axis are the same. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Query Options")
	FKey KeyToMatch;
	
	/** The key slot that will be required to match if set. By default this is EPlayerMappableKeySlot::Unspecified, which will not filter by the slot at all. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Query Options")
	EPlayerMappableKeySlot SlotToMatch;

	/** If true, then only keys that have the same value for IsGamepadKey, IsTouch, and IsGesture will be included in the results of this query */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Query Options")
	uint8 bMatchBasicKeyTypes : 1;

	/** If true, then only keys that have the same value of IsAxis1D, IsAxis2D, and IsAxis3D will be included in the results of this query */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Query Options")
	uint8 bMatchKeyAxisType : 1;
	
	/** If set, then only player mappings whose hardware device identifier that has the same primary input device type will be included in the results of this query */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Query Options")
	EHardwareDevicePrimaryType RequiredDeviceType;

	/** If set, then only player mappings whose Hardware Device Identifier that has the same flags as this will be included in the results */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Query Options", meta=(Bitmask, BitmaskEnum="/Script/Engine.EHardwareDeviceSupportedFeatures"))
	int32 RequiredDeviceFlags;
};

/** Represents one "Profile" that a user can have for their player mappable keys */
UCLASS(MinimalAPI, BlueprintType)
class UEnhancedPlayerMappableKeyProfile : public UObject
{
	GENERATED_BODY()

	friend class UEnhancedInputUserSettings;

public:

	//~ Begin UObject Interface
    /**
     * Because the key mapping profile is serialized as a subobject of the UEnhancedInputUserSettings and requires
     * some custom serialization logic, you should not override the Serialize method on your custom key profile.
     * If you need to add custom serialization logic then you can create a struct UPROPERTY and override the struct's
     * serialization logic, which will prevent you from running into possible issues with subobjects.
     */
    UE_API virtual void Serialize(FArchive& Ar) override final;
    //~ End UObject Interface

	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API virtual void SetDisplayName(const FText& NewDisplayName);

	UE_DEPRECATED(5.6, "Use GetProfileIdString instead")
	UE_API const FGameplayTag& GetProfileIdentifer() const;

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Enhanced Input|User Settings")
	UE_API const FString& GetProfileIdString() const;

	/** Get the localized display name for this profile */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Enhanced Input|User Settings")
	UE_API const FText& GetProfileDisplayName() const;

	/**
	 * Get all known key mappings for this profile.
	 *
	 * This returns a map of "Mapping Name" -> Player Mappings to that name
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Enhanced Input|User Settings")
	UE_API const TMap<FName, FKeyMappingRow>& GetPlayerMappingRows() const;

	/** Resets every player key mapping to this mapping back to it's default value */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API void ResetMappingToDefault(UPARAM(Meta=(GetOptions="EnhancedInput.PlayerMappableKeySettings.GetKnownMappingNames")) const FName InMappingName);
	
	/** Get all the key mappings associated with the given mapping name on this profile */
	UE_API FKeyMappingRow* FindKeyMappingRowMutable(const FName InMappingName);

	/** Get all the key mappings associated with the given mapping name on this profile */
	UE_API const FKeyMappingRow* FindKeyMappingRow(const FName InMappingName) const;

	/** A helper function to print out all the current profile settings to the log. */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API virtual void DumpProfileToLog() const;

	/** Returns a string that can be used to debug the current key mappings.  */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API virtual FString ToString() const;
	
	/**
	 * Returns all FKey's bound to the given mapping Name on this profile.
	 *
	 * Returns the number of keys for the given mapping name
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta = (ReturnDisplayName = "Number of keys"))
	UE_API virtual int32 GetMappedKeysInRow(UPARAM(Meta=(GetOptions="EnhancedInput.PlayerMappableKeySettings.GetKnownMappingNames")) const FName MappingName, /*OUT*/ TArray<FKey>& OutKeys) const;

	/**
	 * Called during IEnhancedInputSubsystemInterface::RebuildControlMappings, this provides your user settings
	 * the opportunity to decide what key mappings get added for a given FEnhancedActionKeyMapping, and applied to the player.
	 *
	 * Override this to change what query options are being used during the controls being rebuilt of enhanced input.
	 *
	 * If the given Out Array is empty, then the Default Mapping will be applied to the player, but no player mapped keys will be.
	 */
	UE_API virtual int32 GetPlayerMappedKeysForRebuildControlMappings(const FEnhancedActionKeyMapping& DefaultMapping, /*OUT*/ TArray<FKey>& OutKeys) const;

	/** 
	* Populates the OutKeys array with any player mapped FKeys for the given default mapping. 
	* 
	* This is what IEnhancedInputSubsystemInterface::RebuildControlMappings calls to determine
	* what keys should actually be applied when building the control mappings.
	*
	* Returns the number of player mapped keys to the given Default Mapping
	*/
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta = (ReturnDisplayName = "Number of mappings"))
	UE_API virtual int32 QueryPlayerMappedKeys(const FPlayerMappableKeyQueryOptions& Options, /*OUT*/ TArray<FKey>& OutKeys) const;

	/**
	 * Populates the OutMappedMappingNames with every mapping on this profile that has a mapping to the given key.
	 *
	 * Returns the number of mappings to this key
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta = (ReturnDisplayName = "Number of mappings"))
	UE_API virtual int32 GetMappingNamesForKey(const FKey& InKey, /*OUT*/ TArray<FName>& OutMappingNames) const;

	/** Returns a pointer to the player key mapping that fits with the given arguments. Returns null if none exist. */
	UE_API virtual FPlayerKeyMapping* FindKeyMapping(const FMapPlayerKeyArgs& InArgs) const;
	
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta=(DisplayName="Find Key Mapping", AutoCreateRefTerm="OutKeyMapping"))
	UE_API void K2_FindKeyMapping(FPlayerKeyMapping& OutKeyMapping, const FMapPlayerKeyArgs& InArgs) const;

	/**
	 * Resets all the key mappings in this profile to their default value from their Input Mapping Context.
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API virtual void ResetToDefault();
	
	/**
	 * Returns true if the given player key mapping passes the query filter provided. 
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API virtual bool DoesMappingPassQueryOptions(const FPlayerKeyMapping& PlayerMapping, const FPlayerMappableKeyQueryOptions& Options) const;

protected:
	
	/**
	 * Equips the current key profile. This will always be called after the previous key profile's UnEquip function.
	*  Make any changes to the given Enhanced Player input object that may be necessary for
	 * your custom profile.
	 * 
	 * This function will only ever be called by UEnhancedInputUserSettings::SetKeyProfile.
	 */
	UE_API virtual void EquipProfile();

	/**
	 * UnEquips the current profile. Make any changes to the given Enhanced Player input object that may be necessary for
	 * your custom profile.
	 * 
	 * This function will only ever be called by UEnhancedInputUserSettings::SetKeyProfile
	 */
	UE_API virtual void UnEquipProfile();
	
	/** The ID of this profile. This can be used by each Key Mapping to filter down which profile is required for it be equipped. */
	UE_DEPRECATED(5.6, "Use ProfileIdentifierString instead")	
	UPROPERTY(SaveGame, meta=(DeprecatedProperty, DeprecationMessage = "Use ProfileIdentifierString"))
	FGameplayTag ProfileIdentifier;

	/** The ID of this profile. This can be used by each Key Mapping to filter down which profile is required for it be equipped. */
	UPROPERTY(BlueprintReadOnly, SaveGame, EditAnywhere, Category="Enhanced Input|User Settings")
	FString ProfileIdentifierString;

	/** The platform user id of the owning Local Player of this profile. */
	UPROPERTY(Transient, BlueprintReadOnly, VisibleAnywhere, Category="Enhanced Input|User Settings")
	FPlatformUserId OwningUserId;
	
	/** The localized display name of this profile */
	UPROPERTY(BlueprintReadWrite, SaveGame, EditAnywhere, Category="Enhanced Input|User Settings")
	FText DisplayName;
	
	/**
	 * A map of "Mapping Row Name" to all key mappings associated with it.
	 * Note: Dirty mappings will be serialized from UEnhancedInputUserSettings::Serialize
	 */
	UPROPERTY(BlueprintReadOnly, Transient, EditAnywhere, Category="Enhanced Input|User Settings")
	TMap<FName, FKeyMappingRow> PlayerMappedKeys;
};

/** Arguments that can be used when creating a new mapping profile */
USTRUCT(BlueprintType)
struct FPlayerMappableKeyProfileCreationArgs final
{
	GENERATED_BODY()
	
	UE_API PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPlayerMappableKeyProfileCreationArgs();
	~FPlayerMappableKeyProfileCreationArgs() = default;
	FPlayerMappableKeyProfileCreationArgs(const FPlayerMappableKeyProfileCreationArgs&) = default;
	FPlayerMappableKeyProfileCreationArgs(FPlayerMappableKeyProfileCreationArgs&&) = default;
	FPlayerMappableKeyProfileCreationArgs& operator=(const FPlayerMappableKeyProfileCreationArgs&) = default;
	FPlayerMappableKeyProfileCreationArgs& operator=(FPlayerMappableKeyProfileCreationArgs&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Enhanced Input|User Settings")
	TSubclassOf<UEnhancedPlayerMappableKeyProfile> ProfileType;
	
	/** The unique identifier that this profile should have */
	UE_DEPRECATED(5.6, "Use ProfileStringIdentifier instead.")
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use ProfileStringIdentifier instead."))
	FGameplayTag ProfileIdentifier;

	/** The unique identifier that this profile should have */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Enhanced Input|User Settings")
	FString ProfileStringIdentifier;

	/** The user ID of the ULocalPlayer that this profile is associated with */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Enhanced Input|User Settings")
	FPlatformUserId UserId;
	
	/** The display name of this profile */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Enhanced Input|User Settings")
	FText DisplayName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Enhanced Input|User Settings")
	uint8 bSetAsCurrentProfile : 1;
};

/**
 * The Enhanced Input User Settings class is a place where you can put all of your Input Related settings
 * that you want your user to be able to change. Things like their key mappings, aim sensitivity, accessibility
 * settings, etc. This also provides a Registration point for Input Mappings Contexts (IMC) from possibly unloaded
 * plugins (i.e. Game Feature Plugins). You can register your IMC from a Game Feature Action plugin here, and then
 * have access to all the key mappings available. This is very useful for building settings screens because you can
 * now access all the mappings in your game, even if the entire plugin isn't loaded yet. 
 *
 * The user settings are stored on each UEnhancedPlayerInput object, so each instance of the settings can represent
 * a single User or Local Player.
 *
 * To customize this for your game, you can create a subclass of it and change the "UserSettingsClass" in the
 * Enhanced Input Project Settings.
 */
UCLASS(MinimalAPI, config=GameUserSettings, DisplayName="Enhanced Input User Settings", Category="Enhanced Input|User Settings")
class UEnhancedInputUserSettings : public USaveGame
{
	GENERATED_BODY()

public:

	//~ Begin UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface
	
	/** Loads or creates new user settings for the owning local player of the given player input */
	static UE_API UEnhancedInputUserSettings* LoadOrCreateSettings(ULocalPlayer* LP);
	UE_API virtual void Initialize(ULocalPlayer* LP);

	/**
	 * Apply any custom input settings to your user. By default, this will just broadcast the OnSettingsApplied delegate
	 * which is a useful hook to maybe rebuild some UI or do other user facing updates.
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API virtual void ApplySettings();

	/**
	 * Synchronously save the settings to a hardcoded save game slot. This will work for simple games,
	 * but if you need to integrate it into an advanced save system you should Serialize this object out with the rest of your save data.
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API virtual void SaveSettings();

	/**
	 * Asynchronously save the settings to a hardcoded save game slot. This will work for simple games,
	 * but if you need to integrate it into an advanced save system you should Serialize this object out with the rest of your save data.
	 *
	 * OnAsyncSaveComplete will be called upon save completion.
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API virtual void AsyncSaveSettings();

private:
	
	/**
	 * Called when a save file is loaded from disk but has a different class then the one specified in the developer settings.
	 * 
	 * @param OldSettingsObject Pointer to the old user settings object
	 */
	UE_API void HandleSettingsClassChangedOnLoad(UEnhancedInputUserSettings* OldSettingsObject);
	
protected:
	
	/**
	 * Override this function to be notified of when the user settings class has changed.
	 * 
	 * @param OldSettingsObject Pointer to the old user settings object
	 */
	UE_API virtual void HandleSettingsClassChangedOnLoad_Impl(UEnhancedInputUserSettings* OldSettingsObject);

	/**
	* Returns the save file name to use when saving the settings file for the given local player
	*/
	static UE_API FString GetSaveFilename(const ULocalPlayer* LP);
	
	UE_API virtual void OnAsyncSaveComplete(const FString& SlotName, const int32 UserIndex, bool bSuccess);

	/**
	 * A virtual function that will be called whenever a player key mapping has been updated, before OnSettingsChanged is called.
	 * This provides subclasses the opportunity to modify this key mapping before any settings screens are updated
	 *
	 * This does nothing by default, but some subclasses may want to modify the mapping. 
	 * Perhaps to change to a specific hardware device ID or something like that
	 *
	 * @param ChangedMapping	The key mapping that has changed
	 * @param InArgs			The arguments that were used to change this mapping.
	 * @param bIsBeingUnmapped	True if this key mapping is being unmapped. False if the key mapping is just being changed
	 */
	UE_API virtual void OnKeyMappingUpdated(FPlayerKeyMapping* ChangedMapping, const FMapPlayerKeyArgs& InArgs, const bool bIsBeingUnmapped);


	/**
	 * A virtual function that will be called whenever a player key mapping is first registered. This will be called once
	 * by UEnhancedInputUserSettings::RegisterKeyMappingsToProfile for each mapping that is created from the Input Mapping Context.
	 *
	 * This is a good place to do any "Post Registration" setup you may want to for you key mappings such as changing the
	 * key that is mapped based on some other settings or data outside of this system (i.e. you are upgrading to this system
	 * and already had a key mapping system in place, you can set the key here!)
	 *
	 * This does nothing by default.
	 * @param RegisteredMapping		The newly registered player key mapping
	 * @param SourceMapping			The source mapping that the registered mapping was created from. 
	 */
	UE_DEPRECATED(5.7, "Use OnKeyMappingRegisteredToProfile instead.")
	UE_API virtual void OnKeyMappingRegistered(FPlayerKeyMapping& RegisteredMapping, const FEnhancedActionKeyMapping& SourceMapping) final;


	/**
	 * A virtual function that will be called whenever a player key mapping is first registered. This will be called once
	 * by UEnhancedInputUserSettings::RegisterKeyMappingsToProfile for each mapping that is created from the Input Mapping Context.
	 *
	 * This is a good place to do any "Post Registration" setup you may want to for you key mappings such as changing the
	 * key that is mapped based on some other settings or data outside of this system (i.e. you are upgrading to this system
	 * and already had a key mapping system in place, you can set the key here!)
	 *
	 * This does nothing by default.
	 * 
	 * @param Profile				The key profile which this mapping was registered to
	 * @param RegisteredMapping		The newly registered player key mapping
	 * @param SourceMapping			The source mapping that the registered mapping was created from.
	 */
	UE_API virtual void OnKeyMappingRegisteredToProfile(UEnhancedPlayerMappableKeyProfile* Profile, FPlayerKeyMapping& RegisteredMapping, const FEnhancedActionKeyMapping& SourceMapping);

public:
	
	UE_API ULocalPlayer* GetLocalPlayer() const;

protected:
	// Used to track when a settings change callback has been deferred till the next frame.
	FTimerHandle DeferredSettingsChangedTimerHandle;
public:
	
	/** Fired when the user settings have changed, such as their key mappings. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FEnhancedInputUserSettingsChanged, UEnhancedInputUserSettings*, Settings);
	UPROPERTY(BlueprintAssignable, Transient, Category = "Enhanced Input|User Settings")
	FEnhancedInputUserSettingsChanged OnSettingsChanged;
	
	/** Called after the settings have been applied from the ApplySettings call. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FEnhancedInputUserSettingsApplied);
	UPROPERTY(BlueprintAssignable, Transient, Category = "Enhanced Input|User Settings")
	FEnhancedInputUserSettingsApplied OnSettingsApplied;

	// Remappable keys API

	/**
	 * Sets the player mapped key on the current key profile.
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta = (AutoCreateRefTerm = "FailureReason"))
	UE_API virtual void MapPlayerKey(const FMapPlayerKeyArgs& InArgs, FGameplayTagContainer& FailureReason);

	/** 
	* Unmaps a single player mapping that matches the given Mapping name, slot, and hardware device.
	*/
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta = (AutoCreateRefTerm = "FailureReason"))
	UE_API virtual void UnMapPlayerKey(const FMapPlayerKeyArgs& InArgs, FGameplayTagContainer& FailureReason);

	/**
	* Resets each player mapped key to it's default value from the Input Mapping Context that it was registered from.
	* If a key did not come from an IMC (i.e. it was added additionally by the player) then it will be reset to EKeys::Invalid.
	* 
	* @param InArgs				Arguments that contain the mapping name and profile ID to find the mapping to reset.
	* @param FailureReason		Populated with failure reasons if the operation fails.
	*/
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta = (AutoCreateRefTerm = "FailureReason"))
	UE_API virtual void ResetAllPlayerKeysInRow(const FMapPlayerKeyArgs& InArgs, FGameplayTagContainer& FailureReason);

	/**
	 * Resets the given key profile to default key mappings
	 *
	 * @param ProfileId		The ID of the key profile to reset
	 * @param FailureReason	Populated with failure reasons if the operation fails.
	 */
	UE_DEPRECATED(5.6, "Use the version which takes an FString instead.")
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta = (AutoCreateRefTerm = "FailureReason"))
	UE_API virtual void ResetKeyProfileToDefault(const FGameplayTag& ProfileId, FGameplayTagContainer& FailureReason) final;

	/**
	 * Resets the given key profile to default key mappings
	 *
	 * @param ProfileId		The ID of the key profile to reset
	 * @param FailureReason	Populated with failure reasons if the operation fails.
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta = (AutoCreateRefTerm = "FailureReason"))
	UE_API virtual void ResetKeyProfileIdToDefault(const FString& ProfileId, FGameplayTagContainer& FailureReason);

	/** Returns a set of all player key mappings for the given mapping name. */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API virtual const TSet<FPlayerKeyMapping>& FindMappingsInRow(UPARAM(Meta=(GetOptions="EnhancedInput.PlayerMappableKeySettings.GetKnownMappingNames")) const FName MappingName) const;

	/** Returns the current player key mapping for the given row name in the given slot */
	UE_API virtual const FPlayerKeyMapping* FindCurrentMappingForSlot(const FName MappingName, const EPlayerMappableKeySlot InSlot) const;

	/** Returns the Input Action associated with the given player mapping name */
	UE_API const UInputAction* FindInputActionForMapping(const FName MappingName) const;
	
	// Modifying key profile

	/** Fired when you equip a different key profile  */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMappableKeyProfileChanged, const UEnhancedPlayerMappableKeyProfile*, NewProfile);
	UPROPERTY(BlueprintAssignable, Transient, Category = "Enhanced Input|User Settings")
	FMappableKeyProfileChanged OnKeyProfileChanged;
	
	/**
	 * Changes the currently active key profile to the one with the given name. Returns true if the operation was successful.
	 */
	UE_DEPRECATED(5.6, "Use the version that takes an FString instead.")
	UE_API virtual bool SetKeyProfile(const FGameplayTag& InProfileId) final;

	/**
	 * Changes the currently active key profile to the one with the given name. Returns true if the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta=(ReturnDisplayName = "Was Successful"))
	UE_API virtual bool SetActiveKeyProfile(const FString& InProfileId);

	/**
	 * Changes the currently active key profile the default profile. Returns true if the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta=(ReturnDisplayName = "Was Successful"))
	UE_API bool SetKeyProfileToDefault();
	
	/** Gets the currently selected key profile */
	UE_DEPRECATED(5.6, "Use GetActiveKeyProfileId instead ")
	UE_API virtual const FGameplayTag& GetCurrentKeyProfileIdentifier() const final;

	/** Gets the currently selected key profile */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Enhanced Input|User Settings")
	UE_API virtual const FString& GetActiveKeyProfileId() const;

	/** Get the current key profile that the user has set */
	UE_DEPRECATED(5.6, "Use GetActiveKeyProfile instead.")
	UE_API UEnhancedPlayerMappableKeyProfile* GetCurrentKeyProfile() const;

	/** Get the currently active key profile that the user has set */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Enhanced Input|User Settings")
	UE_API UEnhancedPlayerMappableKeyProfile* GetActiveKeyProfile() const;

	/** Get the current key profile that the user has set */
	template<class T> UE_DEPRECATED(5.6, "Use GetActiveKeyProfileAs instead.")
	inline T* GetCurrentKeyProfileAs() const
	{
		static_assert(TIsDerivedFrom<T, UEnhancedPlayerMappableKeyProfile>::IsDerived, "T must be a UEnhancedPlayerMappableKeyProfile-based type!");
		return Cast<T>(GetActiveKeyProfile());
	}

	/** Get the current key profile that the user has set, casted to type T */
	template<class T>
	inline T* GetActiveKeyProfileAs() const
	{
		static_assert(TIsDerivedFrom<T, UEnhancedPlayerMappableKeyProfile>::IsDerived, "T must be a UEnhancedPlayerMappableKeyProfile-based type!");
		return Cast<T>(GetActiveKeyProfile());
	}

	/** Returns all player saved key profiles */
	UE_DEPRECATED(5.6, "Use GetAllAvailableKeyProfiles instead.")
	UE_API const TMap<FGameplayTag, TObjectPtr<UEnhancedPlayerMappableKeyProfile>>& GetAllSavedKeyProfiles() const;

	/** Returns all player saved key profiles */
	UE_API const TMap<FString, TObjectPtr<UEnhancedPlayerMappableKeyProfile>>& GetAllAvailableKeyProfiles() const;

	/** Returns true if the given ProfileId is currently available in the user settings. */
	UE_API bool IsKeyProfileAvailable(const FString& ProfileId) const;

	/**
	* Returns a pointer to the default key profile created by Enhanced Input. 
	* This profile has an ID of "InputUserSettings.Profiles.Default"
	*/
	UE_API UEnhancedPlayerMappableKeyProfile* GetDefaultKeyProfile() const;

	/**
	 * Creates a new profile with this name and type.
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings", meta = (DeterminesOutputType = "InArgs.ProfileType"))
	UE_API virtual UEnhancedPlayerMappableKeyProfile* CreateNewKeyProfile(const FPlayerMappableKeyProfileCreationArgs& InArgs);

	/** Returns the key profile with the given name if one exists. Null if one doesn't exist */
	UE_DEPRECATED(5.6, "Use the version which takes an FString instead")
	UE_API virtual UEnhancedPlayerMappableKeyProfile* GetKeyProfileWithIdentifier(const FGameplayTag& ProfileId) const final;

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Enhanced Input|User Settings")
	UE_API virtual UEnhancedPlayerMappableKeyProfile* GetKeyProfileWithId(const FString& ProfileId) const;

	/** Returns the key profile with the given name if one exists. Null if one doesn't exist */
	template<class T>
	UE_DEPRECATED(5.6, "Use the version which takes an FString instead.")
	inline T* GetKeyProfileWithIdentifierAs(const FGameplayTag& ProfileId) const
	{
		static_assert(TIsDerivedFrom<T, UEnhancedPlayerMappableKeyProfile>::IsDerived, "T must be a UEnhancedPlayerMappableKeyProfile-based type!");
		return Cast<T>(GetKeyProfileWithId(ProfileId.ToString()));
	}

	/** Returns the key profile with the given name if one exists. Null if one doesn't exist */
	template<class T>
	inline T* GetKeyProfileWithIdAs(const FString& ProfileId) const
	{
		static_assert(TIsDerivedFrom<T, UEnhancedPlayerMappableKeyProfile>::IsDerived, "T must be a UEnhancedPlayerMappableKeyProfile-based type!");
		return Cast<T>(GetKeyProfileWithId(ProfileId));
	}

	// Registering input mapping contexts for access to them from your UI,
	// even if they are from a plugin

	/** Fired when a new input mapping context is registered. Useful if you need to update your UI */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMappingContextRegisteredWithSettings, const UInputMappingContext*, IMC);
	FMappingContextRegisteredWithSettings OnMappingContextRegistered;

	/**
	 * Registers this mapping context with the user settings. This will iterate all the key mappings
	 * in the context and create an initial Player Mappable Key for every mapping that is marked as mappable.
	 */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API virtual bool RegisterInputMappingContext(const UInputMappingContext* IMC);

	/** Registers multiple mapping contexts with the settings */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API bool RegisterInputMappingContexts(const TSet<UInputMappingContext*>& MappingContexts);

	/** Removes this mapping context from the registered mapping contexts */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API virtual bool UnregisterInputMappingContext(const UInputMappingContext* IMC);

	/** Removes multiple mapping contexts from the registered mapping contexts */
	UFUNCTION(BlueprintCallable, Category="Enhanced Input|User Settings")
	UE_API bool UnregisterInputMappingContexts(const TSet<UInputMappingContext*>& MappingContexts);

	/** Gets all the currently registered mapping contexts with the settings */
	UE_API const TSet<TObjectPtr<const UInputMappingContext>>& GetRegisteredInputMappingContexts() const;

	/** Returns true if this mapping context is currently registered with the settings */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Enhanced Input|User Settings")
	UE_API bool IsMappingContextRegistered(const UInputMappingContext* IMC) const;
	
	/** Draw debug information on the screen for these user settings. */
	UE_API void ShowDebugInfo(UCanvas* Canvas) const;

protected:

	/**
	* This will register the given IMC to the settings system. 
	* 
	* If an IMC specified mappings which should have different defaults based on a certain key profile ID,
	* then we will also create the profile here if needed.
	* 
	* This will do so regardless of if the IMC has been registered already. 
	*/
	bool RegisterInputMappingContextInternal(const UInputMappingContext* IMC);

	/**
	 * This will register individual key mappings from the given Input Mapping context on the profile id.
	 *
	 * If the mappings exist on the profile already, then they will be updated based on the latest data from the IMC
	 * otherwise, they will be added as new FPlayerKeyMapping.
	 *
	 * This will be called for each key profile when users initially call RegisterInputMappingContext, and on
	 * load during Serialize to correctly populate a loaded profile with all the known registered IMC's mappings.
	 */
	UE_API virtual bool RegisterKeyMappingsToProfile(UEnhancedPlayerMappableKeyProfile& Profile, const UInputMappingContext* IMC);

	UE_DEPRECATED(5.6, "Use DetermineHardwareDeviceForActionMapping which takes in a mapping context instead")
	virtual FHardwareDeviceIdentifier DetermineHardwareDeviceForActionMapping(const FEnhancedActionKeyMapping& ActionMapping) const final
	{
		return DetermineHardwareDeviceForActionMapping(ActionMapping, nullptr);
	}
	
	/**
	 * Determines the hardware device that an action key mapping is associated with. By default, the hardware will be FHardwareDeviceIdentifier::Invalid.
	 *
	 * Override this function if you wish to have Input Device Specific Player Key mappings. For example, specify hardware
	 * devices based on some metadata on the UPlayerMappableKeySettings of the action mapping, or infer a device based
	 * on the type of FKey it is.
	 */
	UE_API virtual FHardwareDeviceIdentifier DetermineHardwareDeviceForActionMapping(const FEnhancedActionKeyMapping& ActionMapping, const UInputMappingContext* IMC = nullptr) const;
	
	/** 
	* Provides a space for subclasses to add additional debug info if desired.
	* By default, this will draw info about all player mapped keys.
	*/
	UE_API virtual void ShowDebugInfoInternal(UCanvas* Canvas) const;

	/** The current key profile that is equipped by the user. */
	UE_DEPRECATED(5.6, "Use the CurrentProfileIdentifierString instead.")
	UPROPERTY(SaveGame, meta=(DeprecatedProperty))
	FGameplayTag CurrentProfileIdentifier;
	
	/**
	 * The current key profile that is equipped by the user.
	 */
	UPROPERTY(SaveGame)
	FString CurrentProfileIdentifierString;
	
	/**
	 * All of the known Key Profiles for this user, including the currently active profile.
	 */
	UE_DEPRECATED(5.6, "Use the CurrentProfileIdentifierString instead.")
	UPROPERTY(Transient)
	TMap<FGameplayTag, TObjectPtr<UEnhancedPlayerMappableKeyProfile>> SavedKeyProfiles;

	/**
	 * All of the known Key Profiles for this user, including the currently active profile.
	 */
	UPROPERTY(Transient)
	TMap<FString, TObjectPtr<UEnhancedPlayerMappableKeyProfile>> SavedKeyProfilesMap;
	
	/** The owning Local Player object of these settings */
	UPROPERTY(Transient)
	TWeakObjectPtr<ULocalPlayer> OwningLocalPlayer;
	
	/**
	 * Set of currently registered input mapping contexts that may not be currently
	 * active on the user, but you want to track for creating a menu for key mappings.
	 */
	UPROPERTY(Transient)
	TSet<TObjectPtr<const UInputMappingContext>> RegisteredMappingContexts;
};

#undef UE_API
