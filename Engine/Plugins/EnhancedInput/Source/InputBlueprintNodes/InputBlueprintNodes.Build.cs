// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InputBlueprintNodes : ModuleRules
	{
        public InputBlueprintNodes(ReadOnlyTargetRules Target) : base(Target)
        {
	        PublicDependencyModuleNames.AddRange(new string[] { "InputEditor", });
	        
            PrivateDependencyModuleNames.AddRange(
				new string[] {
					"BlueprintGraph",
                    "Core",
					"CoreUObject",
                    "Engine",
					"EnhancedInput",
					"GraphEditor",
                    "InputCore",
					"KismetCompiler",
					"PropertyEditor",
                    "Slate",
                    "SlateCore",
                    "UnrealEd",
                    "UMGEditor",
                    "UMG",
                    "DataValidation"
                }
            );
        }
    }
}