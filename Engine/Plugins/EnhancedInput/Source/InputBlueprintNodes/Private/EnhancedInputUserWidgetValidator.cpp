// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputUserWidgetValidator.h"

#include "Blueprint/UserWidget.h"
#include "EdGraph/EdGraph.h"
#include "K2Node_EnhancedInputAction.h"
#include "WidgetBlueprint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedInputUserWidgetValidator)

namespace UE::Input
{
	static bool bShouldValidateWidgetBlueprintSettings = true;
	static FAutoConsoleVariableRef CVarShouldValidateWidgetBlueprintSettings(
		TEXT("enhancedInput.bp.ShouldValidateWidgetBlueprintSettings"),
		bShouldValidateWidgetBlueprintSettings,
		TEXT("Should the Enhanced Input event node throw an error if a widget blueprint does not have bAutomaticallyRegisterInputOnConstruction set to true?"),
		ECVF_Default);

	/**
	 * Returns true if the given widget BP has any linked nodes of the enhanced input type
	 * in any of its graphs.
	 */
	static bool HasAnyActiveEnhancedInputNodes(const UWidgetBlueprint* WidgetBP)
	{
		check(WidgetBP);
		
		bool bHasAnyActiveEnhancedInputNodes = false;
		
		TArray<UEdGraph*> Graphs;
		WidgetBP->GetAllGraphs(Graphs);
		
		for (const UEdGraph* Graph : Graphs)
		{
			TArray<UK2Node_EnhancedInputAction*> EventNodes;
			Graph->GetNodesOfClass(EventNodes);

			for (UK2Node_EnhancedInputAction* EventNode : EventNodes)
			{
				if (EventNode && EventNode->HasAnyConnectedEventPins())
				{
					bHasAnyActiveEnhancedInputNodes = true;
					break;
				}
			}

			if (bHasAnyActiveEnhancedInputNodes)
			{
				break;
			}
		}

		return bHasAnyActiveEnhancedInputNodes;
	}
}

bool UEnhancedInputUserWidgetValidator::CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const
{
	return
		UE::Input::bShouldValidateWidgetBlueprintSettings &&
		InAsset &&
		InAsset->IsA<UWidgetBlueprint>();
}

EDataValidationResult UEnhancedInputUserWidgetValidator::ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext)
{
	return CombineDataValidationResults(
			Super::ValidateLoadedAsset_Implementation(InAssetData, InAsset, InContext),
			ValidateWidgetBlueprint(InAssetData, Cast<const UWidgetBlueprint>(InAsset), InContext));
}

EDataValidationResult UEnhancedInputUserWidgetValidator::ValidateWidgetBlueprint(const FAssetData& InAssetData, const UWidgetBlueprint* WidgetBP, FDataValidationContext& InContext)
{
	// We need a valid CDO here to check the value of the bAutomaticallyRegisterInputOnConstruction setting
	const UUserWidget* DefaultObj = WidgetBP->GeneratedClass ? Cast<const UUserWidget>(WidgetBP->GeneratedClass->GetDefaultObject(false)) : nullptr;
	if (!DefaultObj)
	{
		return EDataValidationResult::NotValidated;
	}

	// If there are no Enhanced Input nodes then there is no need to check further
	if (!UE::Input::HasAnyActiveEnhancedInputNodes(WidgetBP))
	{
		return EDataValidationResult::Valid;
	}

	// If the widget has EI nodes and the bAutomaticallyRegisterInputOnConstruction is true, then it is valid...
	if (DefaultObj->bAutomaticallyRegisterInputOnConstruction)
	{
		return EDataValidationResult::Valid;
	}

	// ... otherwise, this widget is invalid.
	// Widgets require the bAutomaticallyRegisterInputOnConstruction to be true in order to receive enhanced input events. 
	const FText CurrentError = NSLOCTEXT("EnhancedInput", "Input.Widget.Error", "'bAutomaticallyRegisterInputOnConstruction' failed to automatically update but must be true in order to use Enhanced Input in the widget");
	
	AssetMessage(InAssetData, EMessageSeverity::Error, CurrentError);
	return EDataValidationResult::Invalid;	
}
