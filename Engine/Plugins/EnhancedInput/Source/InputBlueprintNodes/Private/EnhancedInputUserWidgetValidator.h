// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorValidatorBase.h"

#include "EnhancedInputUserWidgetValidator.generated.h"

class IAssetRegistry;
class UWidgetBlueprint;

/**
 * Validates Widget Blueprints that have any Enhanced Input
 * nodes in them to ensure that they have the correct "bAutomaticallyRegisterInputOnConstruction"
 * setting value.
 *
 * Widgets require bAutomaticallyRegisterInputOnConstruction to be true in order to
 * receive callbacks from Enhanced Input.
 */
UCLASS(NotBlueprintType)
class UEnhancedInputUserWidgetValidator : public UEditorValidatorBase
{
	GENERATED_BODY()

public:
	
	UEnhancedInputUserWidgetValidator() = default;

protected:
	//~UEditorValidatorBase interface
	virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const override;
	virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) override;
	//~End of UEditorValidatorBase interface

private:
	EDataValidationResult ValidateWidgetBlueprint(const FAssetData& InAssetData, const UWidgetBlueprint* InAsset, FDataValidationContext& InContext);
};