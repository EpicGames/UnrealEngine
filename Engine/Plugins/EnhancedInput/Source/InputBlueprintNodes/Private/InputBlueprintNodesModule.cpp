// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputBlueprintNodesModule.h"
#include "K2Node_EnhancedInputAction.h"
#include "K2Node_GetInputActionValue.h"
#include "AssetBlueprintGraphActions.h"
#include "InputAction.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "EdGraphSchema_K2_Actions.h"
#include "BlueprintEditorModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "BlueprintNodeTemplateCache.h"
#include "TickableEditorObject.h"
#include "UObject/UObjectIterator.h"

#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "InputBlueprintNodes"

struct FInputActionGraphActions : public FAssetBlueprintGraphActions
{
	virtual FText GetGraphHoverMessage(const FAssetData& AssetData, const UEdGraph* HoverGraph) const override;
	virtual bool TryCreatingAssetNode(const FAssetData& AssetData, UEdGraph* ParentGraph, const FVector2D Location, EK2NewNodeFlags Options) const override;
};

FText FInputActionGraphActions::GetGraphHoverMessage(const FAssetData& AssetData, const UEdGraph* HoverGraph) const
{
	return FText::Format(LOCTEXT("InputActionHoverMessage", "{0}"), FText::FromName(AssetData.AssetName));
}

bool FInputActionGraphActions::TryCreatingAssetNode(const FAssetData& AssetData, UEdGraph* ParentGraph, const FVector2D Location, EK2NewNodeFlags Options) const
{
	if (AssetData.IsValid())
	{
		if (const UInputAction* Action = Cast<const UInputAction>(AssetData.GetAsset()))
		{
			for (const TObjectPtr<UEdGraphNode>& Node : ParentGraph->Nodes)
			{
				if(const UK2Node_EnhancedInputAction* InputActionNode = Cast<UK2Node_EnhancedInputAction>(Node))
				{
					if (InputActionNode->InputAction.GetFName() == AssetData.AssetName)
					{
						if (const TSharedPtr<IBlueprintEditor> BlueprintEditor = FKismetEditorUtilities::GetIBlueprintEditorForObject(ParentGraph, false))
						{
							BlueprintEditor.Get()->JumpToPin(InputActionNode->GetPinAt(0));
						}
						
						return false;
					}
				}
			}

			UK2Node_EnhancedInputAction* NewNode = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_EnhancedInputAction>(
				ParentGraph,
				Location,
				Options,
				[Action](UK2Node_EnhancedInputAction* NewInstance)
				{
					NewInstance->InputAction = Action;
				}

			);
			return true;
		}
	}
	return false;
}



class FInputBlueprintNodesModule : public IModuleInterface, public FTickableEditorObject
{
	// IModuleInterface interface
	virtual void StartupModule() override
	{
		// Register graph actions:
		FBlueprintGraphModule& GraphModule = FModuleManager::LoadModuleChecked<FBlueprintGraphModule>("BlueprintGraph");
		{
			GraphModule.RegisterGraphAction(UInputAction::StaticClass(), MakeUnique<FInputActionGraphActions>());
		}
	}
	// End IModuleInterface interface

	// FTickableEditorObject interface
	virtual void Tick(float DeltaTime) override
	{
		// Update any blueprints that are referencing an input action with a modified value type
		if (UInputAction::ActionsWithModifiedValueTypes.Num() || UInputAction::ActionsWithModifiedTriggers.Num())
		{
			TSet<UBlueprint*> BPsModifiedFromValueTypeChange;
			TSet<UBlueprint*> BPsModifiedFromTriggerChange;
			
			for (TObjectIterator<UK2Node_EnhancedInputAction> NodeIt; NodeIt; ++NodeIt)
			{
				if (NodeIt->HasValidBlueprint() && NodeIt->GetGraph() && !FBlueprintNodeTemplateCache::IsTemplateOuter(NodeIt->GetGraph()))
				{
					if (UInputAction::ActionsWithModifiedValueTypes.Contains(NodeIt->InputAction))
					{
						NodeIt->ReconstructNode();
						BPsModifiedFromValueTypeChange.Emplace(NodeIt->GetBlueprint());
					}
					if (UInputAction::ActionsWithModifiedTriggers.Contains(NodeIt->InputAction))
					{
						NodeIt->ReconstructNode();
						BPsModifiedFromTriggerChange.Emplace(NodeIt->GetBlueprint());
					}
				}
			}
			for (TObjectIterator<UK2Node_GetInputActionValue> NodeIt; NodeIt; ++NodeIt)
			{
				if (NodeIt->HasValidBlueprint() && NodeIt->GetGraph() && !FBlueprintNodeTemplateCache::IsTemplateOuter(NodeIt->GetGraph()))
				{
					if (UInputAction::ActionsWithModifiedValueTypes.Contains(NodeIt->InputAction))
					{
						NodeIt->ReconstructNode();
						BPsModifiedFromValueTypeChange.Emplace(NodeIt->GetBlueprint());
					}
					if (UInputAction::ActionsWithModifiedTriggers.Contains(NodeIt->InputAction))
					{
						NodeIt->ReconstructNode();
						BPsModifiedFromTriggerChange.Emplace(NodeIt->GetBlueprint());
					}
				}
			}

			if (BPsModifiedFromValueTypeChange.Num())
			{
				FNotificationInfo Info(FText::Format(LOCTEXT("ActionValueTypeChange", "Changing action value type affected {0} blueprint(s)!"), BPsModifiedFromValueTypeChange.Num()));
				Info.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(Info);
			}
			if (BPsModifiedFromTriggerChange.Num())
			{
				FNotificationInfo Info(FText::Format(LOCTEXT("ActionTriggerChange", "Changing action triggers affected {0} blueprint(s)!"), BPsModifiedFromTriggerChange.Num()));
				Info.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(Info);
			}

			UInputAction::ActionsWithModifiedValueTypes.Reset();
			UInputAction::ActionsWithModifiedTriggers.Reset();
		}
	}
	
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FInputBlueprintNodesModule, STATGROUP_Tickables); }
	// End FTickableEditorObject interface
};

IMPLEMENT_MODULE(FInputBlueprintNodesModule, InputBlueprintNodes)

#undef LOCTEXT_NAMESPACE
