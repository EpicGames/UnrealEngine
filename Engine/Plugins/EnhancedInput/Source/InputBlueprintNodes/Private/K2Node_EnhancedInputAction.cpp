// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_EnhancedInputAction.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintNodeSpawner.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Editor.h"
#include "EditorCategoryUtils.h"
#include "GraphEditorSettings.h"
#include "InputAction.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_EnhancedInputActionEvent.h"
#include "K2Node_GetInputActionValue.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"
#include "Misc/PackageName.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "EnhancedInputEditorSettings.h"
#include "WidgetBlueprint.h"
#include "Blueprint/UserWidget.h"
#include "BlueprintNodeTemplateCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_EnhancedInputAction)

#define LOCTEXT_NAMESPACE "K2Node_EnhancedInputAction"

static const FName InputActionPinName = TEXT("InputAction");
static const FName ElapsedSecondsPinName = TEXT("ElapsedSeconds");
static const FName TriggeredSecondsPinName = TEXT("TriggeredSeconds");
static const FName ActionValuePinName = TEXT("ActionValue");

namespace UE::Input
{
	static bool bShouldWarnOnUnsupportedInputPin = false;
	static FAutoConsoleVariableRef CVarShouldWarnOnUnsupportedInputPin(
		TEXT("enhancedInput.bp.bShouldWarnOnUnsupportedInputPin"),
		bShouldWarnOnUnsupportedInputPin,
		TEXT("Should the Enhanced Input event node throw a warning if a \"Unsuported\" pin has a connection?"),
		ECVF_Default);
}

UK2Node_EnhancedInputAction::UK2Node_EnhancedInputAction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void ForEachEventPinName(TFunctionRef<bool(ETriggerEvent Event, FName PinName)> PinLambda)
{
	UEnum* EventEnum = StaticEnum<ETriggerEvent>();
	for (int32 i = 0; i < EventEnum->NumEnums() - 1; ++i)
	{
		if (!EventEnum->HasMetaData(TEXT("Hidden"), i))
		{
			if (!PinLambda(ETriggerEvent(EventEnum->GetValueByIndex(i)), *EventEnum->GetNameStringByIndex(i)))
			{
				break;
			}
		}
	}
}

UInputActionEventNodeSpawner* UInputActionEventNodeSpawner::Create(TSubclassOf<UEdGraphNode> const NodeClass, TObjectPtr<const UInputAction> InAction)
{
	check(NodeClass != nullptr);
	check(NodeClass->IsChildOf<UEdGraphNode>());
	check(InAction != nullptr);
	
	UInputActionEventNodeSpawner* NodeSpawner = NewObject<UInputActionEventNodeSpawner>(GetTransientPackage());
	NodeSpawner->NodeClass = NodeClass;
	NodeSpawner->WeakActionPtr = InAction;
	
	return NodeSpawner;
}

UEdGraphNode* UInputActionEventNodeSpawner::Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const
{
	check(ParentGraph != nullptr);
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);
	
	if (!FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph))
	{
		// Look to see if a node for this input action already exists. If it does, just return that
		// which will jump the focus to it.
		if (UK2Node* PreExistingNode = FindExistingNode(Blueprint))
		{
			return PreExistingNode;
		}		
	}
	
	return Super::Invoke(ParentGraph, Bindings, Location);
}

UK2Node* UInputActionEventNodeSpawner::FindExistingNode(const UBlueprint* Blueprint) const
{
	// We don't want references to node spawners to be keeping any input action assets from GC
	// if you unload a plugin for example, so we keep it as a weak pointer.
	TStrongObjectPtr<const UInputAction> ActionPtr = WeakActionPtr.Pin();
	if (!ActionPtr)
	{
		return nullptr;
	}
	
	TArray<UK2Node_EnhancedInputAction*> AllInputActionNodes;
	FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_EnhancedInputAction>(Blueprint, AllInputActionNodes);

	for (UK2Node_EnhancedInputAction* Node : AllInputActionNodes)
	{
		if (Node->InputAction == ActionPtr)
		{
			return Node;
		}
	}
	
	return nullptr;
}

void UK2Node_EnhancedInputAction::AllocateDefaultPins()
{
	PreloadObject((UObject*)InputAction);

	const ETriggerEventsSupported SupportedTriggerEvents = GetDefault<UBlueprintEditorSettings>()->bEnableInputTriggerSupportWarnings && InputAction ? InputAction->GetSupportedTriggerEvents() : ETriggerEventsSupported::All;
	
	ForEachEventPinName([this, SupportedTriggerEvents](ETriggerEvent Event, FName PinName)
	{
		static const UEnum* EventEnum = StaticEnum<ETriggerEvent>();

		UEdGraphPin* NewPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, PinName);

		// Mark all triggering exec pins as advanced view except for the triggered pin. Most of the time, triggered is what users should be using.
		// More advanced input set ups can use the more advanced pins when they want to! 
		NewPin->bAdvancedView = !(GetDefault<UEnhancedInputEditorSettings>()->VisibleEventPinsByDefault & static_cast<uint8>(Event));

		NewPin->PinToolTip = EventEnum->GetToolTipTextByIndex(EventEnum->GetIndexByValue(static_cast<uint8>(Event))).ToString();

		// Add a special tooltip and display name for pins that are unsupported
		if (UE::Input::bShouldWarnOnUnsupportedInputPin && !UInputTrigger::IsSupportedTriggerEvent(SupportedTriggerEvents, Event))
		{
			static const FText UnsuportedTooltip = LOCTEXT("UnsupportedTooltip", "\n\nThis trigger event is not supported by the action! Add a supported trigger to enable this pin.");
			NewPin->PinToolTip += UnsuportedTooltip.ToString();
			NewPin->PinFriendlyName = FText::Format(LOCTEXT("UnsupportedPinFriendlyName", "(Unsupported) {0}"), FText::FromName(NewPin->GetFName()));
		}

		// Continue iterating
		return true;
	});
	
	HideEventPins(nullptr);
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	
	AdvancedPinDisplay = ENodeAdvancedPins::Hidden;

	UEdGraphPin* ValuePin = CreatePin(EGPD_Output, UK2Node_GetInputActionValue::GetValueCategory(InputAction), UK2Node_GetInputActionValue::GetValueSubCategory(InputAction), UK2Node_GetInputActionValue::GetValueSubCategoryObject(InputAction), ActionValuePinName);
	
	Schema->SetPinAutogeneratedDefaultValueBasedOnType(ValuePin);

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Double, ElapsedSecondsPinName)->bAdvancedView = true;
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Double, TriggeredSecondsPinName)->bAdvancedView = true;

	if(InputAction)
	{
		UEdGraphPin* ActionPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, InputAction->GetClass(), InputActionPinName);
		ActionPin->DefaultObject = const_cast<UObject*>(Cast<UObject>(InputAction));
		ActionPin->DefaultValue = InputAction->GetName();
		ActionPin->bAdvancedView = true;
		Schema->ConstructBasicPinTooltip(*ActionPin, LOCTEXT("InputActionPinDescription", "The input action that caused this event to fire"), ActionPin->PinToolTip);	
	}
	
	Super::AllocateDefaultPins();
}

void UK2Node_EnhancedInputAction::HideEventPins(UEdGraphPin* RetainPin)
{
	// Gather pins
	const ETriggerEventsSupported SupportedTriggerEvents = GetDefault<UBlueprintEditorSettings>()->bEnableInputTriggerSupportWarnings && InputAction ? InputAction->GetSupportedTriggerEvents() : ETriggerEventsSupported::All;

	// Hide any event pins that are not supported by this Action's triggers in the advanced view
	ForEachEventPinName([this, SupportedTriggerEvents](ETriggerEvent Event, FName PinName)
	{
		if (UEdGraphPin* Pin = FindPin(PinName))
		{
			const bool bIsSupported = UInputTrigger::IsSupportedTriggerEvent(SupportedTriggerEvents, Event);
			
			Pin->bAdvancedView = !(GetDefault<UEnhancedInputEditorSettings>()->VisibleEventPinsByDefault & static_cast<uint8>(Event)) || !bIsSupported;
		}

		// Continue iterating
		return true;
	});
}

const ETriggerEvent UK2Node_EnhancedInputAction::GetTriggerTypeFromExecPin(const UEdGraphPin* ExecPin) const
{
	static const UEnum* EventEnum = StaticEnum<ETriggerEvent>();
	
	if(ensure(ExecPin && ExecPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec))
	{
		return (ETriggerEvent)(EventEnum->GetValueByName(ExecPin->PinName));		
	}
	
	return ETriggerEvent::None;
}

void UK2Node_EnhancedInputAction::PostReconstructNode()
{
	Super::PostReconstructNode();
	HideEventPins(nullptr);
}

void UK2Node_EnhancedInputAction::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);
	HideEventPins(Pin);
}

bool UK2Node_EnhancedInputAction::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	if(MyPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec && InputAction)
	{
		const ETriggerEvent Event = GetTriggerTypeFromExecPin(MyPin);
		const ETriggerEventsSupported SupportedEvents = GetDefault<UBlueprintEditorSettings>()->bEnableInputTriggerSupportWarnings ? InputAction->GetSupportedTriggerEvents() : ETriggerEventsSupported::All;

		if(!UInputTrigger::IsSupportedTriggerEvent(SupportedEvents, Event))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ActionName"), FText::FromName(InputAction->GetFName()));
			Args.Add(TEXT("PinName"), FText::FromName(MyPin->PinName));

			OutReason = FText::Format(LOCTEXT("UnsupportedEventType_DragTooltip", "WARNING: '{ActionName}' does not support the '{PinName}' trigger event."), Args).ToString();		
		}
	}
	
	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

FLinearColor UK2Node_EnhancedInputAction::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->EventNodeTitleColor;
}

FName UK2Node_EnhancedInputAction::GetActionName() const
{
	return  InputAction ? InputAction->GetFName() : FName();
}

FText UK2Node_EnhancedInputAction::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	// TODO: Is Using InputAction->GetFName okay here? Full Asset path would be better for disambiguation.
	if (TitleType == ENodeTitleType::MenuTitle)
	{
		return FText::FromName(GetActionName());
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("InputActionName"), FText::FromName(GetActionName()));

		FText LocFormat = LOCTEXT("EnhancedInputAction_Name", "EnhancedInputAction {InputActionName}");
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LocFormat, Args), this);
	}

	return CachedNodeTitle;
}

FText UK2Node_EnhancedInputAction::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		FString ActionPath = InputAction ? InputAction->GetFullName() : TEXT("");
		CachedTooltip.SetCachedText(
			FText::Format(
				LOCTEXT("EnhancedInputAction_Tooltip", "Event for when '{0}' triggers.\n\nNote: This is not guaranteed to fire every frame, only when the Action is triggered and the current Input Mode includes 'Game'.\n\n{1}\n\n{2}"),
				FText::FromString(ActionPath),
				LOCTEXT("EnhancedInputAction_Node_Tooltip_Tip", "Tip: Use the 'showdebug enhancedinput' command while playing to see debug information about Enhanced Input."),
				LOCTEXT("EnhancedInputAction_Node_SettingsTooltip", "You can change what execution pins are visible by default in the Enhanced Input Editor Preferences.")),
			this);
	}
	return CachedTooltip;
}

FSlateIcon UK2Node_EnhancedInputAction::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
	return Icon;
}

bool UK2Node_EnhancedInputAction::IsCompatibleWithGraph(UEdGraph const* Graph) const
{
	// This node expands into event nodes and must be placed in a Ubergraph
	EGraphType const GraphType = Graph->GetSchema()->GetGraphType(Graph);
	bool bIsCompatible = (GraphType == EGraphType::GT_Ubergraph);

	if (bIsCompatible)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);

		UEdGraphSchema_K2 const* K2Schema = Cast<UEdGraphSchema_K2>(Graph->GetSchema());
		bool const bIsConstructionScript = (K2Schema != nullptr) ? UEdGraphSchema_K2::IsConstructionScript(Graph) : false;

		bIsCompatible = (Blueprint != nullptr) && Blueprint->SupportsInputEvents() && !bIsConstructionScript && Super::IsCompatibleWithGraph(Graph);
	}
	return bIsCompatible;
}

UObject* UK2Node_EnhancedInputAction::GetJumpTargetForDoubleClick() const
{
	return const_cast<UObject*>(Cast<UObject>(InputAction));
}

void UK2Node_EnhancedInputAction::JumpToDefinition() const
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(GetJumpTargetForDoubleClick());
}

void UK2Node_EnhancedInputAction::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (!InputAction)
	{
		MessageLog.Error(*LOCTEXT("EnhancedInputAction_ErrorFmt", "EnhancedInputActionEvent references invalid 'null' action for @@").ToString(), this);
		return;
	}
	
	// There are no supported triggers on this action, we should put a note down
	// This would only be the case if the user has added a custom UInputTrigger that uses ETriggeredEventsSupported::None
	if(InputAction->GetSupportedTriggerEvents() == ETriggerEventsSupported::None)
	{
		MessageLog.Warning(
			*LOCTEXT("EnhancedInputAction_NoTriggersOnAction",
				"@@ may not be triggered. There are no triggers supported on this action! Add a trigger to this action to resolve this warning.").ToString(),
				this);
	}
}

void UK2Node_EnhancedInputAction::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if(!InputAction)
	{
		static const FText InvalidActionWarning = LOCTEXT("InvalidInputActionDuringExpansion", "@@ does not have a valid Input Action asset!!");
		CompilerContext.MessageLog.Warning(*InvalidActionWarning.ToString(), this);
		return;
	}
	
	// Establish active pins
	struct ActivePinData
	{
		ActivePinData(UEdGraphPin* InPin, ETriggerEvent InTriggerEvent) : Pin(InPin), TriggerEvent(InTriggerEvent) {}
		UEdGraphPin* Pin;
		ETriggerEvent TriggerEvent;
	};

	const ETriggerEventsSupported SupportedTriggerEvents = GetDefault<UBlueprintEditorSettings>()->bEnableInputTriggerSupportWarnings ? InputAction->GetSupportedTriggerEvents() : ETriggerEventsSupported::All;
		
	TArray<ActivePinData> ActivePins;
	ForEachActiveEventPin([this, &ActivePins, &SupportedTriggerEvents, &CompilerContext](ETriggerEvent Event, UEdGraphPin& InputActionPin)
	{
		ActivePins.Add(ActivePinData(&InputActionPin, Event));
		// Check if this exec pin is supported!
		if (UE::Input::bShouldWarnOnUnsupportedInputPin && !UInputTrigger::IsSupportedTriggerEvent(SupportedTriggerEvents, Event))
		{
			CompilerContext.MessageLog.Warning(*FText::Format(LOCTEXT("UnsuportedEventTypeOnAction", "'{0}'on @@ may not be executed because it is not a supported trigger on this action!"), InputActionPin.GetDisplayName()).ToString(), this);
		}

		// Continue iterating
		return true;
	});

	if (ActivePins.Num() == 0)
	{
		return;
	}

	// Bind all active pins to their action delegate
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	auto CreateInputActionEvent = [this, &CompilerContext, &SourceGraph](UEdGraphPin* Pin, ETriggerEvent TriggerEvent) -> UK2Node_EnhancedInputActionEvent*
	{
		if (!InputAction)
		{
			return nullptr;
		}

		UK2Node_EnhancedInputActionEvent* InputActionEvent = CompilerContext.SpawnIntermediateNode<UK2Node_EnhancedInputActionEvent>(this, SourceGraph);
		InputActionEvent->CustomFunctionName = FName(*FString::Printf(TEXT("InpActEvt_%s_%s"), *GetActionName().ToString(), *InputActionEvent->GetName()));
		InputActionEvent->InputAction = InputAction;
		InputActionEvent->TriggerEvent = TriggerEvent;
		InputActionEvent->EventReference.SetExternalDelegateMember(FName(TEXT("EnhancedInputActionHandlerDynamicSignature__DelegateSignature")));
		InputActionEvent->AllocateDefaultPins();
		return InputActionEvent;
	};

	// Widget blueprints require the bAutomaticallyRegisterInputOnConstruction to be set to true in order to receive callbacks
	if (GetBlueprint()->IsA<UWidgetBlueprint>())
	{
		CompilerContext.SetAutomaticallyRegisterInputEvents(true);
	}

	// Create temporary variables to copy ActionValue and ElapsedSeconds into
	UK2Node_TemporaryVariable* ActionValueVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
	ActionValueVar->VariableType.PinCategory = UK2Node_GetInputActionValue::GetValueCategory(InputAction);
	ActionValueVar->VariableType.PinSubCategory = UK2Node_GetInputActionValue::GetValueSubCategory(InputAction);
	ActionValueVar->VariableType.PinSubCategoryObject = UK2Node_GetInputActionValue::GetValueSubCategoryObject(InputAction);
	ActionValueVar->AllocateDefaultPins();

	UK2Node_TemporaryVariable* ElapsedSecondsVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
	ElapsedSecondsVar->VariableType.PinCategory = UEdGraphSchema_K2::PC_Real;
	ElapsedSecondsVar->VariableType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
	ElapsedSecondsVar->AllocateDefaultPins();
	UK2Node_TemporaryVariable* TriggeredSecondsVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
	TriggeredSecondsVar->VariableType.PinCategory = UEdGraphSchema_K2::PC_Real;
	TriggeredSecondsVar->VariableType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
	TriggeredSecondsVar->AllocateDefaultPins();

	UK2Node_TemporaryVariable* InputActionVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
	InputActionVar->VariableType.PinCategory = UEdGraphSchema_K2::PC_Object;
	InputActionVar->VariableType.PinSubCategoryObject = InputAction->GetClass();
	InputActionVar->AllocateDefaultPins();

	for (ActivePinData& PinData : ActivePins)
	{
		UEdGraphPin* EachPin = PinData.Pin;
		UK2Node_EnhancedInputActionEvent* InputActionEvent = CreateInputActionEvent(EachPin, PinData.TriggerEvent);

		if (!InputActionEvent)
		{
			continue;
		}

		// Create assignment nodes to assign the action value
		UK2Node_AssignmentStatement* ActionValueInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
		ActionValueInitialize->AllocateDefaultPins();
		Schema->TryCreateConnection(ActionValueVar->GetVariablePin(), ActionValueInitialize->GetVariablePin());
		Schema->TryCreateConnection(ActionValueInitialize->GetValuePin(), InputActionEvent->FindPinChecked(ActionValuePinName));
		// Connect the events to the assign location nodes
		Schema->TryCreateConnection(Schema->FindExecutionPin(*InputActionEvent, EGPD_Output), ActionValueInitialize->GetExecPin());

		// Create assignment nodes to assign the elapsed timers and input action
		UK2Node_AssignmentStatement* ElapsedSecondsInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
		ElapsedSecondsInitialize->AllocateDefaultPins();
		Schema->TryCreateConnection(ElapsedSecondsVar->GetVariablePin(), ElapsedSecondsInitialize->GetVariablePin());
		Schema->TryCreateConnection(ElapsedSecondsInitialize->GetValuePin(), InputActionEvent->FindPinChecked(TEXT("ElapsedTime")));

		UK2Node_AssignmentStatement* TriggeredSecondsInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
		TriggeredSecondsInitialize->AllocateDefaultPins();
		Schema->TryCreateConnection(TriggeredSecondsVar->GetVariablePin(), TriggeredSecondsInitialize->GetVariablePin());
		Schema->TryCreateConnection(TriggeredSecondsInitialize->GetValuePin(), InputActionEvent->FindPinChecked(TEXT("TriggeredTime")));

		UK2Node_AssignmentStatement* InputActionInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
		InputActionInitialize->AllocateDefaultPins();
		Schema->TryCreateConnection(InputActionVar->GetVariablePin(), InputActionInitialize->GetVariablePin());
		Schema->TryCreateConnection(InputActionInitialize->GetValuePin(), InputActionEvent->FindPinChecked(TEXT("SourceAction")));
		 
		// Connect the assign location to the assign elapsed time nodes
		Schema->TryCreateConnection(ActionValueInitialize->GetThenPin(), ElapsedSecondsInitialize->GetExecPin());
		Schema->TryCreateConnection(ElapsedSecondsInitialize->GetThenPin(), TriggeredSecondsInitialize->GetExecPin());
		Schema->TryCreateConnection(TriggeredSecondsInitialize->GetThenPin(), InputActionInitialize->GetExecPin());
		
		// Move the original event connections to the then pin of the Input Action assign
		CompilerContext.MovePinLinksToIntermediate(*EachPin, *InputActionInitialize->GetThenPin());

		// Move the original event variable connections to the intermediate nodes
		CompilerContext.MovePinLinksToIntermediate(*FindPin(ActionValuePinName), *ActionValueVar->GetVariablePin());
		CompilerContext.MovePinLinksToIntermediate(*FindPin(ElapsedSecondsPinName), *ElapsedSecondsVar->GetVariablePin());
		CompilerContext.MovePinLinksToIntermediate(*FindPin(TriggeredSecondsPinName), *TriggeredSecondsVar->GetVariablePin());
		CompilerContext.MovePinLinksToIntermediate(*FindPin(InputActionPinName), *InputActionVar->GetVariablePin());
	}
}

void UK2Node_EnhancedInputAction::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	auto CustomizeInputNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, TWeakObjectPtr<const UInputAction> Action)
	{
		UK2Node_EnhancedInputAction* InputNode = CastChecked<UK2Node_EnhancedInputAction>(NewNode);
		InputNode->InputAction = Action.Get();
	};

	// Do a first time registration using the node's class to pull in all existing actions
	if (ActionRegistrar.IsOpenForRegistration(GetClass()))
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

		static bool bRegisterOnce = true;
		if (bRegisterOnce)
		{
			bRegisterOnce = false;
			if (AssetRegistry.IsLoadingAssets())
			{
				AssetRegistry.OnFilesLoaded().AddLambda([]() { FBlueprintActionDatabase::Get().RefreshClassActions(StaticClass()); });
			}
		}

		TArray<FAssetData> ActionAssets;
		AssetRegistry.GetAssetsByClass(UInputAction::StaticClass()->GetClassPathName(), ActionAssets, true);
		for (const FAssetData& ActionAsset : ActionAssets)
		{
			if (FPackageName::GetPackageMountPoint(ActionAsset.PackageName.ToString()) != NAME_None)
			{
				if (const UInputAction* Action = Cast<const UInputAction>(ActionAsset.GetAsset()))
				{
					UBlueprintNodeSpawner* NodeSpawner = UInputActionEventNodeSpawner::Create(GetClass(), Action);
					check(NodeSpawner != nullptr);
					
					NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, TWeakObjectPtr<const UInputAction>(Action));
					ActionRegistrar.AddBlueprintAction(Action, NodeSpawner);
				}
			}
		}
	}
	else if (const UInputAction* Action = Cast<const UInputAction>(ActionRegistrar.GetActionKeyFilter()))
	{
		// If this is a specific UInputAction asset update it.
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, TWeakObjectPtr<const UInputAction>(Action));
		ActionRegistrar.AddBlueprintAction(Action, NodeSpawner);
	}
}

FText UK2Node_EnhancedInputAction::GetMenuCategory() const
{
	static FNodeTextCache CachedCategory;
	if (CachedCategory.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedCategory.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Input, LOCTEXT("ActionMenuCategory", "Enhanced Action Events")), this);	// TODO: Rename Action Events once old action system is removed
	}
	return CachedCategory;
}

FBlueprintNodeSignature UK2Node_EnhancedInputAction::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddKeyValue(GetActionName().ToString());

	return NodeSignature;
}

TSharedPtr<FEdGraphSchemaAction> UK2Node_EnhancedInputAction::GetEventNodeAction(const FText& ActionCategory)
{
	// TODO: Custom EdGraphSchemaAction required?
	TSharedPtr<FEdGraphSchemaAction_K2InputAction> EventNodeAction = MakeShareable(new FEdGraphSchemaAction_K2InputAction(ActionCategory, GetNodeTitle(ENodeTitleType::EditableTitle), GetTooltipText(), 0));
	EventNodeAction->NodeTemplate = this;
	return EventNodeAction;
}

bool UK2Node_EnhancedInputAction::HasAnyConnectedEventPins() const
{
	bool bHasAnyConnectedEventPins = false;
	ForEachActiveEventPin([&bHasAnyConnectedEventPins](ETriggerEvent, UEdGraphPin& Pin)
	{
		// Stop iterating on the first active pin
		bHasAnyConnectedEventPins = true;
		return false;
	});

	return bHasAnyConnectedEventPins;
}

void UK2Node_EnhancedInputAction::ForEachActiveEventPin(TFunctionRef<bool(ETriggerEvent, UEdGraphPin&)> Predicate) const
{
	ForEachEventPinName([this, Predicate](ETriggerEvent Event, FName PinName) 
	{
		UEdGraphPin* InputActionPin = FindPin(PinName, EEdGraphPinDirection::EGPD_Output);
		if (InputActionPin && InputActionPin->LinkedTo.Num() > 0)
		{
			return Predicate(Event, *InputActionPin);
		}
		return true;
	});
}

#undef LOCTEXT_NAMESPACE

