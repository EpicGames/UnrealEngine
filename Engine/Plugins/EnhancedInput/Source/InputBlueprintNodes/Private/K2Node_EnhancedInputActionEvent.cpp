// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_EnhancedInputActionEvent.h"
#include "EnhancedInputActionDelegateBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_EnhancedInputActionEvent)

UK2Node_EnhancedInputActionEvent::UK2Node_EnhancedInputActionEvent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bInternalEvent = true;
	TriggerEvent = ETriggerEvent::None;
}

UClass* UK2Node_EnhancedInputActionEvent::GetDynamicBindingClass() const
{
	return UEnhancedInputActionDelegateBinding::StaticClass();
}

void UK2Node_EnhancedInputActionEvent::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UEnhancedInputActionDelegateBinding* InputActionBindingObject = CastChecked<UEnhancedInputActionDelegateBinding>(BindingObject);

	FBlueprintEnhancedInputActionBinding Binding;
	Binding.InputAction = InputAction;
	Binding.TriggerEvent = TriggerEvent;
	Binding.FunctionNameToBind = CustomFunctionName;

	InputActionBindingObject->InputActionDelegateBindings.Add(Binding);
}

