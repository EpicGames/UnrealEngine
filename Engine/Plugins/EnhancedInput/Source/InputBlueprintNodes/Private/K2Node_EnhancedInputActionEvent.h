// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_Event.h"
#include "K2Node_EnhancedInputActionEvent.generated.h"

#define UE_API INPUTBLUEPRINTNODES_API

class UInputAction;
enum class ETriggerEvent : uint8;

class UDynamicBlueprintBinding;

UCLASS(MinimalAPI)
class UK2Node_EnhancedInputActionEvent : public UK2Node_Event
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<const UInputAction> InputAction;

	UPROPERTY()
	ETriggerEvent TriggerEvent;

	//~ Begin UK2Node Interface
	UE_API virtual UClass* GetDynamicBindingClass() const override;
	UE_API virtual void RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const override;
	//~ End UK2Node Interface
};

#undef UE_API
