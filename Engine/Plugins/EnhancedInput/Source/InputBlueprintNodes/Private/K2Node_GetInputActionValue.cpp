// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GetInputActionValue.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "InputAction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"
#include "Editor.h"
#include "BlueprintNodeSpawner.h"
#include "EditorCategoryUtils.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "K2Node_Self.h"
#include "K2Node_InputActionValueAccessor.h"
#include "Misc/PackageName.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetInputActionValue)

#define LOCTEXT_NAMESPACE "K2Node_GetInputActionValue"


UK2Node_GetInputActionValue::UK2Node_GetInputActionValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


struct FValueTypeData
{
	FValueTypeData(FName InCategory, FName InSubCategory = NAME_None, UScriptStruct* InSubCategoryObject = nullptr)
		: Category(InCategory)
		, SubCategory(InSubCategory)
		, SubCategoryObject(InSubCategoryObject)
	{
	}

	FName Category;
	FName SubCategory;
	UScriptStruct* SubCategoryObject;
};

const TMap<EInputActionValueType, FValueTypeData>& GetValueLookups()
{
	static const TMap<EInputActionValueType, FValueTypeData> ValueLookups =
	{
		{ EInputActionValueType::Boolean, FValueTypeData(UEdGraphSchema_K2::PC_Boolean) },
		{ EInputActionValueType::Axis1D, FValueTypeData(UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Double) },
		{ EInputActionValueType::Axis2D, FValueTypeData(UEdGraphSchema_K2::PC_Struct, NAME_None, TBaseStructure<FVector2D>::Get()) },
		{ EInputActionValueType::Axis3D, FValueTypeData(UEdGraphSchema_K2::PC_Struct, NAME_None, TBaseStructure<FVector>::Get()) },
	};
	return ValueLookups;
}

FName UK2Node_GetInputActionValue::GetValueCategory(const UInputAction* InputAction)
{
	EInputActionValueType Type = InputAction ? InputAction->ValueType : EInputActionValueType::Boolean;
	return GetValueLookups()[Type].Category;
}

FName UK2Node_GetInputActionValue::GetValueSubCategory(const UInputAction* InputAction)
{
	EInputActionValueType Type = InputAction ? InputAction->ValueType : EInputActionValueType::Boolean;
	return GetValueLookups()[Type].SubCategory;
}

UScriptStruct* UK2Node_GetInputActionValue::GetValueSubCategoryObject(const UInputAction* InputAction)
{
	EInputActionValueType Type = InputAction ? InputAction->ValueType : EInputActionValueType::Boolean;
	return GetValueLookups()[Type].SubCategoryObject;
}

void UK2Node_GetInputActionValue::AllocateDefaultPins()
{
	PreloadObject((UObject*)InputAction);

	Super::AllocateDefaultPins();

	// Dynamically typed output
	FName SubCategory = UK2Node_GetInputActionValue::GetValueSubCategory(InputAction);

	if (SubCategory != NAME_None)
	{
		CreatePin(EGPD_Output, UK2Node_GetInputActionValue::GetValueCategory(InputAction), SubCategory, UEdGraphSchema_K2::PN_ReturnValue);
	}
	else
	{
		CreatePin(EGPD_Output, UK2Node_GetInputActionValue::GetValueCategory(InputAction), UK2Node_GetInputActionValue::GetValueSubCategoryObject(InputAction), UEdGraphSchema_K2::PN_ReturnValue);
	}
}

void UK2Node_GetInputActionValue::Initialize(const UInputAction* Action)
{
	InputAction = Action;
}

FName UK2Node_GetInputActionValue::GetActionName() const
{
	return  InputAction ? InputAction->GetFName() : FName();
}

FText UK2Node_GetInputActionValue::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::MenuTitle)
	{
		return FText::FromName(GetActionName());
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("InputActionName"), FText::FromName(GetActionName()));

		FText LocFormat = LOCTEXT("GetInputAction_Name", "Get {InputActionName}");
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LocFormat, Args), this);
	}

	return CachedNodeTitle;
}

void UK2Node_GetInputActionValue::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	UEdGraphPin* ReturnValue = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	if (ReturnValue->LinkedTo.Num() == 0)
	{
		return;
	}

	// Accessor does the call to UEnhancedInputLibrary::GetBoundActionValue, passing in self from SelfNode
	UK2Node_InputActionValueAccessor* AccessorNode = CompilerContext.SpawnIntermediateNode<UK2Node_InputActionValueAccessor>(this, SourceGraph);
	AccessorNode->Initialize(InputAction);
	AccessorNode->AllocateDefaultPins();

	// Create a self node, and wire it to "Actor" pin on the InputActionValueAccessor
	UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, SourceGraph);
	SelfNode->AllocateDefaultPins();
	Schema->TryCreateConnection(SelfNode->FindPinChecked(UEdGraphSchema_K2::PN_Self), AccessorNode->FindPinChecked(TEXT("Actor")));

	// And finally hook the InputActionValueAccessor's return value up to the GetInputActionValue return pin's connected nodes, automatically inserting appropriate conversion nodes.
	TArray<UEdGraphPin*> LinkedTo(ReturnValue->LinkedTo);
	for (UEdGraphPin* EachLink : LinkedTo)
	{
		Schema->TryCreateConnection(AccessorNode->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue), EachLink);
	}
}

FText UK2Node_GetInputActionValue::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		FString ActionPath = InputAction ? InputAction->GetFullName() : TEXT("");
		CachedTooltip.SetCachedText(
			FText::Format(LOCTEXT("GetInputAction_Tooltip", "Returns the current value of {0}.  If input is disabled for the actor the value will be 0. \n\nNote: If the value is being altered by an Input Trigger or Input Modifier (such as a Released trigger) then the value may be 0."),
			FText::FromString(ActionPath)), this);
	}
	return CachedTooltip;
}

bool UK2Node_GetInputActionValue::IsCompatibleWithGraph(UEdGraph const* Graph) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);

	UEdGraphSchema_K2 const* K2Schema = Cast<UEdGraphSchema_K2>(Graph->GetSchema());
	bool const bIsConstructionScript = (K2Schema != nullptr) ? K2Schema->IsConstructionScript(Graph) : false;

	return (Blueprint != nullptr) && Blueprint->SupportsInputEvents() && !bIsConstructionScript && Super::IsCompatibleWithGraph(Graph);
}

UObject* UK2Node_GetInputActionValue::GetJumpTargetForDoubleClick() const
{
	return const_cast<UObject*>(Cast<UObject>(InputAction));
}

void UK2Node_GetInputActionValue::JumpToDefinition() const
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(GetJumpTargetForDoubleClick());
}

void UK2Node_GetInputActionValue::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (!InputAction)
	{
		MessageLog.Error(*LOCTEXT("EnhancedInputAction_ErrorFmt", "EnhancedInputActionEvent references invalid 'null' action for @@").ToString(), this);
	}
}

void UK2Node_GetInputActionValue::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	auto CustomizeInputNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, TWeakObjectPtr<const UInputAction> Action)
	{
		UK2Node_GetInputActionValue* InputNode = CastChecked<UK2Node_GetInputActionValue>(NewNode);
		InputNode->Initialize(Action.Get());
	};
	
	// Do a first time registration using the node's class to pull in all existing actions
	if (ActionRegistrar.IsOpenForRegistration(GetClass()))
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

		static bool bRegisterOnce = true;
		if (bRegisterOnce)
		{
			bRegisterOnce = false;
			if (AssetRegistry.IsLoadingAssets())
			{
				AssetRegistry.OnFilesLoaded().AddLambda([]() { FBlueprintActionDatabase::Get().RefreshClassActions(StaticClass()); });
			}
		}

		TArray<FAssetData> ActionAssets;
		AssetRegistry.GetAssetsByClass(UInputAction::StaticClass()->GetClassPathName(), ActionAssets);
		for (const FAssetData& ActionAsset : ActionAssets)
		{
			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
			check(NodeSpawner != nullptr);

			if (FPackageName::GetPackageMountPoint(ActionAsset.PackageName.ToString()) != NAME_None)
			{
				if (const UInputAction* Action = Cast<const UInputAction>(ActionAsset.GetAsset()))
				{
					NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, TWeakObjectPtr<const UInputAction>(Action));
					ActionRegistrar.AddBlueprintAction(Action, NodeSpawner);
				}
			}
		}
	}
	else if (const UInputAction* Action = Cast<const UInputAction>(ActionRegistrar.GetActionKeyFilter()))
	{
		// If this is a specific UInputAction asset update it.
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, TWeakObjectPtr<const UInputAction>(Action));
		ActionRegistrar.AddBlueprintAction(Action, NodeSpawner);
	}
}

FText UK2Node_GetInputActionValue::GetMenuCategory() const
{
	static FNodeTextCache CachedCategory;
	if (CachedCategory.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedCategory.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Input, LOCTEXT("ActionMenuCategory", "Enhanced Action Values")), this);
	}
	return CachedCategory;
}

FBlueprintNodeSignature UK2Node_GetInputActionValue::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddKeyValue(GetActionName().ToString());

	return NodeSignature;
}

#undef LOCTEXT_NAMESPACE

