// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InputActionValueAccessor.h"
#include "EnhancedInputActionDelegateBinding.h"
#include "EnhancedInputLibrary.h"
#include "InputAction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InputActionValueAccessor)


UK2Node_InputActionValueAccessor::UK2Node_InputActionValueAccessor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_InputActionValueAccessor::Initialize(const UInputAction* Action)
{
	InputAction = Action;
	SetFromFunction(UEnhancedInputLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UEnhancedInputLibrary, GetBoundActionValue)));
}

void UK2Node_InputActionValueAccessor::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	UEdGraphPin* InputActionPin = FindPinChecked(TEXT("Action"));
	InputActionPin->DefaultObject = const_cast<UInputAction*>(ToRawPtr(InputAction));
}

UClass* UK2Node_InputActionValueAccessor::GetDynamicBindingClass() const
{
	return UEnhancedInputActionValueBinding::StaticClass();
}

void UK2Node_InputActionValueAccessor::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UEnhancedInputActionValueBinding* InputActionBindingObject = CastChecked<UEnhancedInputActionValueBinding>(BindingObject);

	FBlueprintEnhancedInputActionBinding Binding;
	Binding.InputAction = InputAction;
	Binding.FunctionNameToBind = GET_FUNCTION_NAME_CHECKED(UEnhancedInputLibrary, GetBoundActionValue);

	InputActionBindingObject->InputActionValueBindings.Add(Binding);
}