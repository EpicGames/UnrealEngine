// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_CallFunction.h"
#include "K2Node_InputActionValueAccessor.generated.h"

#define UE_API INPUTBLUEPRINTNODES_API

class UInputAction;

UCLASS(MinimalAPI)
class UK2Node_InputActionValueAccessor : public UK2Node_CallFunction
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin EdGraphNode Interface
	UE_API virtual void AllocateDefaultPins() override;
	//~ End EdGraphNode Interface

	//~ Begin UK2Node Interface
	UE_API virtual UClass* GetDynamicBindingClass() const override;
	UE_API virtual void RegisterDynamicBinding(class UDynamicBlueprintBinding* BindingObject) const override;
	virtual bool IsNodePure() const override { return true; }
	//~ End UK2Node Interface

	UE_API void Initialize(const UInputAction* Action);

protected:
	//~ Begin UK2Node_CallFunction interface
	virtual bool CanToggleNodePurity() const override { return false; }
	//~ End UK2Node_CallFunction interface

private:
	UPROPERTY()
	TObjectPtr<const UInputAction> InputAction;
};

#undef UE_API
