// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InputDebugKey.h"
#include "GraphEditorSettings.h"
#include "EdGraphSchema_K2_Actions.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_InputDebugKeyEvent.h"
#include "KismetCompiler.h"
#include "BlueprintNodeSpawner.h"
#include "EditorCategoryUtils.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "InputActionValue.h"
#include "Styling/AppStyle.h"
#include "GameFramework/InputSettings.h"
#include "Editor.h"								// for FEditorDelegates::OnEnableGestureRecognizerChanged
#include "Blueprint/UserWidget.h"
#include "BlueprintNodeTemplateCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InputDebugKey)

#define LOCTEXT_NAMESPACE "UK2Node_InputDebugKey"

static const FName ActionPinName = TEXT("ActionValue");

UK2Node_InputDebugKey::UK2Node_InputDebugKey(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if PLATFORM_MAC && WITH_EDITOR
	if (IsTemplate())
	{
		FProperty* ControlProp = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UK2Node_InputDebugKey, bControl));
		FProperty* CommandProp = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UK2Node_InputDebugKey, bCommand));

		ControlProp->SetMetaData(TEXT("DisplayName"), TEXT("Command"));
		CommandProp->SetMetaData(TEXT("DisplayName"), TEXT("Control"));
	}
#endif

	// Show the development only banner to warn the user they're not going to get the benefits of this node in a shipping build
	SetEnabledState(ENodeEnabledState::DevelopmentOnly, false);
}

UInputDebugKeyEventNodeSpawner* UInputDebugKeyEventNodeSpawner::Create(TSubclassOf<UEdGraphNode> const NodeClass, const FKey& InDebugKey)
{
	check(NodeClass != nullptr);
	check(NodeClass->IsChildOf<UEdGraphNode>());
	
	UInputDebugKeyEventNodeSpawner* NodeSpawner = NewObject<UInputDebugKeyEventNodeSpawner>(GetTransientPackage());
	NodeSpawner->NodeClass = NodeClass;
	NodeSpawner->DebugKey = InDebugKey;
	
	return NodeSpawner;
}

UEdGraphNode* UInputDebugKeyEventNodeSpawner::Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const
{
	check(ParentGraph != nullptr);
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);
	
	if (!FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph))
	{
		// Look to see if a node for this input action already exists. If it does, just return that
		// which will jump the focus to it.
		if (UK2Node* PreExistingNode = FindExistingNode(Blueprint))
		{
			return PreExistingNode;
		}		
	}
	
	return Super::Invoke(ParentGraph, Bindings, Location);
}

UK2Node* UInputDebugKeyEventNodeSpawner::FindExistingNode(const UBlueprint* Blueprint) const
{
	TArray<UK2Node_InputDebugKey*> AllDebugKeyNodes;
	FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_InputDebugKey>(Blueprint, AllDebugKeyNodes);

	for (UK2Node_InputDebugKey* Node : AllDebugKeyNodes)
	{
		if (Node->InputKey == DebugKey)
		{
			return Node;
		}
	}
	
	return nullptr;
}

void UK2Node_InputDebugKey::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	CachedNodeTitle.Clear();
	CachedTooltip.Clear();
}

void UK2Node_InputDebugKey::AllocateDefaultPins()
{
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, TEXT("Pressed"));
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, TEXT("Released"));
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FKey::StaticStruct(), TEXT("Key"));
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FInputActionValue::StaticStruct(), ActionPinName);
	
	Super::AllocateDefaultPins();
}

FLinearColor UK2Node_InputDebugKey::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->EventNodeTitleColor;
}

FName UK2Node_InputDebugKey::GetModifierName() const
{
    FString ModName;

	auto AddMod = [&ModName](bool bHasModifier, FString Name)
	{
		if (bHasModifier)
		{
			ModName += Name + (ModName.Len() ? TEXT("+") : TEXT(""));
		}
	};

	AddMod(bControl, TEXT("Ctrl"));
	AddMod(bCommand, TEXT("Cmd"));
	AddMod(bAlt, TEXT("Alt"));
	AddMod(bShift, TEXT("Shift"));

	return FName(*ModName);
}

FText UK2Node_InputDebugKey::GetModifierText() const
{
#if PLATFORM_MAC
    const FText CommandText = LOCTEXT("KeyName_Control", "Ctrl");
    const FText ControlText = LOCTEXT("KeyName_Command", "Cmd");
#else
    const FText ControlText = LOCTEXT("KeyName_Control", "Ctrl");
    const FText CommandText = LOCTEXT("KeyName_Command", "Cmd");
#endif
    const FText AltText = LOCTEXT("KeyName_Alt", "Alt");
    const FText ShiftText = LOCTEXT("KeyName_Shift", "Shift");

	const FText AppenderText = LOCTEXT("ModAppender", "+");

	FFormatNamedArguments Args;
	int32 ModCount = 0;

	auto AddMod = [&Args, &ModCount](bool bHasModifier, const FText Text) {
		if (bHasModifier)
		{
			Args.Add(FString::Printf(TEXT("Mod%d"), ++ModCount), Text);
		}
	};

	AddMod(bControl, ControlText);
	AddMod(bCommand, CommandText);
	AddMod(bAlt, AltText);
	AddMod(bShift, ShiftText);

	for (int32 i = 1; i <= 4; ++i)
	{
		if (i > ModCount)
		{
			Args.Add(FString::Printf(TEXT("Mod%d"), i), FText::GetEmpty());
		}

		Args.Add(FString::Printf(TEXT("Appender%d"), i), (i < ModCount ? AppenderText : FText::GetEmpty()));
	}

	Args.Add(TEXT("Key"), GetKeyText());

	return FText::Format(LOCTEXT("NodeTitle", "{Mod1}{Appender1}{Mod2}{Appender2}{Mod3}{Appender3}{Mod4}"), Args);
}

FText UK2Node_InputDebugKey::GetKeyText() const
{
	return InputKey.GetDisplayName();
}

FText UK2Node_InputDebugKey::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ModifierKey"), GetModifierText());
		Args.Add(TEXT("Key"), GetKeyText());

		if (bControl || bAlt || bShift || bCommand)
		{
			// FText::Format() is slow, so we cache this to save on performance
			CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("InputDebugKey_Name_WithModifiers", "Debug Key {ModifierKey} {Key}"), Args), this);
		}
		else
		{
			CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("InputDebugKey_Name_NoModifiers", "Debug Key {Key}"), Args), this);
		}
	}
	return CachedNodeTitle;
}

FText UK2Node_InputDebugKey::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		FText ModifierText = GetModifierText();
		FText KeyText = GetKeyText();

		// FText::Format() is slow, so we cache this to save on performance
		if (!ModifierText.IsEmpty())
		{
			CachedTooltip.SetCachedText(FText::Format(LOCTEXT("InputDebugKey_Tooltip_Modifiers", "Events for when the {0} key is pressed or released while {1} is also held."), KeyText, ModifierText), this);
		}
		else
		{
			CachedTooltip.SetCachedText(FText::Format(LOCTEXT("InputDebugKey_Tooltip", "Events for when the {0} key is pressed or released."), KeyText), this);
		}
	}
	return CachedTooltip;
}

FSlateIcon UK2Node_InputDebugKey::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), EKeys::GetMenuCategoryPaletteIcon(InputKey.GetMenuCategory()));
}

bool UK2Node_InputDebugKey::IsCompatibleWithGraph(UEdGraph const* Graph) const
{
	// This node expands into event nodes and must be placed in a Ubergraph
	EGraphType const GraphType = Graph->GetSchema()->GetGraphType(Graph);
	bool bIsCompatible = (GraphType == EGraphType::GT_Ubergraph);

	if (bIsCompatible)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);

		UEdGraphSchema_K2 const* K2Schema = Cast<UEdGraphSchema_K2>(Graph->GetSchema());
		bool const bIsConstructionScript = (K2Schema != nullptr) ? UEdGraphSchema_K2::IsConstructionScript(Graph) : false;

		bIsCompatible = (Blueprint != nullptr) && Blueprint->SupportsInputEvents() && !bIsConstructionScript && Super::IsCompatibleWithGraph(Graph);
	}
	return bIsCompatible;
}

UEdGraphPin* UK2Node_InputDebugKey::GetPressedPin() const
{
	return FindPin(TEXT("Pressed"));
}

UEdGraphPin* UK2Node_InputDebugKey::GetReleasedPin() const
{
	return FindPin(TEXT("Released"));
}

UEdGraphPin* UK2Node_InputDebugKey::GetActionValuePin() const
{
	return FindPin(ActionPinName);
}

void UK2Node_InputDebugKey::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (!InputKey.IsValid() || InputKey == EKeys::AnyKey)
	{
		MessageLog.Warning(*FText::Format(LOCTEXT("Invalid_InputDebugKey_Warning", "InputDebugKey Event specifies invalid FKey'{0}' for @@"), FText::FromString(InputKey.ToString())).ToString(), this);
	}
	else if (!InputKey.IsBindableInBlueprints())
	{
		MessageLog.Warning( *FText::Format( LOCTEXT("NotBindanble_InputDebugKey_Warning", "InputDebugKey Event specifies FKey'{0}' that is not blueprint bindable for @@"), FText::FromString(InputKey.ToString())).ToString(), this);
	}
}

void UK2Node_InputDebugKey::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* InputDebugKeyPressedPin = GetPressedPin();
	UEdGraphPin* InputDebugKeyReleasedPin = GetReleasedPin();
	UEdGraphPin* ActionValuePin = GetActionValuePin();
	
	struct EventPinData
	{
		EventPinData(UEdGraphPin* InPin,TEnumAsByte<EInputEvent> InEvent ){	Pin=InPin;EventType=InEvent;};
		UEdGraphPin* Pin;
		TEnumAsByte<EInputEvent> EventType;
	};

	TArray<EventPinData> ActivePins;
	if(( InputDebugKeyPressedPin != nullptr ) && (InputDebugKeyPressedPin->LinkedTo.Num() > 0 ))
	{
		ActivePins.Add(EventPinData(InputDebugKeyPressedPin,IE_Pressed));
	}
	if((InputDebugKeyReleasedPin != nullptr) && (InputDebugKeyReleasedPin->LinkedTo.Num() > 0 ))
	{
		ActivePins.Add(EventPinData(InputDebugKeyReleasedPin,IE_Released));
	}

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	auto CreateDebugKeyEvent = [this, &CompilerContext, &SourceGraph](UEdGraphPin* Pin, EInputEvent EventType)
	{
		// Create the input touch event
		UK2Node_InputDebugKeyEvent* InputDebugKeyEvent = CompilerContext.SpawnIntermediateNode<UK2Node_InputDebugKeyEvent>(this, SourceGraph);
		const FName ModifierName = GetModifierName();
		if (ModifierName != NAME_None)
		{
			InputDebugKeyEvent->CustomFunctionName = FName(*FString::Printf(TEXT("InpActEvt_%s_%s_%s"), *ModifierName.ToString(), *InputKey.ToString(), *InputDebugKeyEvent->GetName()));
		}
		else
		{
			InputDebugKeyEvent->CustomFunctionName = FName(*FString::Printf(TEXT("InpActEvt_%s_%s"), *InputKey.ToString(), *InputDebugKeyEvent->GetName()));
		}
		InputDebugKeyEvent->InputChord.Key = InputKey;
		InputDebugKeyEvent->InputChord.bCtrl = bControl;
		InputDebugKeyEvent->InputChord.bAlt = bAlt;
		InputDebugKeyEvent->InputChord.bShift = bShift;
		InputDebugKeyEvent->InputChord.bCmd = bCommand;
		InputDebugKeyEvent->bExecuteWhenPaused = bExecuteWhenPaused;
		InputDebugKeyEvent->InputKeyEvent = EventType;
		InputDebugKeyEvent->EventReference.SetExternalDelegateMember(FName(TEXT("InputDebugKeyHandlerDynamicSignature__DelegateSignature")));
		InputDebugKeyEvent->bInternalEvent = true;
		InputDebugKeyEvent->AllocateDefaultPins();
		return InputDebugKeyEvent;
	};

	// If more than one is linked we have to do more complicated behaviors
	if( ActivePins.Num() > 1 )
	{
		// Create a temporary variable to copy Key in to
		static UScriptStruct* KeyStruct = FKey::StaticStruct();
		UK2Node_TemporaryVariable* KeyVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
		KeyVar->VariableType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		KeyVar->VariableType.PinSubCategoryObject = KeyStruct;
		KeyVar->AllocateDefaultPins();

		// Create a temporary variable to copy Action Value in to
		static UScriptStruct* ActionValueStruct = FInputActionValue::StaticStruct();
		UK2Node_TemporaryVariable* ActionValueVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
		ActionValueVar->VariableType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		ActionValueVar->VariableType.PinSubCategoryObject = ActionValueStruct;
		ActionValueVar->AllocateDefaultPins();

		for (auto PinIt = ActivePins.CreateIterator(); PinIt; ++PinIt)
		{
			UEdGraphPin *EachPin = (*PinIt).Pin;

			UK2Node_InputDebugKeyEvent* InputDebugKeyEvent = CreateDebugKeyEvent(EachPin, (*PinIt).EventType);

			// Create assignment nodes to assign the key
			UK2Node_AssignmentStatement* KeyInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
			KeyInitialize->AllocateDefaultPins();
			Schema->TryCreateConnection(KeyVar->GetVariablePin(), KeyInitialize->GetVariablePin());
			Schema->TryCreateConnection(KeyInitialize->GetValuePin(), InputDebugKeyEvent->FindPinChecked(TEXT("Key")));

			// Create Assignment nodes to assign the action value
			UK2Node_AssignmentStatement* ActionValueInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
			ActionValueInitialize->AllocateDefaultPins();
			Schema->TryCreateConnection(ActionValueVar->GetVariablePin(), ActionValueInitialize->GetVariablePin());
			Schema->TryCreateConnection(ActionValueInitialize->GetValuePin(), InputDebugKeyEvent->FindPinChecked(ActionPinName));
			
			// Connect the events to the assign key nodes
			Schema->TryCreateConnection(Schema->FindExecutionPin(*InputDebugKeyEvent, EGPD_Output), KeyInitialize->GetExecPin());
			Schema->TryCreateConnection(KeyInitialize->GetThenPin(), ActionValueInitialize->GetExecPin());
			
			// Move the original event connections to the then pin of the key assign
			CompilerContext.MovePinLinksToIntermediate(*EachPin, *ActionValueInitialize->GetThenPin());

			// Move the original event variable connections to the intermediate nodes
			CompilerContext.MovePinLinksToIntermediate(*FindPin(TEXT("Key")), *KeyVar->GetVariablePin());
			CompilerContext.MovePinLinksToIntermediate(*ActionValuePin, *ActionValueVar->GetVariablePin());
		}
	}
	else if( ActivePins.Num() == 1 )
	{
		UEdGraphPin* InputDebugKeyPin = ActivePins[0].Pin;

		if (InputDebugKeyPin->LinkedTo.Num() > 0)
		{
			UK2Node_InputDebugKeyEvent* InputDebugKeyEvent = CreateDebugKeyEvent(InputDebugKeyPin, ActivePins[0].EventType);

			CompilerContext.MovePinLinksToIntermediate(*InputDebugKeyPin, *Schema->FindExecutionPin(*InputDebugKeyEvent, EGPD_Output));
			CompilerContext.MovePinLinksToIntermediate(*FindPin(TEXT("Key")), *InputDebugKeyEvent->FindPin(TEXT("Key")));
			CompilerContext.MovePinLinksToIntermediate(*ActionValuePin, *InputDebugKeyEvent->FindPin(ActionPinName));
		}
	}

	// Widget blueprints require the bAutomaticallyRegisterInputOnConstruction to be set to true in order to receive callbacks
	CompilerContext.SetAutomaticallyRegisterInputEvents(true);
}

void UK2Node_InputDebugKey::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	TArray<FKey> AllKeys;
	EKeys::GetAllKeys(AllKeys);

	auto CustomizeInputNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FKey Key)
	{
		UK2Node_InputDebugKey* InputNode = CastChecked<UK2Node_InputDebugKey>(NewNode);
		InputNode->InputKey = Key;
	};

	auto RefreshClassActions = []()
	{
		FBlueprintActionDatabase::Get().RefreshClassActions(StaticClass());
	};

	// actions get registered under specific object-keys; the idea is that
	// actions might have to be updated (or deleted) if their object-key is
	// mutated (or removed)... here we use the node's class (so if the node
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();

	const bool bAllowGestures = GetDefault<UInputSettings>()->bEnableGestureRecognizer;
	
	// to keep from needlessly instantiating a UBlueprintNodeSpawner (and
	// iterating over keys), first check to make sure that the registrar is
	// looking for actions of this type (could be regenerating actions for a
	// specific asset, and therefore the registrar would only accept actions
	// corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		// Refresh the action database of this node when the input settings are changed
		static bool bRegisterOnce = true;
		if (bRegisterOnce)
		{
			bRegisterOnce = false;
			FEditorDelegates::OnEnableGestureRecognizerChanged.AddStatic(RefreshClassActions);
		}
		
		for (const FKey& Key : AllKeys)
		{
			// Do not show gesture keys if they are not enabled in the settings
			const bool bInvalidGestureKey = !bAllowGestures && Key.IsGesture();
			
			// AnyKey is not supported as a debug key, gestures can only be used if they are turned on in the settings
			if (!Key.IsBindableInBlueprints() || Key == EKeys::AnyKey || bInvalidGestureKey)
			{
				continue;
			}

			UBlueprintNodeSpawner* NodeSpawner = UInputDebugKeyEventNodeSpawner::Create(GetClass(), Key);
			check(NodeSpawner != nullptr);

			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, Key);
			ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
		}
	}
}

FText UK2Node_InputDebugKey::GetMenuCategory() const
{
	static TMap<FName, FNodeTextCache> CachedCategories;

	const FName KeyCategory = InputKey.GetMenuCategory();
	const FText SubCategoryDisplayName = FText::Format(LOCTEXT("DebugKeyEventsCategory", "Debug Events|{0} Events"), EKeys::GetMenuCategoryDisplayName(KeyCategory));
	FNodeTextCache& NodeTextCache = CachedCategories.FindOrAdd(KeyCategory);

	if (NodeTextCache.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		NodeTextCache.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Input, SubCategoryDisplayName), this);
	}
	return NodeTextCache;
}

FBlueprintNodeSignature UK2Node_InputDebugKey::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddKeyValue(InputKey.ToString());

	return NodeSignature;
}

TSharedPtr<FEdGraphSchemaAction> UK2Node_InputDebugKey::GetEventNodeAction(const FText& ActionCategory)
{
	TSharedPtr<FEdGraphSchemaAction_K2InputAction> EventNodeAction = MakeShareable(new FEdGraphSchemaAction_K2InputAction(ActionCategory, GetNodeTitle(ENodeTitleType::EditableTitle), GetTooltipText(), 0));
	EventNodeAction->NodeTemplate = this;
	return EventNodeAction;
}

#undef LOCTEXT_NAMESPACE

