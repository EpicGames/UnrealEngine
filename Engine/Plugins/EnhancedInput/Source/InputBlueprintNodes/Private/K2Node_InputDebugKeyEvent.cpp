// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InputDebugKeyEvent.h"
#include "InputDebugKeyDelegateBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InputDebugKeyEvent)

UK2Node_InputDebugKeyEvent::UK2Node_InputDebugKeyEvent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bInternalEvent = true;
}

UClass* UK2Node_InputDebugKeyEvent::GetDynamicBindingClass() const
{
	return UInputDebugKeyDelegateBinding::StaticClass();
}

void UK2Node_InputDebugKeyEvent::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UInputDebugKeyDelegateBinding* InputKeyBindingObject = CastChecked<UInputDebugKeyDelegateBinding>(BindingObject);

	FBlueprintInputDebugKeyDelegateBinding Binding;
	Binding.InputChord = InputChord;
	Binding.InputKeyEvent = InputKeyEvent;
	Binding.bExecuteWhenPaused = bExecuteWhenPaused;
	Binding.FunctionNameToBind = CustomFunctionName;

	InputKeyBindingObject->InputDebugKeyDelegateBindings.Add(Binding);
}

