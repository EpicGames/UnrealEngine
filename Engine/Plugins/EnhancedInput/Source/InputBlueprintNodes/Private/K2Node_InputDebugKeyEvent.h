// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/InputChord.h"
#include "K2Node_Event.h"
#include "K2Node_InputDebugKeyEvent.generated.h"

#define UE_API INPUTBLUEPRINTNODES_API

enum EInputEvent : int;

class UDynamicBlueprintBinding;

UCLASS(MinimalAPI)
class UK2Node_InputDebugKeyEvent : public UK2Node_Event
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FInputChord InputChord;

	UPROPERTY()
	TEnumAsByte<EInputEvent> InputKeyEvent;

	UPROPERTY()
	bool bExecuteWhenPaused = false;

	//~ Begin UK2Node Interface
	UE_API virtual UClass* GetDynamicBindingClass() const override;
	UE_API virtual void RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const override;
	//~ End UK2Node Interface
};

#undef UE_API
