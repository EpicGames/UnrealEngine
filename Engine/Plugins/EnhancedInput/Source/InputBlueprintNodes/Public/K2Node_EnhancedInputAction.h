// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNodeUtils.h"
#include "K2Node.h"
#include "K2Node_EventNodeInterface.h"
#include "BlueprintNodeSpawner.h"

#include "K2Node_EnhancedInputAction.generated.h"

#define UE_API INPUTBLUEPRINTNODES_API

class UInputAction;
enum class ETriggerEvent : uint8;
namespace ENodeTitleType { enum Type : int; }
struct FBlueprintNodeSignature;

class FBlueprintActionDatabaseRegistrar;
class FKismetCompilerContext;

/**
 * A node spawner which will check if there is already an enhanced input event node (UK2Node_EnhancedInputAction)
 * before spawning a new one in the graph. This makes the behavior of enhanced input action nodes the same as other
 * event nodes. 
 */
UCLASS(Transient)
class UInputActionEventNodeSpawner final : public UBlueprintNodeSpawner
{
	GENERATED_BODY()
public:
	static UInputActionEventNodeSpawner* Create(TSubclassOf<UEdGraphNode> const NodeClass, TObjectPtr<const UInputAction> InAction);
private:
	virtual UEdGraphNode* Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const override;

	UK2Node* FindExistingNode(const UBlueprint* Blueprint) const;

	// We don't want references to node spawners to be keeping any input action assets from GC
    // if you unload a plugin for example, so we keep it as a weak pointer.
	UPROPERTY()
	TWeakObjectPtr<const UInputAction> WeakActionPtr = nullptr;
};

UCLASS(MinimalAPI)
class UK2Node_EnhancedInputAction : public UK2Node, public IK2Node_EventNodeInterface
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<const UInputAction> InputAction;

	//~ Begin UEdGraphNode Interface.
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual bool IsCompatibleWithGraph(UEdGraph const* Graph) const override;
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	UE_API virtual void JumpToDefinition() const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface
	UE_API virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	UE_API virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual FText GetMenuCategory() const override;
	virtual bool CanUserEditPinAdvancedViewFlag() const override { return true; }
	UE_API virtual FBlueprintNodeSignature GetSignature() const override;
	UE_API virtual void PostReconstructNode();
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin);
	UE_API virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	//~ End UK2Node Interface

	//~ Begin IK2Node_EventNodeInterface Interface.
	UE_API virtual TSharedPtr<FEdGraphSchemaAction> GetEventNodeAction(const FText& ActionCategory) override;
	//~ End IK2Node_EventNodeInterface Interface.

	UE_API bool HasAnyConnectedEventPins() const;

private:
	UE_API FName GetActionName() const;
	UE_API void HideEventPins(UEdGraphPin* RetainPin);

	// Iterates each connected trigger event pin until Predicate returns false or we've iterated all active pins 
	UE_API void ForEachActiveEventPin(TFunctionRef<bool(ETriggerEvent, UEdGraphPin&)> Predicate) const;

	/** Gets the ETriggerEvent from an exec pin based on the Pins name. */
	UE_API const ETriggerEvent GetTriggerTypeFromExecPin(const UEdGraphPin* ExecPin) const;
	
	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;
};

#undef UE_API
