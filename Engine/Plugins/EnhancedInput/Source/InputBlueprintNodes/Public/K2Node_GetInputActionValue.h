// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNodeUtils.h"
#include "K2Node.h"
#include "K2Node_GetInputActionValue.generated.h"

class UInputAction;
namespace ENodeTitleType { enum Type : int; }
struct FBlueprintNodeSignature;

class FBlueprintActionDatabaseRegistrar;
class UDynamicBlueprintBinding;

UCLASS(MinimalAPI, meta=(Keywords = "Get, Input"))
class UK2Node_GetInputActionValue : public UK2Node
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<const UInputAction> InputAction;

	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual bool IsCompatibleWithGraph(UEdGraph const* Graph) const override;
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual void JumpToDefinition() const override;
	//~ End EdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual FBlueprintNodeSignature GetSignature() const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual bool IsNodePure() const override { return true; }
	//~ End UK2Node Interface

	void Initialize(const UInputAction* Action);

	// Auto pin generation helpers
	static FName GetValueCategory(const UInputAction* InputAction);
	static FName GetValueSubCategory(const UInputAction* InputAction);
	static UScriptStruct* GetValueSubCategoryObject(const UInputAction* InputAction);

private:
	FName GetActionName() const;

	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;
};
