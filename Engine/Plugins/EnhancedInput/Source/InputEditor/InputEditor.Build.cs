// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InputEditor : ModuleRules
	{
        public InputEditor(ReadOnlyTargetRules Target) : base(Target)
        {
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"EnhancedInput",
                }
            );

            PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AssetDefinition",
					"ApplicationCore",
					"BlueprintGraph",
                    "Core",
					"CoreUObject",
                    "DetailCustomizations",
                    "Engine",
                    "GraphEditor",
                    "InputCore",
					"KismetCompiler",
					"PropertyEditor",
                    "SharedSettingsWidgets",
                    "Slate",
                    "SlateCore",
                    "UnrealEd",
					"AssetTools",
					"DeveloperSettings",
					"EditorSubsystem",
					"ToolMenus",
					"ContentBrowser",
					"SourceControl",
					"GameplayTags",
                }
            );
        }
    }
}