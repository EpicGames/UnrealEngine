// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActionMappingDetails.h"
#include "InputMappingContext.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailDragDropHandler.h"
#include "IDetailGroup.h"
#include "Misc/PackageName.h"
#include "PropertyCustomizationHelpers.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Widgets/Views/STableRow.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "ActionMappingDetails"

// TODO: This is derived from (and will eventually replace) InputSettingsDetails.cpp

FActionMappingsNodeBuilderEx::FActionMappingsNodeBuilderEx(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	: ActionMappingsPropertyHandle(InPropertyHandle)
{
	// Support for updating references to renamed input actions
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	if (!AssetRegistry.OnAssetAdded().IsBoundToObject(this))
	{
		AssetRegistry.OnAssetRenamed().AddRaw(this, &FActionMappingsNodeBuilderEx::OnAssetRenamed);
	}
}

FActionMappingsNodeBuilderEx::~FActionMappingsNodeBuilderEx()
{
	// Unregister settings panel listeners
	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(FName("AssetRegistry")))
	{
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetRenamed().RemoveAll(this);
		}
	}
}

void FActionMappingsNodeBuilderEx::Tick(float DeltaTime)
{
	if (GroupsRequireRebuild())
	{
		RebuildChildren();
	}
	HandleDelayedGroupExpansion();
}

int32 FActionMappingsNodeBuilderEx::GetNumGroupedMappings() const
{
	return GroupedMappings.Num();	
}

int32 FActionMappingsNodeBuilderEx::GetNumMappings() const
{
	int32 NumMappings = 0;
	for (const ActionMappingDetails::FMappingSet& MappingSet : GroupedMappings)
	{
		NumMappings += MappingSet.Mappings.Num();
	}

	return NumMappings;
}

void FActionMappingsNodeBuilderEx::ReorderMappings(int32 OriginalIndex, int32 NewIndex, EReorderMode ReorderMode)
{
	TArray<UObject*> OuterObjects;
	ActionMappingsPropertyHandle->GetOuterObjects(OuterObjects);
	if (OuterObjects.IsEmpty())
	{
		return;
	}
	UInputMappingContext* InputContext = CastChecked<UInputMappingContext>(OuterObjects[0]);
	if (InputContext == nullptr)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("ReorderMapping_Transaction", "Move Mapping"));
	InputContext->Modify();
	ActionMappingsPropertyHandle->NotifyPreChange();

	DelayedGroupExpansionStates.Emplace(nullptr, true);

	TSharedPtr<IPropertyHandleArray> ActionMappingsArrayHandle = ActionMappingsPropertyHandle->AsArray();
	
	if (ReorderMode == EReorderMode::Group)
	{
		// The mapping array we get here can have different entries with the groups interleaved. The following algo will ensure to clean that, while applying the reordering, without changing the order inside a given category.

		//1: Get the list of groups in their current order, and assign them an index, with room to insert the moved group between any groups.
		TMap<const UObject*, uint32> Groups;
		uint32 GroupIndex = 1;
		
		uint32 NumMappings;
		ActionMappingsArrayHandle->GetNumElements(NumMappings);
		for (uint32 Index = 0; Index < NumMappings; ++Index)
		{
			TSharedRef<IPropertyHandle> ActionMapping = ActionMappingsArrayHandle->GetElement(Index);
			const UObject* Action;
			const FPropertyAccess::Result Result = ActionMapping->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, Action))->GetValue(Action);

			if (Result == FPropertyAccess::Success)
			{
				if (!Groups.Contains(Action))
				{
					Groups.Add(Action, GroupIndex);
					GroupIndex += 2;
				}
			}
		}

		//2: Compute a new index to insert at the right place.
		const UInputAction* ActionOfOriginalIndex = GroupedMappings[OriginalIndex].SharedAction;
		Groups[ActionOfOriginalIndex] = (NewIndex * 2 + ((OriginalIndex < NewIndex) ? 2 : 0));

		//3: Stable sort, to sort group in the new required order, but keep the order inside each category.
		TSharedPtr<IPropertyHandle> OuterStructContainerHandle = ActionMappingsPropertyHandle->GetParentHandle();
		if(OuterStructContainerHandle.IsValid())
		{
			void* StructData = nullptr;
			if(FPropertyAccess::Success == OuterStructContainerHandle->GetValueData(StructData))
			{
				FInputMappingContextMappingData* RawData = static_cast<FInputMappingContextMappingData*>(StructData);
				if (RawData)
				{
					RawData->Mappings.StableSort([&Groups](const FEnhancedActionKeyMapping& A, const FEnhancedActionKeyMapping& B) { return Groups[A.Action] < Groups[B.Action]; });
				}
			}
		}
	}
	else if (ReorderMode == EReorderMode::Single)
	{
		ActionMappingsArrayHandle->SwapItems(OriginalIndex, NewIndex);
	}
	else
	{
		checkf(false, TEXT("Unsupported value for enum EReorderMode!"));
	}

	ActionMappingsPropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
}

/** Drag-and-drop operation that stores data about the source input mapping being dragged */
class FInputMappingIndexDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FUserDefinedEnumIndexDragDropOp, FDecoratedDragDropOp);

	FInputMappingIndexDragDropOp(ActionMappingDetails::FMappingSet* InTargetMappingSet, int32 InInputMappingIndex, FActionMappingsNodeBuilderEx* InActionMappingNodeBuilder, bool InIsGroup)
		: TargetMappingSet(InTargetMappingSet)
		, InputMappingIndex(InInputMappingIndex)
		, ActionMappingNodeBuilder(InActionMappingNodeBuilder)
		, IsGroup(InIsGroup)
	{
		if (InTargetMappingSet->SharedAction)
		{
			InputMappingDisplayText = FText::FromName(InTargetMappingSet->SharedAction->GetFName());
		}
		else
		{
			InputMappingDisplayText = FText::FromString("None");
		}
		MouseCursor = EMouseCursor::GrabHandClosed;
	}

	void Init()
	{
		SetValidTarget(false);
		SetupDefaults();
		Construct();
	}

	void SetValidTarget(bool IsValidTarget)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("InputMappingName"), InputMappingDisplayText);

		if (IsValidTarget)
		{
			CurrentHoverText = FText::Format(LOCTEXT("MoveInputMappingHere", "Move '{InputMappingName}' Here"), Args);
			CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.OK");
		}
		else
		{
			CurrentHoverText = FText::Format(LOCTEXT("CannotMoveInputMappingHere", "Cannot Move '{InputMappingName}' Here"), Args);
			CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.Error");
		}
	}

	ActionMappingDetails::FMappingSet* GetMappingSet() const
	{
		return TargetMappingSet;
	}

	int32 GetInputMappingIndex() const
	{
		return InputMappingIndex;
	}

	bool GetIsGroup() const
	{
		return IsGroup;
	}

private:
	ActionMappingDetails::FMappingSet* TargetMappingSet;
	int32 InputMappingIndex;
	FActionMappingsNodeBuilderEx* ActionMappingNodeBuilder;
	FText InputMappingDisplayText;
	bool IsGroup;
};

/** Handler base class for customizing the drag-and-drop behavior for input mappings (single or in group), allowing them to be reordered */
class FInputMappingIndexDragDropHandlerBase : public IDetailDragDropHandler
{
public:
	FInputMappingIndexDragDropHandlerBase(ActionMappingDetails::FMappingSet* InTargetMappingSet, int32 InTargetMappingIndex, FActionMappingsNodeBuilderEx* InActionMappingNodeBuilder, bool InIsGroup)
		: TargetInputMappingSet(InTargetMappingSet)
		, InputMappingIndex(InTargetMappingIndex)
		, ActionMappingNodeBuilder(InActionMappingNodeBuilder)
		, IsGroup(InIsGroup)
	{
		check(TargetInputMappingSet);
		check(ActionMappingNodeBuilder);
	}

	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation() const override
	{
		TSharedPtr<FInputMappingIndexDragDropOp> DragOp = MakeShared<FInputMappingIndexDragDropOp>(TargetInputMappingSet, InputMappingIndex, ActionMappingNodeBuilder, IsGroup);
		DragOp->Init();
		return DragOp;
	}

	virtual bool AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override
	{
		const TSharedPtr<FInputMappingIndexDragDropOp> DragOp = DragDropEvent.GetOperationAs<FInputMappingIndexDragDropOp>();
		if (!DragOp.IsValid() || DropZone == EItemDropZone::OntoItem)
		{
			return false;
		}

		const int32 NewIndex = ComputeNewIndex(DragOp->GetInputMappingIndex(), InputMappingIndex, DropZone);
		ActionMappingNodeBuilder->ReorderMappings(DragOp->GetInputMappingIndex(), NewIndex, IsGroup ? FActionMappingsNodeBuilderEx::EReorderMode::Group: FActionMappingsNodeBuilderEx::EReorderMode::Single);
		return true;
	}

protected:

	/** Compute new target index based on drop zone (above vs below) */
	static int32 ComputeNewIndex(int32 OriginalIndex, int32 DropOntoIndex, EItemDropZone DropZone)
	{
		check(DropZone != EItemDropZone::OntoItem);

		int32 NewIndex = DropOntoIndex;
		if (DropZone == EItemDropZone::BelowItem)
		{
			// If the drop zone is below, then we actually move it to the next item's index
			NewIndex++;
		}
		if (OriginalIndex < NewIndex)
		{
			// If the item is moved down the list, then all the other elements below it are shifted up one
			NewIndex--;
		}

		return ensure(NewIndex >= 0) ? NewIndex : 0;
	}

	ActionMappingDetails::FMappingSet* TargetInputMappingSet;
	int32 InputMappingIndex;
	FActionMappingsNodeBuilderEx* ActionMappingNodeBuilder;
	bool IsGroup;
};

/** Handler for customizing the drag-and-drop behavior for input grouped mappings, allowing them to be reordered */
class FInputGroupedMappingIndexDragDropHandler : public FInputMappingIndexDragDropHandlerBase
{
public:
	FInputGroupedMappingIndexDragDropHandler(ActionMappingDetails::FMappingSet* InTargetMappingSet, int32 InTargetMappingIndex, FActionMappingsNodeBuilderEx* InActionMappingNodeBuilder)
		: FInputMappingIndexDragDropHandlerBase(InTargetMappingSet, InTargetMappingIndex, InActionMappingNodeBuilder, /*InIsGroup*/true)
	{
		check(InputMappingIndex >= 0 && InputMappingIndex < ActionMappingNodeBuilder->GetNumGroupedMappings());
	}

	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override
	{
		const TSharedPtr<FInputMappingIndexDragDropOp> DragOp = DragDropEvent.GetOperationAs<FInputMappingIndexDragDropOp>();
		if (!DragOp.IsValid())
		{
			return TOptional<EItemDropZone>();
		}

		if (!DragOp->GetIsGroup())
		{
			return TOptional<EItemDropZone>();
		}

		// We're reordering, so there's no logical interpretation for dropping directly onto another item.
		// Just change it to a drop-above in this case.
		const EItemDropZone OverrideZone = (DropZone == EItemDropZone::BelowItem) ? EItemDropZone::BelowItem : EItemDropZone::AboveItem;
		const int32 NewIndex = ComputeNewIndex(DragOp->GetInputMappingIndex(), InputMappingIndex, OverrideZone);

		// Make sure that the new index is valid *and* that it represents an actual move from the current position.
		if (NewIndex < 0 || NewIndex >= ActionMappingNodeBuilder->GetNumGroupedMappings() || NewIndex == DragOp->GetInputMappingIndex())
		{
			return TOptional<EItemDropZone>();
		}

		DragOp->SetValidTarget(true);
		return OverrideZone;
	}
};

/** Handler for customizing the drag-and-drop behavior for input mappings, allowing them to be reordered */
class FInputMappingIndexDragDropHandler : public FInputMappingIndexDragDropHandlerBase
{
public:
	FInputMappingIndexDragDropHandler(ActionMappingDetails::FMappingSet* InParentTargetMappingSet, int32 InTargetMappingIndex, FActionMappingsNodeBuilderEx* InActionMappingNodeBuilder)
		: FInputMappingIndexDragDropHandlerBase(InParentTargetMappingSet, InTargetMappingIndex, InActionMappingNodeBuilder, /*InIsGroup*/false)
	{
		check(InputMappingIndex >= 0 && InputMappingIndex < ActionMappingNodeBuilder->GetNumMappings());
	}

	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override
	{
		const TSharedPtr<FInputMappingIndexDragDropOp> DragOp = DragDropEvent.GetOperationAs<FInputMappingIndexDragDropOp>();
		if (!DragOp.IsValid())
		{
			return TOptional<EItemDropZone>();
		}

		if (DragOp->GetIsGroup())
		{
			return TOptional<EItemDropZone>();
		}

		//Forbid drag n drop between input mapping owned different parents.
		if (DragOp->GetMappingSet() != TargetInputMappingSet)
		{
			return TOptional<EItemDropZone>();
		}

		// We're reordering, so there's no logical interpretation for dropping directly onto another item.
		// Just change it to a drop-above in this case.
		const EItemDropZone OverrideZone = (DropZone == EItemDropZone::BelowItem) ? EItemDropZone::BelowItem : EItemDropZone::AboveItem;
		const int32 NewIndex = ComputeNewIndex(DragOp->GetInputMappingIndex(), InputMappingIndex, OverrideZone);

		// Make sure that the new index is valid *and* that it represents an actual move from the current position.
		if (NewIndex < 0 || NewIndex >= ActionMappingNodeBuilder->GetNumMappings() || NewIndex == DragOp->GetInputMappingIndex())
		{
			return TOptional<EItemDropZone>();
		}

		DragOp->SetValidTarget(true);
		return OverrideZone;
	}
};

void FActionMappingsNodeBuilderEx::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	TSharedRef<SWidget> AddButton = PropertyCustomizationHelpers::MakeAddButton(
		FSimpleDelegate::CreateSP(this, &FActionMappingsNodeBuilderEx::AddActionMappingButton_OnClick),
		TAttribute<FText>(this, &FActionMappingsNodeBuilderEx::GetAddNewActionTooltip),
		TAttribute<bool>(this, &FActionMappingsNodeBuilderEx::CanAddNewActionMapping));

	TSharedRef<SWidget> ClearButton = PropertyCustomizationHelpers::MakeEmptyButton(FSimpleDelegate::CreateSP(this, &FActionMappingsNodeBuilderEx::ClearActionMappingButton_OnClick),
		LOCTEXT("ClearActionMappingToolTip", "Removes all Action Mappings"));

	FSimpleDelegate RebuildChildrenDelegate = FSimpleDelegate::CreateSP(this, &FActionMappingsNodeBuilderEx::RebuildChildren);
	ActionMappingsPropertyHandle->SetOnPropertyValueChanged(RebuildChildrenDelegate);
	ActionMappingsPropertyHandle->AsArray()->SetOnNumElementsChanged(RebuildChildrenDelegate);

	FUIAction CopyAction;
	FUIAction PasteAction;
	ActionMappingsPropertyHandle->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);

	NodeRow
	.CopyAction(CopyAction)
	.PasteAction(PasteAction)
	.FilterString(ActionMappingsPropertyHandle->GetPropertyDisplayName())
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			ActionMappingsPropertyHandle->CreatePropertyNameWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			AddButton
		]
		+ SHorizontalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			ClearButton
		]
	];
}

bool FActionMappingsNodeBuilderEx::CanAddNewActionMapping() const
{
	// If the last action mapping the user has added is still null, then do not allow adding another one
	TSharedPtr<IPropertyHandleArray> ActionMappingsArrayHandle = ActionMappingsPropertyHandle->AsArray();

	uint32 NumMappings;
	ActionMappingsArrayHandle->GetNumElements(NumMappings);

	if(NumMappings > 0)
	{
		TSharedRef<IPropertyHandle> ActionMapping = ActionMappingsArrayHandle->GetElement(NumMappings - 1);
		const UObject* Action;
		FPropertyAccess::Result Result = ActionMapping->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, Action))->GetValue(Action);
		return Result == FPropertyAccess::Success && Action;
	}

	// If there are no mappings, then the user is allowed to add one
	return true;
}

FText FActionMappingsNodeBuilderEx::GetAddNewActionTooltip() const
{
	if(CanAddNewActionMapping())
	{
		return LOCTEXT("AddActionMappingToolTip_Enabled", "Adds Action Mapping");
	}
	else
	{
		return LOCTEXT("AddActionMappingToolTip_Disabled", "Cannot add an action mapping while an empty mapping exists");
	}
}

void FActionMappingsNodeBuilderEx::OnAssetRenamed(const FAssetData& AssetData, const FString& OldName)
{
	// If this is an Input Action asset, then we need to check for any references to it and replace them
	if (AssetData.GetClass() == UInputAction::StaticClass())
	{
		const FName OldPackageName = *FPackageName::ObjectPathToPackageName(OldName);
		ActionsBeingRenamed.Add(AssetData.PackageName, OldPackageName);
	}
}

void FActionMappingsNodeBuilderEx::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	RebuildGroupedMappings();

	int32 GlobalIndex = 0;

	for (int32 Index = 0; Index < GroupedMappings.Num(); ++Index)
	{
		ActionMappingDetails::FMappingSet& MappingSet = GroupedMappings[Index];

		FString GroupNameString(TEXT("ActionMappings."));
		GroupNameString += MappingSet.SharedAction->GetPathName();
		FName GroupName(*GroupNameString);
		IDetailGroup& ActionMappingGroup = ChildrenBuilder.AddGroup(GroupName, FText::FromString(MappingSet.SharedAction->GetPathName()));
		MappingSet.DetailGroup = &ActionMappingGroup;

		TSharedRef<SWidget> AddButton = PropertyCustomizationHelpers::MakeAddButton(FSimpleDelegate::CreateSP(this, &FActionMappingsNodeBuilderEx::AddActionMappingToGroupButton_OnClick, MappingSet),
			LOCTEXT("AddActionMappingToGroupToolTip", "Add a control binding to the Action Mapping"));

		TSharedRef<SWidget> RemoveButton = PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateSP(this, &FActionMappingsNodeBuilderEx::RemoveActionMappingGroupButton_OnClick, MappingSet),
			LOCTEXT("RemoveActionMappingGroupToolTip", "Remove the Action Mapping Group"));

		ActionMappingGroup.HeaderRow()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(ActionMappingDetails::InputConstants::PropertyPadding)
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(ActionMappingDetails::InputConstants::TextBoxWidth)
				[
					SNew(SObjectPropertyEntryBox)
					.AllowedClass(UInputAction::StaticClass())
					.ObjectPath(MappingSet.SharedAction ? MappingSet.SharedAction->GetPathName() : FString())
					.DisplayUseSelected(true)
					.OnObjectChanged(this, &FActionMappingsNodeBuilderEx::OnActionMappingActionChanged, MappingSet)
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(ActionMappingDetails::InputConstants::PropertyPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				AddButton
			]
			+ SHorizontalBox::Slot()
			.Padding(ActionMappingDetails::InputConstants::PropertyPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				RemoveButton
			]
		]
		.DragDropHandler(MakeShared<FInputGroupedMappingIndexDragDropHandler>(&GroupedMappings[Index], Index, this));

		for (int32 MappingIndex = 0; MappingIndex < MappingSet.Mappings.Num(); ++MappingIndex)
		{
			ActionMappingGroup.AddPropertyRow(MappingSet.Mappings[MappingIndex])
			.ShowPropertyButtons(false)
			.DragDropHandler(MakeShared<FInputMappingIndexDragDropHandler>(&GroupedMappings[Index], GlobalIndex, this));

			++GlobalIndex;
		}
	}
}

void FActionMappingsNodeBuilderEx::AddActionMappingButton_OnClick()
{
	static const FName BaseActionMappingName(*LOCTEXT("NewActionMappingName", "NewActionMapping").ToString());
	static int32 NewMappingCount = 0;
	const FScopedTransaction Transaction(LOCTEXT("AddActionMapping_Transaction", "Add Action Mapping"));

	TArray<UObject*> OuterObjects;
	ActionMappingsPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() == 1)
	{
		UInputMappingContext* InputContext = CastChecked<UInputMappingContext>(OuterObjects[0]);
		InputContext->Modify();
		ActionMappingsPropertyHandle->NotifyPreChange();

		DelayedGroupExpansionStates.Emplace(nullptr, true);		
		// Add the new mapping to the array...
		TSharedPtr<IPropertyHandleArray> ArrayHandle = ActionMappingsPropertyHandle->AsArray();
		const FPropertyHandleItemAddResult Res = ArrayHandle->AddItem();

		ActionMappingsPropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
	}
}

void FActionMappingsNodeBuilderEx::ClearActionMappingButton_OnClick()
{
	ActionMappingsPropertyHandle->AsArray()->EmptyArray();
}

void FActionMappingsNodeBuilderEx::OnActionMappingActionChanged(const FAssetData& AssetData, const ActionMappingDetails::FMappingSet MappingSet)
{
	const FScopedTransaction Transaction(LOCTEXT("SwitchActionMapping_Transaction", "Switch Action Mapping"));

	const UInputAction* SelectedAction = Cast<const UInputAction>(AssetData.GetAsset());

	const UObject* CurrentAction = nullptr;
	if (MappingSet.Mappings.Num() > 0)
	{
		MappingSet.Mappings[0]->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, Action))->GetValue(CurrentAction);
	}

	if (SelectedAction != CurrentAction)
	{
		// If the IMC already has a mapping to this Input Action, then all of it's mappings will be moved.
		// This can be jarring for the user if they are working with a complex IMC, so we will put a
		// small toast up to tell them about it if we can
		{
			bool bHasExistingAction = false;
				
			for (int32 Index = 0; Index < MappingSet.Mappings.Num(); ++Index)
			{
				const UObject* Action;
				MappingSet.Mappings[Index]->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, Action))->GetValue(Action);
				if (Action == SelectedAction)
				{
					bHasExistingAction = true;
					break;
				}
			}
			
			if (bHasExistingAction)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("ExistingAction"), FText::FromString(*GetNameSafe(SelectedAction)));
				const FText NotifText = FText::Format(LOCTEXT("CombiningInputActionMappings_Notif", "'{ExistingAction}' already has mappings in this context, combining them!"), Args);
				
				FNotificationInfo Notif(NotifText);
				Notif.ExpireDuration = 3.0f;
				FSlateNotificationManager::Get().AddNotification(Notif);						
			}
		}
		
		for (int32 Index = 0; Index < MappingSet.Mappings.Num(); ++Index)
		{
			MappingSet.Mappings[Index]->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, Action))->SetValue(SelectedAction);
		}

		if (MappingSet.DetailGroup)
		{
			DelayedGroupExpansionStates.Emplace(SelectedAction, MappingSet.DetailGroup->GetExpansionState());

			// Don't want to save expansion state of old asset
			MappingSet.DetailGroup->ToggleExpansion(false);
		}
	}
}

void FActionMappingsNodeBuilderEx::AddActionMappingToGroupButton_OnClick(const ActionMappingDetails::FMappingSet MappingSet)
{
	const FScopedTransaction Transaction(LOCTEXT("AddActionMappingToGroup_Transaction", "Add a control binding to the Action Mapping"));

	TArray<UObject*> OuterObjects;
	ActionMappingsPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() == 1)
	{
		UInputMappingContext* InputContext = CastChecked<UInputMappingContext>(OuterObjects[0]);
		InputContext->Modify();
		ActionMappingsPropertyHandle->NotifyPreChange();

		DelayedGroupExpansionStates.Emplace(MappingSet.SharedAction, true);

		// Create a new FEnhancedActionKeyMapping entry in the mappings array, and set its default Input Action
		// value to the UInputAction that we would like.
		{
			// Add the item to the array...
			TSharedPtr<IPropertyHandleArray> ArrayHandle = ActionMappingsPropertyHandle->AsArray();
			const FPropertyHandleItemAddResult Res = ArrayHandle->AddItem();

			// Initalize the UInputAction property...
			TSharedPtr<IPropertyHandle> NewMappingInputActionProperty = ArrayHandle->GetElement(Res.GetIndex())->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, Action));
			NewMappingInputActionProperty->SetValue(MappingSet.SharedAction);
		}
		ActionMappingsPropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
	}
}

void FActionMappingsNodeBuilderEx::RemoveActionMappingGroupButton_OnClick(const ActionMappingDetails::FMappingSet MappingSet)
{
	const FScopedTransaction Transaction(LOCTEXT("RemoveActionMappingGroup_Transaction", "Remove Action Mapping and all control bindings"));

	TSharedPtr<IPropertyHandleArray> ActionMappingsArrayHandle = ActionMappingsPropertyHandle->AsArray();

	TArray<int32> SortedIndices;
	for (int32 Index = 0; Index < MappingSet.Mappings.Num(); ++Index)
	{
		SortedIndices.AddUnique(MappingSet.Mappings[Index]->GetIndexInArray());
	}
	SortedIndices.Sort();

	for (int32 Index = SortedIndices.Num() - 1; Index >= 0; --Index)
	{
		ActionMappingsArrayHandle->DeleteItem(SortedIndices[Index]);
	}
}

bool FActionMappingsNodeBuilderEx::GroupsRequireRebuild() const
{
	// If any input actions have been renamed, then we need to update the details
	if (!ActionsBeingRenamed.IsEmpty())
	{
		return true;
	}
	
	for (int32 GroupIndex = 0; GroupIndex < GroupedMappings.Num(); ++GroupIndex)
	{
		const ActionMappingDetails::FMappingSet& MappingSet = GroupedMappings[GroupIndex];
		for (int32 MappingIndex = 0; MappingIndex < MappingSet.Mappings.Num(); ++MappingIndex)
		{
			const UObject* Action;
			MappingSet.Mappings[MappingIndex]->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, Action))->GetValue(Action);
			if (MappingSet.SharedAction != Action)
			{
				return true;
			}
		}
	}
	return false;
}

void FActionMappingsNodeBuilderEx::RebuildGroupedMappings()
{
	GroupedMappings.Empty();
	ActionsBeingRenamed.Empty();

	TSharedPtr<IPropertyHandleArray> ActionMappingsArrayHandle = ActionMappingsPropertyHandle->AsArray();

	uint32 NumMappings;
	ActionMappingsArrayHandle->GetNumElements(NumMappings);
	for (uint32 Index = 0; Index < NumMappings; ++Index)
	{
		TSharedRef<IPropertyHandle> ActionMapping = ActionMappingsArrayHandle->GetElement(Index);
		const UObject* Action;
		FPropertyAccess::Result Result = ActionMapping->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, Action))->GetValue(Action);

		if (Result == FPropertyAccess::Success)
		{
			int32 FoundIndex = INDEX_NONE;
			for (int32 GroupIndex = 0; GroupIndex < GroupedMappings.Num(); ++GroupIndex)
			{
				if (GroupedMappings[GroupIndex].SharedAction == Action)
				{
					FoundIndex = GroupIndex;
					break;
				}
			}
			if (FoundIndex == INDEX_NONE)
			{
				FoundIndex = GroupedMappings.Num();
				GroupedMappings.AddZeroed();
				GroupedMappings[FoundIndex].SharedAction = Cast<const UInputAction>(Action);
			}
			GroupedMappings[FoundIndex].Mappings.Add(ActionMapping);
		}
	}
}

void FActionMappingsNodeBuilderEx::HandleDelayedGroupExpansion()
{
	if (DelayedGroupExpansionStates.Num() > 0)
	{
		for (auto GroupState : DelayedGroupExpansionStates)
		{
			for (auto& MappingSet : GroupedMappings)
			{
				if (MappingSet.SharedAction == GroupState.Key)
				{
					MappingSet.DetailGroup->ToggleExpansion(GroupState.Value);
					break;
				}
			}
		}
		DelayedGroupExpansionStates.Empty();
	}
}

#undef LOCTEXT_NAMESPACE
