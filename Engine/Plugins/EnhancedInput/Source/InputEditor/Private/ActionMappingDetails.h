// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Margin.h"
#include "IDetailCustomNodeBuilder.h"

// TODO: This is derived from (and will eventually replace) InputSettingsDetails.h

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IDetailGroup;
class IDetailLayoutBuilder;
class UInputAction;

namespace ActionMappingDetails
{
	namespace InputConstants
	{
		const FMargin PropertyPadding(2.0f, 0.0f, 2.0f, 0.0f);
		const float TextBoxWidth = 250.0f;
		const float ScaleBoxWidth = 50.0f;
	}

	struct FMappingSet
	{
		const class UInputAction* SharedAction;
		IDetailGroup* DetailGroup;
		TArray<TSharedRef<IPropertyHandle>> Mappings;
	};
}

class FActionMappingsNodeBuilderEx : public IDetailCustomNodeBuilder, public TSharedFromThis<FActionMappingsNodeBuilderEx>
{
public:
	FActionMappingsNodeBuilderEx(const TSharedPtr<IPropertyHandle>& InPropertyHandle);
	virtual ~FActionMappingsNodeBuilderEx() override;

	/** IDetailCustomNodeBuilder interface */
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRebuildChildren) override { OnRebuildChildren = InOnRebuildChildren; }
	virtual bool RequiresTick() const override { return true; }
	virtual void Tick(float DeltaTime) override;
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual bool InitiallyCollapsed() const override { return true; };
	virtual FName GetName() const override { return FName(TEXT("ActionMappings")); }

	int32 GetNumGroupedMappings() const;
	int32 GetNumMappings() const;

	enum class EReorderMode : uint8
	{
		Group,
		Single
	};
	void ReorderMappings(int32 OriginalIndex, int32 NewIndex, EReorderMode ReorderMode);
private:
	void AddActionMappingButton_OnClick();
	void ClearActionMappingButton_OnClick();
	void OnActionMappingActionChanged(const FAssetData& AssetData, const ActionMappingDetails::FMappingSet MappingSet);
	void AddActionMappingToGroupButton_OnClick(const ActionMappingDetails::FMappingSet MappingSet);
	void RemoveActionMappingGroupButton_OnClick(const ActionMappingDetails::FMappingSet MappingSet);

	bool GroupsRequireRebuild() const;
	void RebuildGroupedMappings();
	void RebuildChildren()
	{
		OnRebuildChildren.ExecuteIfBound();
	}
	/** Makes sure that groups have their expansion set after any rebuilding */
	void HandleDelayedGroupExpansion();

	/** Returns true if the "Adds Action Mapping" button can be pressed */
	bool CanAddNewActionMapping() const;

	/** The tooltip attribute for the "Adds Action Mapping" button */
	FText GetAddNewActionTooltip() const;

	/**
	 * Called by the asset registry when an asset is renamed. Replaces any references to the old Input Action with
	 * the new one
	 */
	void OnAssetRenamed(const FAssetData& AssetData, const FString& OldName);

private:
	/** Called to rebuild the children of the detail tree */
	FSimpleDelegate OnRebuildChildren;

	/** Property handle to associated Action Mappings */
	TSharedPtr<IPropertyHandle> ActionMappingsPropertyHandle;

	TArray<ActionMappingDetails::FMappingSet> GroupedMappings;

	TArray<TPair<const UInputAction*, bool>> DelayedGroupExpansionStates;

	/** A map of Input Action's that have been renamed. This map is a map of the New Name to the Old Name. */
	TMap<FName, FName> ActionsBeingRenamed;
};
