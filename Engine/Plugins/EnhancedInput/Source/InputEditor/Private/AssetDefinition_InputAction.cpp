// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_InputAction.h"
#include "AssetToolsModule.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "InputEditorModule.h"
#include "InputMappingContext.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_InputAction"

namespace MenuExtension::InputAction
{
	void GetCreateContextFromActionsMenu(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return;
		}

		TArray<UInputAction*> Actions = CBContext->LoadSelectedObjects<UInputAction>();
		if (Actions.IsEmpty())
		{
			return;
		}

		static const FString MappingContextPrefix(TEXT("IMC_"));
		static const FString ActionPrefixToIgnore(TEXT("IA_"));
	
		if (UInputAction* FirstIA = Actions[0])
		{
			FString EffectiveIMCName = FirstIA->GetName();
			EffectiveIMCName.RemoveFromStart(ActionPrefixToIgnore);
			// Have the IMC_ prefix at the front of the new asset
			EffectiveIMCName.InsertAt(0, MappingContextPrefix);
		
			const FString ActionPathName = FirstIA->GetOutermost()->GetPathName();
			const FString LongPackagePath = FPackageName::GetLongPackagePath(ActionPathName);
			const FString NewIMCDefaultPath = LongPackagePath / EffectiveIMCName;

			FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		
			// Make sure the name is unique
			FString AssetName;
			FString DefaultSuffix;
			FString PackageName;
			AssetToolsModule.Get().CreateUniqueAssetName(NewIMCDefaultPath, DefaultSuffix, /*out*/ PackageName, /*out*/ AssetName);
			const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

			// Create the new IMC
			UInputMappingContext_Factory* IMCFactory = NewObject<UInputMappingContext_Factory>();
			const TArray<TWeakObjectPtr<UInputAction>> WeakActionsPointers(Actions);
			IMCFactory->SetInitialActions(WeakActionsPointers);
			ContentBrowserModule.Get().CreateNewAsset(AssetName, PackagePath, UInputMappingContext::StaticClass(), IMCFactory);
		}
	}
	
		static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			
			// Input Action Action Registration
			{
				UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UInputAction::StaticClass());

				FToolMenuSection& InputSection = Menu->FindOrAddSection("GetAssetActions");

				InputSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
					{
						if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
						{
							if (Context->SelectedAssets.Num() > 0)
							{
								const FAssetData& AssetData = Context->SelectedAssets[0];
								if (AssetData.AssetClassPath == UInputAction::StaticClass()->GetClassPathName())
								{
									// Create Context From Selection
									{
										const TAttribute<FText> Label = LOCTEXT("InputAction_CreateContextFromSelection", "Create an Input Mapping Context");
										const TAttribute<FText> ToolTip = LOCTEXT("InputAction_CreateContextFromSelectionTooltip", "Create an Input Mapping Context that is filled with the selected Input Actions.");
										const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.InputMappingContext");

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&GetCreateContextFromActionsMenu);
										InSection.AddMenuEntry("InputAction_CreateContextFromSelection", Label, ToolTip, Icon, UIAction);
									}
								}
							}
						}
					}));
			}
		}));
	});
}

#undef LOCTEXT_NAMESPACE
