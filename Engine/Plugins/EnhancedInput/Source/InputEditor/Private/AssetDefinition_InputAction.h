// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "InputAction.h"
#include "AssetDefinition_InputAction.generated.h"

UCLASS()
class UAssetDefinition_InputAction : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_InputAction", "Input Action"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(127, 255, 255); }
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_InputActionDesc", "Represents an abstract game action that can be mapped to arbitrary hardware input devices."); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UInputAction::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Input, NSLOCTEXT("AssetDefinition", "AssetDefinition_InputActionSubMenu", "Enhanced Input"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
