// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "InputMappingContext.h"
#include "AssetDefinition_InputMappingContext.generated.h"

UCLASS()
class UAssetDefinition_InputMappingContext : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_InputMappingContext", "Input Mapping Context"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(255, 255, 127); }
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_InputMappingContextDesc", "A collection of device input to action mappings."); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UInputMappingContext::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Input, NSLOCTEXT("AssetDefinition", "AssetDefinition_InputMappingContextSubMenu", "Enhanced Input"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
