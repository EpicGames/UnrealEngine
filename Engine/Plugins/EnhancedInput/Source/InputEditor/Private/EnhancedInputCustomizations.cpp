// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputCustomizations.h"

#include "ActionMappingDetails.h"
#include "AssetRegistry/ARFilter.h"
#include "DetailLayoutBuilder.h"
#include "Engine/Blueprint.h"
#include "IDetailChildrenBuilder.h"
#include "InputMappingContext.h"
#include "KeyStructCustomization.h"
#include "PropertyCustomizationHelpers.h"
#include "EnhancedInputDeveloperSettings.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/BlueprintSupport.h"
#include "Misc/PackageName.h"
#include "UObject/UObjectIterator.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Text/STextBlock.h"
#include "EnhancedInputPlayerMappableNameValidator.h"
#include "PlayerMappableKeySettings.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "EnhancedInputPlayerMappableNameValidator.h"
#include "InputEditorModule.h"

#define LOCTEXT_NAMESPACE "InputCustomization"


//////////////////////////////////////////////////////////
// FEnhancedActionMappingCustomization

const FEnhancedActionKeyMapping* GetActionKeyMapping(TSharedPtr<IPropertyHandle> MappingPropertyHandle)
{
	if (MappingPropertyHandle->IsValidHandle())
	{
		void* Data = nullptr;
		if (MappingPropertyHandle->GetValueData(Data) != FPropertyAccess::Fail)
		{
			return static_cast<FEnhancedActionKeyMapping*>(Data);
		}
	}

	return nullptr;
}

void FEnhancedActionMappingCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	MappingPropertyHandle = PropertyHandle;

	// Grab the FKey property
	TSharedPtr<IPropertyHandle> KeyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, Key));
	TSharedPtr<IPropertyHandle> TriggersHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, Triggers));

	TriggersHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FEnhancedActionMappingCustomization::OnTriggersChanged));
	TriggersHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FEnhancedActionMappingCustomization::OnTriggersChanged));
	TSharedRef<SWidget> RemoveButton = PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateSP(this, &FEnhancedActionMappingCustomization::RemoveMappingButton_OnClick),
		LOCTEXT("RemoveMappingToolTip", "Remove Mapping"));

	// Create  a new instance of the key customization.
	KeyStructInstance = FKeyStructCustomization::MakeInstance();

	// TODO: Use FDetailArrayBuilder?

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bContainsComboTrigger = DoesTriggerArrayContainCombo();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	// Pass our header row into the key struct customizeheader method so it populates our row with the key struct header
	KeyStructCustomization = StaticCastSharedPtr<FKeyStructCustomization>(KeyStructInstance);
	KeyStructCustomization->SetDefaultKeyName("ComboKey");
	KeyStructCustomization->SetDisabledKeySelectorToolTip(LOCTEXT("DisabledSelectorComboToolTip", "A Combo Trigger isn't triggered through a key so Key Selection has been disabled."));
	KeyStructCustomization->SetEnableKeySelector(!bContainsComboTrigger);
	KeyStructCustomization->CustomizeHeaderOnlyWithButton(KeyHandle.ToSharedRef(), HeaderRow, CustomizationUtils, RemoveButton);
}

void FEnhancedActionMappingCustomization::AddInputActionProperties(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder)
{
    if (const FEnhancedActionKeyMapping* ActionKeyMapping = GetActionKeyMapping(PropertyHandle))
    {
    	// Make sure ActionKeyMapping action is valid. If triggers and modifiers are empty we can just back out - nothing to add
    	if (ActionKeyMapping->Action)
    	{
    		// Convert InputAction to non const so we can properly use it in the UObject Array for AddExternalObjectProperty
    		const UInputAction* InputActionPtr = ActionKeyMapping->Action;
    		UInputAction* InputAction = const_cast<UInputAction*>(InputActionPtr);
    		TArray<UObject*> ActionsAsUObjects { InputAction };

    		if (IDetailPropertyRow* InputActionTriggersRow = ChildBuilder.AddExternalObjectProperty(ActionsAsUObjects, GET_MEMBER_NAME_CHECKED(UInputAction, Triggers)))
    		{
    			InputActionTriggersPropertyRow = InputActionTriggersRow;
    			InputActionTriggersRow->DisplayName(LOCTEXT("InputActionTriggersDisplayName", "Triggers From Input Action"));
    			InputActionTriggersRow->ToolTip(FText::Format(LOCTEXT("InputActionTriggersToolTip", "Triggers from the {0} Input Action"), FText::FromName(ActionKeyMapping->Action.GetFName())));
    			InputActionTriggersRow->IsEnabled(false);
    			InputAction->OnTriggersChanged.AddSP(this, &FEnhancedActionMappingCustomization::OnInputActionTriggersChanged);
    			
    			// hide it if the trigger array is empty
    			if (ActionKeyMapping->Action->Triggers.IsEmpty())
    			{
					InputActionTriggersRow->Visibility(EVisibility::Hidden);
    			}
    			else
    			{
    				InputActionTriggersRow->Visibility(EVisibility::Visible);
    			}
    		}
    		if (IDetailPropertyRow* InputActionModifiersRow = ChildBuilder.AddExternalObjectProperty(ActionsAsUObjects, GET_MEMBER_NAME_CHECKED(UInputAction, Modifiers)))
    		{
    			InputActionModifiersPropertyRow = InputActionModifiersRow;
    			InputActionModifiersRow->DisplayName(LOCTEXT("InputActionModifiersDisplayName", "Modifiers From Input Action"));
    			InputActionModifiersRow->ToolTip(FText::Format(LOCTEXT("InputActionModifiersToolTip", "Modifiers from the {0} Input Action"), FText::FromName(ActionKeyMapping->Action.GetFName())));
    			InputActionModifiersRow->IsEnabled(false);
    			InputAction->OnModifiersChanged.AddSP(this, &FEnhancedActionMappingCustomization::OnInputActionModifiersChanged);
    			
    			// hide it if the modifier array is empty
    			if (ActionKeyMapping->Action->Modifiers.IsEmpty())
    			{
    				InputActionModifiersRow->Visibility(EVisibility::Hidden);
    			}
    			else
    			{
    				InputActionModifiersRow->Visibility(EVisibility::Visible);	
    			}
    		}
    	}
    }
}

void FEnhancedActionMappingCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> TriggersHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, Triggers));
	TSharedPtr<IPropertyHandle> ModifiersHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, Modifiers));
	TSharedPtr<IPropertyHandle> SettingBehavior = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, SettingBehavior));
	TSharedPtr<IPropertyHandle> PlayerMappableKeySettings = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnhancedActionKeyMapping, PlayerMappableKeySettings));

	// TODO: ResetToDefault needs to be disabled for arrays
	ChildBuilder.AddProperty(TriggersHandle.ToSharedRef());
	ChildBuilder.AddProperty(ModifiersHandle.ToSharedRef());
	AddInputActionProperties(PropertyHandle, ChildBuilder);
	ChildBuilder.AddProperty(SettingBehavior.ToSharedRef());
	ChildBuilder.AddProperty(PlayerMappableKeySettings.ToSharedRef());
}

void FEnhancedActionMappingCustomization::RemoveMappingButton_OnClick() const
{
	if (MappingPropertyHandle->IsValidHandle())
	{
		const TSharedPtr<IPropertyHandle> ParentHandle = MappingPropertyHandle->GetParentHandle();
		const TSharedPtr<IPropertyHandleArray> ParentArrayHandle = ParentHandle->AsArray();

		ParentArrayHandle->DeleteItem(MappingPropertyHandle->GetIndexInArray());
	}
}

void FEnhancedActionMappingCustomization::OnTriggersChanged() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bContainsComboTrigger = DoesTriggerArrayContainCombo();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	// if it is currently disabled and doesn't contain a combo now we should set the key to something other than ComboKey
	if (!KeyStructCustomization->GetEnableKeySelector() && !bContainsComboTrigger)
	{
		// setting KeySelector to none key
		KeyStructCustomization->SetKey(TEXT("None"));
	}
	
    KeyStructCustomization->SetEnableKeySelector(!bContainsComboTrigger);
	// updating the default key when the KeySelector is disabled
	KeyStructCustomization->SetDefaultKeyName(bContainsComboTrigger ? TEXT("ComboKey") : TEXT("None"));
}

void FEnhancedActionMappingCustomization::OnInputActionTriggersChanged() const
{
	if (const FEnhancedActionKeyMapping* ActionKeyMapping = GetActionKeyMapping(MappingPropertyHandle))
	{
		// Make sure ActionKeyMapping action and the property row are valid
		if (ActionKeyMapping->Action && InputActionTriggersPropertyRow)
		{
			// if so we want to hide the row or show it based off contents of the array (whether it's empty or not)
			if (!ActionKeyMapping->Action->Triggers.IsEmpty())
			{
				InputActionTriggersPropertyRow->Visibility(EVisibility::Visible);
			}
			else
			{
				InputActionTriggersPropertyRow->Visibility(EVisibility::Hidden);
			}
		}
	}
}

void FEnhancedActionMappingCustomization::OnInputActionModifiersChanged() const
{
    if (const FEnhancedActionKeyMapping* ActionKeyMapping = GetActionKeyMapping(MappingPropertyHandle))
    {
    	// Make sure ActionKeyMapping action and the property row are valid
    	if (ActionKeyMapping->Action && InputActionModifiersPropertyRow)
    	{
    		// if so we want to hide the row or show it based off contents of the array (whether it's empty or not)
    		if (!ActionKeyMapping->Action->Modifiers.IsEmpty())
    		{
    			InputActionModifiersPropertyRow->Visibility(EVisibility::Visible);
    		}
    		else
    		{
    			InputActionModifiersPropertyRow->Visibility(EVisibility::Hidden);
    		}
    	}
    }
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FEnhancedActionMappingCustomization::DoesTriggerArrayContainCombo() const
{
	if (const FEnhancedActionKeyMapping* ActionKeyMapping = GetActionKeyMapping(MappingPropertyHandle))
	{
		// checking context triggers for combo triggers
		for (const TObjectPtr<UInputTrigger>& Trigger : ActionKeyMapping->Triggers)
		{
			if (Trigger.IsA(UInputTriggerCombo::StaticClass()))
			{
				return true;
			}
		}
		// checking input action triggers for combo triggers
		if (ActionKeyMapping->Action)
		{
			for (const TObjectPtr<UInputTrigger>& Trigger : ActionKeyMapping->Action->Triggers)
			{
				if (Trigger.IsA(UInputTriggerCombo::StaticClass()))
				{
					return true;
				}
			}
		}
	}
	
	return false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

//////////////////////////////////////////////////////////
// FEnhancedInputDeveloperSettingsCustomization

TSet<FName> FEnhancedInputDeveloperSettingsCustomization::ExcludedAssetNames;

TSharedRef<IDetailCustomization> FEnhancedInputDeveloperSettingsCustomization::MakeInstance()
{
	LLM_SCOPE_BYNAME(TEXT("EnhancedInput/Editor"));
	return MakeShareable(new FEnhancedInputDeveloperSettingsCustomization());
}

FEnhancedInputDeveloperSettingsCustomization::~FEnhancedInputDeveloperSettingsCustomization()
{
	// Unregister settings panel listeners
	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry"))
	{
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
			AssetRegistry->OnAssetRenamed().RemoveAll(this);
		}
	}
	
	CachedDetailBuilder = nullptr;
}

void FEnhancedInputDeveloperSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Note: The details view for the UEnhancedInputDeveloperSettings object will be on by default
	// This 'EditCategory' call will ensure that the developer settings are displayed on top of the trigger/modifier default values
	DetailBuilder.EditCategory(UEnhancedInputDeveloperSettings::StaticClass()->GetFName(), FText::GetEmpty(), ECategoryPriority::Important);

	static const FName TriggerCategoryName = TEXT("Trigger Default Values");
	static const FName ModifierCategoryName = TEXT("Modifier Default Values");

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	TArray<UObject*> ModifierCDOs;
	TArray<UObject*> TriggerCDOs;
	// Search BPs via asset registry
	{
		FARFilter Filter;
		Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
		Filter.bRecursiveClasses = true;
		TArray<FAssetData> BlueprintAssetData;
		AssetRegistry.GetAssets(Filter, BlueprintAssetData);

		FName NativeParentClass(TEXT("NativeParentClass"));
		for (FAssetData& Asset : BlueprintAssetData)
		{
			FAssetDataTagMapSharedView::FFindTagResult Result = Asset.TagsAndValues.FindTag(NativeParentClass);
			if (Result.IsSet() && !ExcludedAssetNames.Contains(Asset.AssetName))
			{
				const FString ClassObjectPath = FPackageName::ExportTextPathToObjectPath(Result.GetValue());
				if (UClass* ParentClass = FindObjectSafe<UClass>(nullptr, *ClassObjectPath, EFindObjectFlags::ExactClass))
				{
					// TODO: Forcibly loading these assets could cause problems on projects with a large number of them.
					if(ParentClass->IsChildOf(UInputModifier::StaticClass()))
					{
						UBlueprint* BP = CastChecked<UBlueprint>(Asset.GetAsset());
						ModifierCDOs.AddUnique(BP->GeneratedClass->GetDefaultObject());
					}
					if (ParentClass->IsChildOf(UInputTrigger::StaticClass()))
					{
						UBlueprint* BP = CastChecked<UBlueprint>(Asset.GetAsset());
						TriggerCDOs.AddUnique(BP->GeneratedClass->GetDefaultObject());
					}
				}
			}
		}
	}

	GatherNativeClassDetailsCDOs(UInputModifier::StaticClass(), ModifierCDOs);
	GatherNativeClassDetailsCDOs(UInputTrigger::StaticClass(), TriggerCDOs);
	ExcludedAssetNames.Reset();
	
	// Add The modifier/trigger defaults that are generated via CDO to the details builder
	CustomizeCDOValues(DetailBuilder, ModifierCategoryName, ModifierCDOs);
	CustomizeCDOValues(DetailBuilder, TriggerCategoryName, TriggerCDOs);
	
	// Support for updating blueprint based triggers and modifiers in the settings panel
	if (!AssetRegistry.OnAssetAdded().IsBoundToObject(this))
	{
		AssetRegistry.OnAssetAdded().AddRaw(this, &FEnhancedInputDeveloperSettingsCustomization::OnAssetAdded);
		AssetRegistry.OnAssetRemoved().AddRaw(this, &FEnhancedInputDeveloperSettingsCustomization::OnAssetRemoved);
		AssetRegistry.OnAssetRenamed().AddRaw(this, &FEnhancedInputDeveloperSettingsCustomization::OnAssetRenamed);
	}
}

void FEnhancedInputDeveloperSettingsCustomization::CustomizeCDOValues(IDetailLayoutBuilder& DetailBuilder, const FName CategoryName, const TArray<UObject*>& ObjectsToCustomize)
{
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(CategoryName);

	// All of the Objects in this array are the CDO, i.e. the class name starts with "Default__"
	for (UObject* CDO : ObjectsToCustomize)
	{
		if (!ensure(CDO && CDO->IsTemplate()))
		{
			continue;
		}

		// Add the CDO as an external object reference to this customization
		IDetailPropertyRow* Row = CategoryBuilder.AddExternalObjects({ CDO }, EPropertyLocation::Default, FAddPropertyParams().UniqueId(CDO->GetClass()->GetFName()));

		if (Row)
		{
			// We need to add a custom "Name" widget here, otherwise all the categories will just say "Object"
			Row->CustomWidget()
			.NameContent()
			[
				SNew(STextBlock)
					.Text(CDO->GetClass()->GetDisplayNameText())
			];	
		}
	}
}

void FEnhancedInputDeveloperSettingsCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CachedDetailBuilder = DetailBuilder;
	CustomizeDetails(*DetailBuilder);
}

void FEnhancedInputDeveloperSettingsCustomization::GatherNativeClassDetailsCDOs(UClass* Class, TArray<UObject*>& CDOs)
{
	// Search native classes
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		if (!ClassIt->IsNative() || !ClassIt->IsChildOf(Class))
		{
			continue;
		}

		// Ignore abstract, hidedropdown, and deprecated.
		if (ClassIt->HasAnyClassFlags(CLASS_Abstract | CLASS_HideDropDown | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			continue;
		}

		CDOs.AddUnique(ClassIt->GetDefaultObject());
	}

	// Strip objects with no config stored properties
	CDOs.RemoveAll([Class](UObject* Object) {
		UClass* ObjectClass = Object->GetClass();
		if (ObjectClass->GetMetaData(TEXT("NotInputConfigurable")).ToBool())
		{
			return true;
		}
		while (ObjectClass)
		{
			for (FProperty* Property : TFieldRange<FProperty>(ObjectClass, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::ExcludeDeprecated))
			{
				if (Property->HasAnyPropertyFlags(CPF_Config))
				{
					return false;
				}
			}

			// Stop searching at the base type. We don't care about configurable properties lower than that.
			ObjectClass = ObjectClass != Class ? ObjectClass->GetSuperClass() : nullptr;
		}
		return true;
	});
}

bool FEnhancedInputDeveloperSettingsCustomization::DoesClassHaveSubtypes(UClass* Class)
{
	// Search native classes
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		// Make sure it's a native class and a child of Class passed in
		if (ClassIt->IsNative() && ClassIt->IsChildOf(Class))
		{
			// Make sure it doesn't have any flags that would disqualify it from being used
			if (!ClassIt->HasAnyClassFlags(CLASS_Abstract | CLASS_HideDropDown | CLASS_Deprecated | CLASS_NewerVersionExists))
			{
				UObject* CDO = ClassIt->GetDefaultObject();
		
				// Make Sure it isn't the Class itself
				if (CDO && CDO->GetClass() != Class)
				{
					return true;
				}
			}
		}
	}

	// Search BPs via asset registry
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	FARFilter Filter;
	Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;
	TArray<FAssetData> BlueprintAssetData;
	AssetRegistry.GetAssets(Filter, BlueprintAssetData);

	for (FAssetData& Asset : BlueprintAssetData)
	{
		FAssetDataTagMapSharedView::FFindTagResult Result = Asset.TagsAndValues.FindTag(TEXT("NativeParentClass"));
		if (Result.IsSet() && !ExcludedAssetNames.Contains(Asset.AssetName))
		{
			const FString ClassObjectPath = FPackageName::ExportTextPathToObjectPath(Result.GetValue());
			if (UClass* ParentClass = FindObjectSafe<UClass>(nullptr, *ClassObjectPath, EFindObjectFlags::ExactClass))
			{
				if (ParentClass->IsChildOf(Class))
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FEnhancedInputDeveloperSettingsCustomization::RebuildDetailsViewForAsset(const FAssetData& AssetData, const bool bIsAssetBeingRemoved)
{
	LLM_SCOPE_BYNAME(TEXT("EnhancedInput/Editor"));
	
	// If the asset was a blueprint...
	if (AssetData.AssetClassPath == UBlueprint::StaticClass()->GetClassPathName())
	{
		// With a native parent class...
		FAssetDataTagMapSharedView::FFindTagResult Result = AssetData.TagsAndValues.FindTag(FBlueprintTags::NativeParentClassPath);
		if (Result.IsSet())
		{
			// And the base class is UInputModifier or UInputTrigger, then we should rebuild the details
			const FString ClassObjectPath = FPackageName::ExportTextPathToObjectPath(Result.GetValue());
			if (UClass* ParentClass = FindObjectSafe<UClass>(nullptr, *ClassObjectPath, EFindObjectFlags::ExactClass))
			{
				if (ParentClass == UInputModifier::StaticClass() || ParentClass == UInputTrigger::StaticClass())
				{
					if (bIsAssetBeingRemoved)
					{
						ExcludedAssetNames.Add(AssetData.AssetName);
					}
					
					if (IDetailLayoutBuilder* DetailBuilder = CachedDetailBuilder.Pin().Get())
					{
						DetailBuilder->ForceRefreshDetails();
					}
				}
			}
		}
	}
}

/////////////////////////////////////////////////////////////////
// FPlayerMappableKeyChildSettingsCustomization

namespace UE::EnhancedInput
{
	/** Flag to enable or disable name validation in the editor for UPlayerMappableKeySettings assets */
	static bool bEnableNameValidation = true;
	static FAutoConsoleVariableRef CVarEnableNameValidation(
		TEXT("EnhancedInput.bEnableNameValidation"),
		bEnableNameValidation,
		TEXT("Flag to enable or disable name validation in the editor for UPlayerMappableKeySettings assets"),
		ECVF_Default);
}

TSharedRef<IPropertyTypeCustomization> FPlayerMappableKeyChildSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FPlayerMappableKeyChildSettingsCustomization());
}

void FPlayerMappableKeyChildSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	SettingsPropHandle = PropertyHandle;
	UPlayerMappableKeySettings* Settings = GetSettingsObject();
	
	HeaderRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];
	
	HeaderRow.ValueContent()
	[
		PropertyHandle->CreatePropertyValueWidget()
	];	
}

void FPlayerMappableKeyChildSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{				
	// Customize the name handle to have some validation on what users can type in
	static const FName MappingPropertyHandleName = GET_MEMBER_NAME_CHECKED(UPlayerMappableKeySettings, Name);

	uint32 NumChildren = 0;
	TSharedPtr<IPropertyHandle> SettingsHandle;
	TSharedPtr<IPropertyHandle> TempHandle = PropertyHandle->GetChildHandle(0);

	if (TempHandle.IsValid())
	{
		TempHandle->GetNumChildren(NumChildren);

		// For instanced properties there can be multiple layers of "children" until we get to the ones
		// that we care about. Iterate all the children until we find one with more then one
		// child, or we run out of children.
		while (!SettingsHandle.IsValid() || !TempHandle.IsValid())
		{
			TempHandle->GetNumChildren(NumChildren);

			if (NumChildren > 1)
			{
				SettingsHandle = TempHandle;
			}
			else
			{
				TempHandle = TempHandle->GetChildHandle(0);
			}		
		}	
	}	
	
	if (!SettingsHandle.IsValid())
	{
		return;
	}

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = SettingsHandle->GetChildHandle(ChildIndex);
		if (!ChildHandle.IsValid())
		{
			continue;
		}
		
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		// Add a customization to handle name validation on the mappable name if it's enabled
		if (PropertyName == MappingPropertyHandleName && UE::EnhancedInput::bEnableNameValidation)
		{
			MappingNamePropHandle = ChildHandle;
			IDetailPropertyRow& NameRow = ChildBuilder.AddProperty(MappingNamePropHandle.ToSharedRef());       
            
            NameRow.CustomWidget()
            .NameContent()
            [
            	MappingNamePropHandle->CreatePropertyNameWidget()
            ]
            .ValueContent()
            [
            	SNew(SEditableTextBox)
            		.Text(this, &FPlayerMappableKeyChildSettingsCustomization::OnGetMappingNameText)
            		.OnTextCommitted(this, &FPlayerMappableKeyChildSettingsCustomization::OnMappingNameTextCommited)
            		.ToolTipText(this, &FPlayerMappableKeyChildSettingsCustomization::OnGetMappingNameToolTipText)
            		.OnVerifyTextChanged(this, &FPlayerMappableKeyChildSettingsCustomization::OnVerifyMappingName)
            		.Font(IDetailLayoutBuilder::GetDetailFont())
            ];
		}
		else
		{
			ChildBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
	}
}

FName FPlayerMappableKeyChildSettingsCustomization::GetCurrentName() const
{
	if (UPlayerMappableKeySettings* Settings = GetSettingsObject())
	{
		return Settings->Name;
	}
	ensure(false);
	return NAME_None;
}

UPlayerMappableKeySettings* FPlayerMappableKeyChildSettingsCustomization::GetSettingsObject() const
{
	if (SettingsPropHandle.IsValid())
	{
		UObject* SettingsObject = nullptr;
		SettingsPropHandle->GetValue(SettingsObject);
		return Cast<UPlayerMappableKeySettings>(SettingsObject);	
	}

	return nullptr;
}

FText FPlayerMappableKeyChildSettingsCustomization::OnGetMappingNameText() const
{
	// this is the value in the actual text box
	if (UPlayerMappableKeySettings* Settings = GetSettingsObject())
	{
		return FText::FromName(Settings->Name);
	}
	
	return FText::FromName(NAME_None);
}

FText FPlayerMappableKeyChildSettingsCustomization::OnGetMappingNameToolTipText() const
{
	static const FText Tooltip = LOCTEXT("MappingNameToolTip", "This mapping name will be the key that is used to save player mapping data for this action. It can be overriden by individual mappings, or specified by the Input Action. These names need to be unique for each player mapping you wish to save.");
	return Tooltip;
}

void FPlayerMappableKeyChildSettingsCustomization::OnMappingNameTextCommited(const FText& NewText, ETextCommit::Type InTextCommit)
{
	// set the actual property value on the settings object
	if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
	{
		MappingNamePropHandle->SetValue(NewText.ToString());
	}
}

bool FPlayerMappableKeyChildSettingsCustomization::OnVerifyMappingName(const FText& InNewText, FText& OutErrorMessage)
{
	UPlayerMappableKeySettings* Settings = GetSettingsObject();
	const FName CurrentName = Settings ? Settings->Name : NAME_None;
	
	// Validate that the new name can be used
	FEnhancedInputPlayerMappableNameValidator NameValidator(CurrentName);
	EValidatorResult Result = NameValidator.IsValid(InNewText.ToString(), false);
	OutErrorMessage = FEnhancedInputPlayerMappableNameValidator::GetErrorText(InNewText.ToString(), Result);

	return Result == EValidatorResult::Ok || Result == EValidatorResult::ExistingName;
}

/////////////////////////////////////////////////////////////////////////////////////
// FMappingContextKeyContainerCustomization

TSharedRef<IPropertyTypeCustomization> FMappingContextKeyContainerCustomization::MakeInstance()
{
	return MakeShareable(new FMappingContextKeyContainerCustomization());
}

void FMappingContextKeyContainerCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> PropertyHandle,
	FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	MappingsArrayHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputMappingContextMappingData, Mappings));
	
	HeaderRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];
	
	HeaderRow.ValueContent()
	[
		PropertyHandle->CreatePropertyValueWidget()
	];	
}

void FMappingContextKeyContainerCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> PropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (!ensure(MappingsArrayHandle.IsValid()))
	{
		return;
	}
	
	MappingsArrayHandle->MarkHiddenByCustomization();

	const TSharedRef<FActionMappingsNodeBuilderEx> ActionMappingsBuilder = MakeShareable(new FActionMappingsNodeBuilderEx(MappingsArrayHandle));
	ChildBuilder.AddCustomBuilder(ActionMappingsBuilder);
}

#undef LOCTEXT_NAMESPACE
