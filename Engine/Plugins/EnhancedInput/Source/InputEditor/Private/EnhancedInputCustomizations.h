// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"

class FKeyStructCustomization;
class UPlayerMappableKeySettings;
class IDetailPropertyRow;
class IDetailLayoutBuilder;


class FEnhancedActionMappingCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FEnhancedActionMappingCustomization());
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	/** Searches the trigger array of an input action - returns true if any combo input triggers are present */
	UE_DEPRECATED(5.8, "The combo trigger is now deprecated")
	bool DoesTriggerArrayContainCombo() const;
private:

	void RemoveMappingButton_OnClick() const;
	void OnTriggersChanged() const;
	void AddInputActionProperties(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder);
	void OnInputActionTriggersChanged() const;
	void OnInputActionModifiersChanged() const;

	TSharedPtr<IPropertyTypeCustomization> KeyStructInstance;
	TSharedPtr<IPropertyHandle> MappingPropertyHandle;
	TSharedPtr<FKeyStructCustomization> KeyStructCustomization;
	IDetailPropertyRow* InputActionTriggersPropertyRow = nullptr;
	IDetailPropertyRow* InputActionModifiersPropertyRow = nullptr;
};

/**
 * Customization for UEnhancedInputDeveloperSettings.
 *
 * This will just make the normal details panel for UEnhancedInputDeveloperSettings, and then add all the default settings
 * of Input Triggers and Input Modifiers by gather all the CDO's for them.
 */
class FEnhancedInputDeveloperSettingsCustomization : public IDetailCustomization
{
public:
	//~ IDetailCustomization interface
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual ~FEnhancedInputDeveloperSettingsCustomization() override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
	//~ End IDetailCustomization interface
	
	/** Returns true if something inherits from the UClass passed in. Checks BPs and Native */
	static bool DoesClassHaveSubtypes(UClass* Class);

private:

	/** Gather all of the native class CDOs for the given class. */
	static void GatherNativeClassDetailsCDOs(UClass* Class, TArray<UObject*>& CDOs);

	/**
	 * Called when any Asset is added, removed, or renamed.
	 *
	 * This will rebuild the Modifier and Trigger CDO views to make sure that
	 * any newly added Blueprint
	 */
	void RebuildDetailsViewForAsset(const FAssetData& AssetData, const bool bIsAssetBeingRemoved);

	/** Callbacks that are triggered from the Asset Registry. */
	void OnAssetAdded(const FAssetData& AssetData) { RebuildDetailsViewForAsset(AssetData, false); }
	void OnAssetRemoved(const FAssetData& AssetData) { RebuildDetailsViewForAsset(AssetData, true); }
	void OnAssetRenamed(const FAssetData& AssetData, const FString&) { RebuildDetailsViewForAsset(AssetData, false); }
	
	/**
	 * Create a new category on the DetailBuilder and add each object in the given array as an external reference.
	 * 
	 * @param DetailBuilder			The details builder that can be used to add categories
	 * @param CategoryName			The name of the new category to add
	 * @param ObjectsToCustomize	Array of CDO objects to customize and add as external references
	 */
	void CustomizeCDOValues(IDetailLayoutBuilder& DetailBuilder, const FName CategoryName, const TArray<UObject*>& ObjectsToCustomize);

	// Cached details builder so that we can rebuild the details when a new BP asset is added
	TWeakPtr<IDetailLayoutBuilder> CachedDetailBuilder;

	/**
	 * Populated by RebuildDetailsViewForAsset so that we can exclude any blueprint classes
	 * that have been removed from the asset registry but are still loaded in memory.
	 * This is reset when CustomizeCDOValues is called
	 */
	static TSet<FName> ExcludedAssetNames;
};

/**
 * Input customization for UPlayerMappableKeySettings to have some name validation
 * on the custom saved FNames for mappings.
 */
class FPlayerMappableKeyChildSettingsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	
	UPlayerMappableKeySettings* GetSettingsObject() const;
	FName GetCurrentName() const;
	
	FText OnGetMappingNameText() const;
	FText OnGetMappingNameToolTipText() const;
	void OnMappingNameTextCommited(const FText& NewText, ETextCommit::Type InTextCommit);
	bool OnVerifyMappingName(const FText& InNewText, FText& OutErrorMessage);

	TSharedPtr<IPropertyHandle> SettingsPropHandle;
	TSharedPtr<IPropertyHandle> MappingNamePropHandle;
};

/**
 * Customization for FInputMappingContextMappingData.
 */
class FMappingContextKeyContainerCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
	
	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> PropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;
		
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> PropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	
	TSharedPtr<IPropertyHandle> MappingsArrayHandle;
};