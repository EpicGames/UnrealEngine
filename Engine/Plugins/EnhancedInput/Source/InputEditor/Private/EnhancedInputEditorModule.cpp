// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputEditorModule.h"

#include "AssetBlueprintGraphActions.h"
#include "BlueprintEditorModule.h"
#include "BlueprintNodeTemplateCache.h"
#include "Brushes/SlateImageBrush.h"
#include "Framework/Notifications/NotificationManager.h"
#include "InputMappingContext.h"
#include "Misc/PackageName.h"
#include "EnhancedInputCustomizations.h"
#include "ISettingsModule.h"
#include "ToolMenuSection.h"
#include "PropertyEditorModule.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "AssetTypeActions/AssetTypeActions_DataAsset.h"
#include "EnhancedInputDeveloperSettings.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "ContentBrowserModule.h"
#include "EdGraphSchema_K2_Actions.h"
#include "IContentBrowserSingleton.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "Kismet2/SClassPickerDialog.h"
#include "GameFramework/InputSettings.h"
#include "EnhancedInputComponent.h"
#include "EnhancedPlayerInput.h"
#include "Interfaces/IMainFrameModule.h"
#include "SourceControlHelpers.h"
#include "HAL/FileManager.h"
#include "PlayerMappableKeySettings.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputEditorModule)

#define LOCTEXT_NAMESPACE "InputEditor"

DEFINE_LOG_CATEGORY(LogEnhancedInputEditor);

EAssetTypeCategories::Type FInputEditorModule::InputAssetsCategory;

namespace UE::Input
{
	static bool bEnableAutoUpgradeToEnhancedInput = true;
	static FAutoConsoleVariableRef CVarEnableAutoUpgradeToEnhancedInput(
		TEXT("EnhancedInput.bEnableAutoUpgrade"),
		bEnableAutoUpgradeToEnhancedInput,
		TEXT("Should your project automatically be set to use Enhanced Input if it is currently using the legacy input system?"),
		ECVF_Default);
}

IMPLEMENT_MODULE(FInputEditorModule, InputEditor)

class FInputClassParentFilter : public IClassViewerFilter
{
public:
	FInputClassParentFilter()
		: DisallowedClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_HideDropDown) {}

	/** All children of these classes will be included unless filtered out by another setting. */
	TSet<const UClass*> AllowedChildrenOfClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};

// Asset factories

// InputContext
UInputMappingContext_Factory::UInputMappingContext_Factory(const class FObjectInitializer& OBJ) : Super(OBJ) {
	SupportedClass = UInputMappingContext::StaticClass();
	bEditAfterNew = true;
	bCreateNew = true;
}

bool UInputMappingContext_Factory::ConfigureProperties()
{
	if (!FEnhancedInputDeveloperSettingsCustomization::DoesClassHaveSubtypes(UInputMappingContext::StaticClass()))
	{
		return true;
	}
	
	// nullptr the InputMappingContextClass so we can check for selection
	InputMappingContextClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<FInputClassParentFilter> Filter = MakeShareable(new FInputClassParentFilter);
	Filter->AllowedChildrenOfClasses.Add(UInputMappingContext::StaticClass());

	Options.ClassFilters.Add(Filter.ToSharedRef());

	const FText TitleText = LOCTEXT("CreateInputMappingContextOptions", "Pick Class For Input Mapping Context Instance");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UInputMappingContext::StaticClass());

	if (bPressedOk)
	{
		InputMappingContextClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* UInputMappingContext_Factory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UInputMappingContext* IMC = nullptr;

	if (InputMappingContextClass != nullptr)
	{
		IMC = NewObject<UInputMappingContext>(InParent, InputMappingContextClass, Name, Flags | RF_Transactional, Context);
	}
	else
	{
		check(Class->IsChildOf(UInputMappingContext::StaticClass()));
		IMC = NewObject<UInputMappingContext>(InParent, Class, Name, Flags | RF_Transactional, Context);
	}

	check(IMC);

	// Populate the IMC with some initial input actions if they were specified. This will be the case if the IMC is being created from the FAssetTypeActions_InputAction
	for (TWeakObjectPtr<UInputAction> WeakIA : InitialActions)
	{
		if (UInputAction* IA = WeakIA.Get())
		{
			IMC->MapKey(IA, FKey());
		}
	}
	
	return IMC;
}

void UInputMappingContext_Factory::SetInitialActions(TArray<TWeakObjectPtr<UInputAction>> InInitialActions)
{
	InitialActions.Empty();
	InitialActions = InInitialActions;
}

// InputAction
UInputAction_Factory::UInputAction_Factory(const class FObjectInitializer& OBJ)
	: Super(OBJ)
{
	SupportedClass = UInputAction::StaticClass();
	bEditAfterNew = true;
	bCreateNew = true;
}

bool UInputAction_Factory::ConfigureProperties()
{
	if (!FEnhancedInputDeveloperSettingsCustomization::DoesClassHaveSubtypes(UInputAction::StaticClass()))
	{
		return true;
	}
	
	// nullptr the InputActionClass so we can check for selection
	InputActionClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<FInputClassParentFilter> Filter = MakeShareable(new FInputClassParentFilter);
	Filter->AllowedChildrenOfClasses.Add(UInputAction::StaticClass());

	Options.ClassFilters.Add(Filter.ToSharedRef());

	const FText TitleText = LOCTEXT("CreateInputActionOptions", "Pick Class For Input Action Instance");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UInputAction::StaticClass());

	if (bPressedOk)
	{
		InputActionClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* UInputAction_Factory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{	
	if (InputActionClass != nullptr)
	{
		return NewObject<UInputAction>(InParent, InputActionClass, Name, Flags | RF_Transactional, Context);
	}
	else
	{
		check(Class->IsChildOf(UInputAction::StaticClass()));
		return NewObject<UInputAction>(InParent, Class, Name, Flags | RF_Transactional, Context);
	}
}

/** Custom style set for Enhanced Input */
class FEnhancedInputSlateStyle final : public FSlateStyleSet
{
public:
	FEnhancedInputSlateStyle()
		: FSlateStyleSet("EnhancedInputEditor")
	{
		SetParentStyleName(FAppStyle::GetAppStyleSetName());

		// The icons are located in /Engine/Plugins/EnhancedInput/Content/Editor/Slate/Icons
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("EnhancedInput/Content/Editor/Slate"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		// Enhanced Input Editor icons
		static const FVector2D Icon16 = FVector2D(16.0f, 16.0f);
		static const FVector2D Icon64 = FVector2D(64.0f, 64.0f);

		Set("EnhancedInputIcon_Small", new IMAGE_BRUSH_SVG("Icons/EnhancedInput_16", Icon16));
		Set("EnhancedInputIcon_Large", new IMAGE_BRUSH_SVG("Icons/EnhancedInput_64", Icon64));
		
		Set("ClassIcon.InputAction", new IMAGE_BRUSH_SVG("Icons/InputAction_16", Icon16));
		Set("ClassThumbnail.InputAction", new IMAGE_BRUSH_SVG("Icons/InputAction_64", Icon64));
		
		Set("ClassIcon.InputMappingContext", new IMAGE_BRUSH_SVG("Icons/InputMappingContext_16", Icon16));
		Set("ClassThumbnail.InputMappingContext", new IMAGE_BRUSH_SVG("Icons/InputMappingContext_64", Icon64));
	}
};

void FInputEditorModule::StartupModule()
{
	LLM_SCOPE_BYNAME(TEXT("EnhancedInput/Editor"));
	
	// Register customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout(FInputMappingContextMappingData::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMappingContextKeyContainerCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("EnhancedActionKeyMapping", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FEnhancedActionMappingCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(UEnhancedInputDeveloperSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FEnhancedInputDeveloperSettingsCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(UPlayerMappableKeySettings::StaticClass()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPlayerMappableKeyChildSettingsCustomization::MakeInstance));
	PropertyModule.NotifyCustomizationModuleChanged();

	// Register input assets
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	InputAssetsCategory = AssetTools.RegisterAdvancedAssetCategory(FName(TEXT("Input")), LOCTEXT("InputAssetsCategory", "Input"));

	// Make a new style set for Enhanced Input, which will register any custom icons for the types in this plugin
	StyleSet = MakeShared<FEnhancedInputSlateStyle>();
	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());

	// Listen for when the editor is ready and then try to upgrade the input classes. We will send a slate notification
	// if we upgrade the input classes, which is why we need to wait for the editor to be ready
    IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
    MainFrameModule.OnMainFrameCreationFinished().AddRaw(this, &FInputEditorModule::OnMainFrameCreationFinished);
}

void FInputEditorModule::ShutdownModule()
{
	// Unregister input settings
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Engine", "Enhanced Input");
	}

	// Unregister customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.UnregisterCustomPropertyTypeLayout("InputMappingContextMappingData");
	PropertyModule.UnregisterCustomPropertyTypeLayout("EnhancedActionKeyMapping");
	PropertyModule.UnregisterCustomClassLayout("EnhancedInputDeveloperSettings");
	PropertyModule.UnregisterCustomPropertyTypeLayout("PlayerMappableKeySettings");
	PropertyModule.NotifyCustomizationModuleChanged();

	// Unregister slate stylings
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
	}

	// Remove any listeners to the editor startup delegate
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule::Get().OnMainFrameCreationFinished().RemoveAll(this);
	}
}

EAssetTypeCategories::Type FInputEditorModule::GetInputAssetsCategory()
{
	return InputAssetsCategory;
}

void FInputEditorModule::OnMainFrameCreationFinished(TSharedPtr<SWindow> InRootWindow, bool bIsRunningStartupDialog)
{
	if (FApp::HasProjectName())
	{
		AutoUpgradeDefaultInputClasses();
	}	
}

namespace UE::Input
{
	static void ShowToast(const FText& ClassBeingChanged, const FString& NewClassName, const FString& OldClassName)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ClassBeingChanged"), ClassBeingChanged);
		Args.Add(TEXT("NewClass"), FText::FromString(NewClassName));
		Args.Add(TEXT("OldClass"), FText::FromString(OldClassName));

		const FText Message = FText::Format(LOCTEXT("EnhancedInputUpgradeToast", "Upgrading default {ClassBeingChanged} class from '{OldClass}' to '{NewClass}'"), Args);
		
		FNotificationInfo* Info = new FNotificationInfo(Message);
		Info->ExpireDuration = 5.0f;
		Info->bFireAndForget = true;
		FSlateNotificationManager::Get().QueueNotification(Info);
		
		UE_LOGF(LogEnhancedInputEditor, Log, "Upgrading Default %ls class from '%ls' to '%ls'", *ClassBeingChanged.ToString(), *OldClassName, *NewClassName);
	}
}

void FInputEditorModule::AutoUpgradeDefaultInputClasses()
{
	if (!UE::Input::bEnableAutoUpgradeToEnhancedInput)
	{
		return;
	}
	
	if (UInputSettings* InputSettings = GetMutableDefault<UInputSettings>())
	{
		const UClass* OriginalInputComponentClass = InputSettings->GetDefaultInputComponentClass();
		const UClass* OriginalPlayerInputClass = InputSettings->GetDefaultPlayerInputClass();
		bool bNeedsConfigSave = false;
		
		if (OriginalInputComponentClass && OriginalInputComponentClass == UInputComponent::StaticClass())
		{
			InputSettings->SetDefaultInputComponentClass(UEnhancedInputComponent::StaticClass());

			static const FText ClassName = LOCTEXT("InputComponentClassLabel", "Input Component");
			UE::Input::ShowToast(ClassName, UEnhancedInputComponent::StaticClass()->GetName(), OriginalInputComponentClass->GetName());
			bNeedsConfigSave = true;
		}

		if (OriginalPlayerInputClass && OriginalPlayerInputClass == UPlayerInput::StaticClass())
		{
			InputSettings->SetDefaultPlayerInputClass(UEnhancedPlayerInput::StaticClass());

			static const FText ClassName = LOCTEXT("PlayerInputClassLabel", "Player Input");
			UE::Input::ShowToast(ClassName, UEnhancedPlayerInput::StaticClass()->GetName(), OriginalPlayerInputClass->GetName());
			bNeedsConfigSave = true;
		}

		// Make sure that the config file gets updated with these new values
		if (bNeedsConfigSave)
		{
			const FString DefaultConfigFile = InputSettings->GetDefaultConfigFilename();
			
			// We can write to the file if it is not read only. If it is read only, then we can write to it if we successfully check it out with source control
			bool bCanWriteToFile = !IFileManager::Get().IsReadOnly(*DefaultConfigFile) || (USourceControlHelpers::IsEnabled() && USourceControlHelpers::CheckOutFile(DefaultConfigFile));
			
			if (bCanWriteToFile)
			{
				InputSettings->TryUpdateDefaultConfigFile(DefaultConfigFile, /* bWarnIfFail */ false);
			}
		}
	}
}

const UPlayerMappableKeySettings* FInputEditorModule::FindMappingByName(const FName InName)
{
	// Instanced objects that have been deleted will have an outer of the transient package, and we don't want to include them
	for (TObjectIterator<UPlayerMappableKeySettings> Itr(RF_ClassDefaultObject | RF_Transient, true, EInternalObjectFlags::Garbage | EInternalObjectFlags::Unreachable); Itr; ++Itr)
	{
		const UPlayerMappableKeySettings* Settings = *Itr;
		if (Settings && Settings->GetOuter() != GetTransientPackage() && Settings->Name == InName)
		{
			// What if we had just removed a mapping of this name?
			return Settings;
		}
	}
	return nullptr;
}

bool FInputEditorModule::IsMappingNameInUse(const FName InName)
{
	// If a mapping with this name exists, then it is in use
	return FindMappingByName(InName) != nullptr;
}

#undef LOCTEXT_NAMESPACE