// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputEditorSettings.h"
#include "EnhancedInputDeveloperSettings.h"
#include "EnhancedPlayerInput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedInputEditorSettings)

////////////////////////////////////////////////////////////////////////
// UEnhancedInputEditorProjectSettings
UEnhancedInputEditorProjectSettings::UEnhancedInputEditorProjectSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, DefaultEditorInputClass(UEnhancedPlayerInput::StaticClass())
{
}

////////////////////////////////////////////////////////////////////////
// UEnhancedInputEditorSettings
UEnhancedInputEditorSettings::UEnhancedInputEditorSettings()
	: bLogAllInput(false)
	, bAutomaticallyStartConsumingInput(false)
	// By default only show the triggered event 
	, VisibleEventPinsByDefault(static_cast<uint8>(ETriggerEvent::Triggered))
{
}

