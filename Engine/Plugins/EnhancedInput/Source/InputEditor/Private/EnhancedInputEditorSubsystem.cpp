// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputEditorSubsystem.h"
#include "EnhancedInputDeveloperSettings.h"
#include "InputMappingContext.h"
#include "EnhancedInputComponent.h"
#include "Subsystems/UnrealEditorSubsystem.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor.h"
#include "EnhancedInputEditorProcessor.h"
#include "EnhancedInputEditorSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "UserSettings/EnhancedInputUserSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnhancedInputEditorSubsystem)

#define LOCTEXT_NAMESPACE "EnhancedInputEditorSubsystem"

DEFINE_LOG_CATEGORY(LogEditorInput);

void UEnhancedInputEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	const UClass* InputClassToUse = GetDefault<UEnhancedInputEditorProjectSettings>()->DefaultEditorInputClass.Get();
	
	if (!ensureMsgf(InputClassToUse, TEXT("The UEnhancedInputEditorProjectSettings input class is null! Make sure this has a value set. Using UEnhancedPlayerInput as a fallback.")))
	{
		InputClassToUse = UEnhancedPlayerInput::StaticClass();
	}
	PlayerInput = NewObject<UEnhancedPlayerInput>(this, InputClassToUse);

	ensureMsgf(PlayerInput, TEXT("UEnhancedInputEditorSubsystem::Initialize failed to create PlayerInput! This subsystem will not tick!"));
	
	// Create and register the input preprocessor, this is what will call our "InputKey"
	// function to drive input instead of a player controller
	if (ensure(FSlateApplication::IsInitialized()))
	{
		InputPreprocessor = MakeShared<FEnhancedInputEditorProcessor>();
		FSlateApplication::Get().RegisterInputPreProcessor(InputPreprocessor, 0);	
	}

	if (GetDefault<UEnhancedInputEditorSettings>()->bAutomaticallyStartConsumingInput)
	{
		StartConsumingInput();
	}
}

void UEnhancedInputEditorSubsystem::Deinitialize()
{
	Super::Deinitialize();

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().UnregisterInputPreProcessor(InputPreprocessor);
	}
	
	InputPreprocessor.Reset();
	CurrentInputStack.Empty();
	PlayerInput = nullptr;
}

bool UEnhancedInputEditorSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	// We don't ever want to create this subsystem outside of the editor
	// If you are outside the editor, then your input delegates would be driven by
	// the regular Enhanced Input subsystem.
#if !WITH_EDITOR
	return false;
#endif
	
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}
	
	return Super::ShouldCreateSubsystem(Outer);
}

UWorld* UEnhancedInputEditorSubsystem::GetTickableGameObjectWorld() const
{
	return GetWorld();
}

ETickableTickType UEnhancedInputEditorSubsystem::GetTickableTickType() const
{
	// Don't let the CDO ever tick
	return IsTemplate() ? ETickableTickType::Never : ETickableTickType::Conditional;
}

bool UEnhancedInputEditorSubsystem::IsTickable() const
{
	// Only tick if we have created a valid player input (i.e. we have been initalized)
	return PlayerInput != nullptr && bIsCurrentlyConsumingInput;
}

void UEnhancedInputEditorSubsystem::Tick(float DeltaTime)
{
	if (!ensure(PlayerInput))
	{
		UE_LOGF(LogEditorInput, Error, "UEnhancedInputEditorSubsystem is ticking before it has created a PlayerInput!");
		return;
	}

	FModifyContextOptions Options;
	Options.bForceImmediately = true;
	
	// Rebuild the control mappings and tick any forced input that may have been injected
	RequestRebuildControlMappings(Options);
	TickForcedInput(DeltaTime);

	static TArray<UInputComponent*> InputStack;

	// Build the input stack
	{
		for (int32 i = 0; i < CurrentInputStack.Num(); ++i)
		{
			if (UInputComponent* IC = CurrentInputStack[i].Get())
			{
				InputStack.Push(IC);
			}
			else
			{
				CurrentInputStack.RemoveAt(i--);
			}
		}
	}
	
	// Process input stack on the player input
	PlayerInput->Tick(DeltaTime);
	PlayerInput->ProcessInputStack(InputStack, DeltaTime, false);

	InputStack.Reset();
}

void UEnhancedInputEditorSubsystem::PushInputComponent(UInputComponent* InInputComponent)
{
	if (InInputComponent)
	{
		if (GetDefault<UEnhancedInputEditorSettings>()->bLogAllInput)
		{
			UE_LOGF(LogEditorInput, Log, "PUSHING AN INPUT COMPONENT! '%ls'", *InInputComponent->GetName());
		}

		// Mark this input component for firing delegates in the editor
		if (UEnhancedInputComponent* EIC = Cast<UEnhancedInputComponent>(InInputComponent))
		{
			EIC->SetShouldFireDelegatesInEditor(true);
		}

		bool bPushed = false;
		CurrentInputStack.RemoveSingle(InInputComponent);
		for (int32 Index = CurrentInputStack.Num() - 1; Index >= 0; --Index)
		{
			UInputComponent* IC = CurrentInputStack[Index].Get();
			if (IC == nullptr)
			{
				CurrentInputStack.RemoveAt(Index);
			}
			else if (IC->Priority <= InInputComponent->Priority)
			{
				CurrentInputStack.Insert(InInputComponent, Index + 1);
				bPushed = true;
				break;
			}
		}
		if (!bPushed)
		{
			CurrentInputStack.Insert(InInputComponent, 0);
			RequestRebuildControlMappings();
		}
	}
	else
	{
		UE_LOGF(LogEditorInput, Error, "Attempted to push a null Input Component to the Enhanced Input Editor Subsystem!");
	}
}

bool UEnhancedInputEditorSubsystem::PopInputComponent(UInputComponent* InInputComponent)
{
	if (InInputComponent)
	{
		if (CurrentInputStack.RemoveSingle(InInputComponent) > 0)
		{
			InInputComponent->ClearBindingValues();
			RequestRebuildControlMappings();
			return true;
		}
	}
	return false;
}

void UEnhancedInputEditorSubsystem::StartConsumingInput()
{
	bIsCurrentlyConsumingInput = true;
	
	AddDefaultMappingContexts();

	RequestRebuildControlMappings();
}

void UEnhancedInputEditorSubsystem::StopConsumingInput()
{
	bIsCurrentlyConsumingInput = false;

	RemoveDefaultMappingContexts();

	RequestRebuildControlMappings();
}

UEnhancedPlayerInput* UEnhancedInputEditorSubsystem::GetPlayerInput() const
{
	return PlayerInput.Get();
}

UWorld* UEnhancedInputEditorSubsystem::GetWorld() const
{
	if (UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>())
	{
		if (GUnrealEd->IsPlayingSessionInEditor())
		{
			return UnrealEditorSubsystem->GetGameWorld();
		}
		else
		{
			return UnrealEditorSubsystem->GetEditorWorld();
		}
	}

	return Super::GetWorld();
}

bool UEnhancedInputEditorSubsystem::InputKey(const FInputKeyEventArgs& Params)
{
	if (PlayerInput)
	{
		if (bIsCurrentlyConsumingInput)
		{
			if (GetDefault<UEnhancedInputEditorSettings>()->bLogAllInput)
			{
				UE_LOGF(LogEditorInput, Log, "EI Editor Subsystem InputKey : [%ls]", *Params.Key.ToString());
			}
			return PlayerInput->InputKey(Params);
		}
		return false;
	}

	ensureAlwaysMsgf(false, TEXT("Attempting to input a key to the EnhancedInputEditorSubsystem, but there is no Player Input!"));
	return false;
}

void UEnhancedInputEditorSubsystem::AddDefaultMappingContexts()
{
	for (const FDefaultContextSetting& ContextSetting : GetMutableDefault<UEnhancedInputEditorProjectSettings>()->DefaultMappingContexts)
	{
		if (const UInputMappingContext* IMC = ContextSetting.InputMappingContext.LoadSynchronous())
		{
			if (ContextSetting.bAddImmediately)
			{
				if (!HasMappingContext(IMC))
				{
					FModifyContextOptions Opts = {};
					Opts.bNotifyUserSettings = ContextSetting.bRegisterWithUserSettings;
					AddMappingContext(IMC, ContextSetting.Priority, Opts);	
				}
			}
			else if (ContextSetting.bRegisterWithUserSettings)
			{
				if (UEnhancedInputUserSettings* Settings = GetUserSettings())
				{
					Settings->RegisterInputMappingContext(IMC);
				}
			}
		}
	}
}

void UEnhancedInputEditorSubsystem::RemoveDefaultMappingContexts()
{
	for (const FDefaultContextSetting& ContextSetting : GetMutableDefault<UEnhancedInputEditorProjectSettings>()->DefaultMappingContexts)
	{
		if (const UInputMappingContext* IMC = ContextSetting.InputMappingContext.LoadSynchronous())
		{
			FModifyContextOptions Opts = {};
			Opts.bNotifyUserSettings = ContextSetting.bRegisterWithUserSettings;
			RemoveMappingContext(IMC, Opts);
		}
	}
}

#undef LOCTEXT_NAMESPACE

