// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnhancedInputPlayerMappableNameValidator.h"
#include "PlayerMappableKeySettings.h"
#include "InputEditorModule.h"
#include "Internationalization/Text.h"	// For FFormatNamedArguments
#include "HAL/IConsoleManager.h"		// For FAutoConsoleVariableRef

namespace UE::EnhancedInput
{
	/** Enables editor validation on player mapping names */
	static bool bEnableMappingNameValidation = true;
	
	FAutoConsoleVariableRef CVarEnableMappingNameValidation(
	TEXT("EnhancedInput.Editor.EnableMappingNameValidation"),
	bEnableMappingNameValidation,
	TEXT("Enables editor validation on player mapping names"),
	ECVF_Default);
	
	/**
	 * Returns the max acceptable length for a player mappable name.
	 */
	static int32 GetMaxAcceptableLength()
	{
		// Player Mappable names are just FNames, so using the max FName size here is acceptable.
		return NAME_SIZE;
	}
}

FEnhancedInputPlayerMappableNameValidator::FEnhancedInputPlayerMappableNameValidator(FName InExistingName)
	: FStringSetNameValidator(InExistingName.ToString())
{ }

EValidatorResult FEnhancedInputPlayerMappableNameValidator::IsValid(const FString& Name, bool bOriginal)
{
	// Ensure that the length of this name is a valid length
	if (Name.Len() >= UE::EnhancedInput::GetMaxAcceptableLength())
	{
		return EValidatorResult::TooLong;
	}
	
	EValidatorResult Result = FStringSetNameValidator::IsValid(Name, bOriginal);

	if (UE::EnhancedInput::bEnableMappingNameValidation &&
		Result != EValidatorResult::ExistingName &&
		FInputEditorModule::IsMappingNameInUse(FName(Name)))
	{
		Result = EValidatorResult::AlreadyInUse;
	}

	return Result;
}

FText FEnhancedInputPlayerMappableNameValidator::GetErrorText(const FString& Name, EValidatorResult ErrorCode)
{
	// Attempt to specify what asset is using this name
	if (ErrorCode == EValidatorResult::AlreadyInUse)
	{
		if (const UPlayerMappableKeySettings* Settings = FInputEditorModule::FindMappingByName(FName(Name)))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("AssetUsingName"), FText::FromString(GetNameSafe(Settings->GetOuter())));

			return FText::Format(NSLOCTEXT("EnhancedInput", "MappingNameInUseBy_Error", "Name is already in use by '{AssetUsingName}'"), Args);	
		}
	}
	else if (ErrorCode == EValidatorResult::TooLong)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("MaxLength"), UE::EnhancedInput::GetMaxAcceptableLength());

		return FText::Format(NSLOCTEXT("EnhancedInput", "MappingNameTooLong_Error", "Names must be fewer then '{MaxLength}' characters"), Args);
	}
	
	return INameValidatorInterface::GetErrorText(Name, ErrorCode);
}