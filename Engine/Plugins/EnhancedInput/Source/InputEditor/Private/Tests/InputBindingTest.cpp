// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputTestFramework.h"

#include "EnhancedInputComponent.h"
#include "GameFramework/PlayerController.h"
#include "Misc/AutomationTest.h"

// Tests focused on binding logic for both delegates and input devices



constexpr auto BasicBindingTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter;	// TODO: Run as Smoke/Client? No world on RunSmokeTests startup...

UControllablePlayer& ABasicBindingTest(FAutomationTestBase* Test, EInputActionValueType ForValueType)
{
	// Initialise
	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	Test->TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());	// TODO: Can we early out on a failed Test?

	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	UInputAction* Action =
	AND(AnInputAction(Data, TestAction, ForValueType));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));

	return Data;
}

// ******************************
// Delegate firing (notification) tests
// ******************************


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputBindingDigitalTest, "Input.Binding.DigitalTrigger", BasicBindingTestFlags)

bool FInputBindingDigitalTest::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(ABasicBindingTest(this, EInputActionValueType::Boolean));

	// Unpressed shouldn't trigger
	WHEN(InputIsTicked(Data));
	THEN(!FInputTestHelper::TestTriggered(Data, TestAction));

	// Pressing
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	// Holding for multiple ticks
	const int32 HoldTicks = 100;
	for (int32 i = 0; i < HoldTicks; ++i)
	{
		WHEN(InputIsTicked(Data));
		THEN(HoldingKeyTriggersAction(Data, TestAction));
	}

	// Releasing - does not fire canceled!
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	WHEN(InputIsTicked(Data));
	THEN(ReleasingKeyDoesNotTrigger(Data, TestAction));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputBindingAnalogTest, "Input.Binding.AnalogTrigger", BasicBindingTestFlags)

bool FInputBindingAnalogTest::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(ABasicBindingTest(this, EInputActionValueType::Axis1D));

	// Unpressed shouldn't trigger
	WHEN(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));
	AND(InputIsTicked(Data));
	THEN(!FInputTestHelper::TestTriggered(Data, TestAction));

	const TArray<float> TestValues = { 0.25f, 0.5f, 1.f, 1.f, -1.f, 0.75f, 0.5f, -0.1f};
	for(float TestValue : TestValues)
	{
		WHEN(AKeyIsActuated(Data, TestAxis, TestValue));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, TestAction));
		AND(TestEqual(TEXT("Trigger value"), FInputTestHelper::GetTriggered<float>(Data, TestAction), TestValue));
	}

	// Releasing - does not fire canceled!
	WHEN(AKeyIsReleased(Data, TestAxis));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputBindingMultipleKeyValuesTest, "Input.Binding.MultipleKeyValues", BasicBindingTestFlags)

bool FInputBindingMultipleKeyValuesTest::RunTest(const FString& Parameters)
{
	const int32 HoldTicks = 10;

	UControllablePlayer& Data =
	GIVEN(ABasicBindingTest(this, EInputActionValueType::Boolean));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey2));	// Bind second key
	AND(AKeyIsActuated(Data, TestKey));	// Depress first key

	// Holding for multiple ticks
	for (int32 i = 0; i < HoldTicks; ++i)
	{
		WHEN(InputIsTicked(Data));
		THEN(HoldingKeyTriggersAction(Data, TestAction));
		AND(TestEqual(TEXT("Single keypress generates a consistent value over multiple ticks"), FInputTestHelper::GetTriggered<FVector>(Data, TestAction), FVector(1.f, 0.f, 0.f)));
	}

	// Switch keys in a single tick
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(AKeyIsActuated(Data, TestKey2));

	// Holding for multiple ticks
	for (int32 i = 0; i < HoldTicks; ++i)
	{
		WHEN(InputIsTicked(Data));
		THEN(HoldingKeyTriggersAction(Data, TestAction));
		AND(TestEqual(TEXT("Switching keys in a single tick generates a consistent value"), FInputTestHelper::GetTriggered<FVector>(Data, TestAction), FVector(1.f, 0.f, 0.f)));
	}

	// Switch keys over 2 ticks
	WHEN(AKeyIsReleased(Data, TestKey2));
	AND(InputIsTicked(Data));
	AND(AKeyIsActuated(Data, TestKey));

	// Holding for multiple ticks
	for (int32 i = 0; i < HoldTicks; ++i)
	{
		WHEN(InputIsTicked(Data));
		THEN(HoldingKeyTriggersAction(Data, TestAction));
		AND(TestEqual(TEXT("Switching keys over two ticks generates a consistent value"), FInputTestHelper::GetTriggered<FVector>(Data, TestAction), FVector(1.f, 0.f, 0.f)));
	}

	// Release both keys
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	// Double key press on a single tick with hold
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(AKeyIsActuated(Data, TestKey2));
	for (int32 i = 0; i < HoldTicks; ++i)
	{
		WHEN(InputIsTicked(Data));
		THEN(HoldingKeyTriggersAction(Data, TestAction));
		AND(TestEqual(TEXT("Multiple key actuations on a single tick generates a consistent value"), FInputTestHelper::GetTriggered<FVector>(Data, TestAction), FVector(1.f, 0.f, 0.f)));
	}

	// Unpressed shouldn't trigger
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(AKeyIsReleased(Data, TestKey2));
	WHEN(InputIsTicked(Data));
	THEN(!FInputTestHelper::TestTriggered(Data, TestAction));

	return true;
}

/**
 * Creates an input component and pushes it to the given controller's input stack at the priority specified
 */
template<class InputCompClass>
InputCompClass* AnInputComponent(FAutomationTestBase* Test, UControllablePlayer& PlayerData, const int32 Priority)
{
	static_assert(std::is_base_of<UInputComponent, InputCompClass>::value, "AnInputComponent called with a class non derived from UInputComponent");
	
	Test->TestNotNull(TEXT("Valid player controller object"), PlayerData.Player.Get());

	InputCompClass* Component = NewObject<InputCompClass>(/*outer*/PlayerData.Player);
	Test->TestNotNull(TEXT("Successfully created an input comp"), Component);
	
	Component->Priority = Priority;
	
	PlayerData.Player->PushInputComponent(Component);
	
	return Component;
}

// Test to ensure that the order of input delegates is received based on the priority of the bound input component.
// This also covers that you can indeed bind to the same input action across multiple input components and receive
// it on the same frame.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputStackOrdering, "Input.Processing.InputStackOrder", BasicBindingTestFlags)
bool FInputStackOrdering::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(ABasicBindingTest(this, EInputActionValueType::Boolean));

	UInputAction* Action = FInputTestHelper::FindAction(Data, TestAction);
	
	// Push a test input component at different priorities
	UEnhancedInputComponent* TestComp_First = AnInputComponent<UEnhancedInputComponent>(this, Data, 10);
	UEnhancedInputComponent* TestComp_Second = AnInputComponent<UEnhancedInputComponent>(this, Data, 5);
	UEnhancedInputComponent* TestComp_Third = AnInputComponent<UEnhancedInputComponent>(this, Data, -1);

	int32 TestValue = 0;
	
	// Bind delegates to each component and test that the order of when the delegates is received is correct
	TestComp_First->BindActionValueLambda(Action, ETriggerEvent::Triggered, [this, &TestValue](const FInputActionValue& Val)
	{
		TestTrue(TEXT("The higher priority input component was processed first"), TestValue == 0);
		TestValue++;
	});

	TestComp_Second->BindActionValueLambda(Action, ETriggerEvent::Triggered, [this, &TestValue](const FInputActionValue& Val)
	{
			TestTrue(TEXT("The lower priority input component was processed second"), TestValue == 1);
			TestValue++;
	});

	TestComp_Third->BindActionValueLambda(Action, ETriggerEvent::Triggered, [this, &TestValue](const FInputActionValue& Val)
	{
			TestTrue(TEXT("The lowest priority input component was processed last"), TestValue == 2);
			TestValue++;
	});
	
	// Trigger the input action
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));

	// Ensure that all bound input delegates have fired
	TestTrue(TEXT("Both input component bindings were fired"), TestValue == 3);

	return true;
}

// Test of the UEnhancedInputComponent ability to consume delegates away from lower priority input components
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputStackConsumption, "Input.Processing.Consumption.Components", BasicBindingTestFlags)
bool FInputStackConsumption::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(ABasicBindingTest(this, EInputActionValueType::Boolean));

	UInputAction* Action = FInputTestHelper::FindAction(Data, TestAction);
	
	// Push a test input component at different priorities
	UEnhancedInputComponent* TestComp_First = AnInputComponent<UEnhancedInputComponent>(this, Data, 10);
	UEnhancedInputComponent* TestComp_Second = AnInputComponent<UEnhancedInputComponent>(this, Data, 5);

	int32 TestValue = 0;
	constexpr int32 SuccessfulTestValue = 47;
	
	// Bind a listener to the "Triggered" event for an action that should consume 
	FEnhancedInputActionEventBinding& FirstHandle = TestComp_First->BindActionValueLambda(Action, ETriggerEvent::Triggered, [this, &TestValue](const FInputActionValue& Val)
	{
		TestTrue(TEXT("The higher priority input component was processed first"), TestValue == 0);
		TestValue = SuccessfulTestValue;
	});
	
	FirstHandle.SetShouldConsume(true);

	TestTrue(TEXT("The bound delegate is correctly flagged to consume"), FirstHandle.ShouldConsume());

	FEnhancedInputActionEventBinding& SecondHandle = TestComp_Second->BindActionValueLambda(Action, ETriggerEvent::Triggered, [this, &TestValue](const FInputActionValue& Val)
	{
			TestFalse(TEXT("The lower priority input component was incorrectly triggered"), true);
			TestValue = -1;
	});

	TestFalse(TEXT("The second bound delegate is correctly flagged to NOT consume"), SecondHandle.ShouldConsume());
	
	// Trigger the input action
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));

	// Ensure that all bound input delegates have fired
	TestTrue(TEXT("Only the first delegate binding was fired"), TestValue == SuccessfulTestValue);

	return true;
}

// TODO: Need to check merging logic for keys generating different levels of "pressedness" activity
// TODO: Merging of multiple input mapping contexts including priority and actions of various bConsumeInput states


// A test of one mapping context of a higher priority blocking input to another mapping context which is a lower priority bound to the same key
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputStackConsumptionMappingContexts, "Input.Processing.Consumption.MappingContexts", BasicBindingTestFlags)
bool FInputStackConsumptionMappingContexts::RunTest(const FString& Parameters)
{
	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());

	UEnhancedInputComponent* InputComponent = Data.InputComponent.Get();
	TestNotNull(TEXT("Valid input component"),InputComponent);

	// Build IMC_HighPri (TestContext) with a mapping of TestAction to the X key with a priority of 10
	static FName IMC_HighPri_Name = TEXT("HighPriContext");
	UInputAction* Action_A = AnInputAction(Data, TestAction, EInputActionValueType::Boolean);
	TestNotNull("Mock Input Action", Action_A);
	UInputMappingContext* IMC_HighPri = AnInputContextIsAppliedToAPlayer(Data, IMC_HighPri_Name, 10);
	TestNotNull("Mock Input Mapping IMC_HighPri", IMC_HighPri);
	AnActionIsMappedToAKey(Data, IMC_HighPri_Name, TestAction, EKeys::X);
	
	// Build IMC_LowPri with a mapping to TestAction2 to the X key with a priority of 5
	static FName IMC_LowPri_Name = TEXT("LowPriContext");
	UInputAction* Action_B = AnInputAction(Data, TestAction2, EInputActionValueType::Boolean);
	TestNotNull("Mock Input Action", Action_B);
	UInputMappingContext* IMC_LowPri = AnInputContextIsAppliedToAPlayer(Data, IMC_LowPri_Name, 5);
	TestNotNull("Mock Input Mapping IMC_LowPri", IMC_LowPri);
	AnActionIsMappedToAKey(Data, IMC_LowPri_Name, TestAction2, EKeys::X);

	int32 TestValue = 0;
	constexpr int32 SuccessfulTestValue = 47;
	
	// Bind a listener to the "Triggered" event for an action that should consume 
	InputComponent->BindActionValueLambda(Action_A, ETriggerEvent::Triggered, [this, &TestValue](const FInputActionValue& Val)
	{
		TestTrue(TEXT("The higher priority input component was processed first"), TestValue == 0);
		TestValue = SuccessfulTestValue;
	});

	InputComponent->BindActionValueLambda(Action_B, ETriggerEvent::Triggered, [this, &TestValue](const FInputActionValue& Val)
	{
			TestFalse(TEXT("The lower priority input mapping context action was incorrectly triggered"), true);
			TestValue = -1;
	});
	
	// Trigger the input action
	WHEN(AKeyIsActuated(Data, EKeys::X));
	AND(InputIsTicked(Data));

	TestTrue(TEXT("Only the first delegate binding to the higher priority IMC was fired"), TestValue == SuccessfulTestValue);

	return true;
}

// A test to ensure that when you have two mapping contexts with bindings to the same key, but to different actions, with the same priority
// both input actions fire their delegates
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputStackConsumptionMappingContextsOfSamePri, "Input.Processing.Consumption.MappingContextsWithSamePri", BasicBindingTestFlags)
bool FInputStackConsumptionMappingContextsOfSamePri::RunTest(const FString& Parameters)
{
	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());

	UEnhancedInputComponent* InputComponent = Data.InputComponent.Get();
	TestNotNull(TEXT("Valid input component"),InputComponent);

	// Add two IMCs with the same key mapping of the same priority
	
	// Build IMC_HighPri (TestContext) with a mapping of TestAction to the X key with a priority of 11
	static FName IMC_HighPri_Name = TEXT("HighPriContext");
	UInputAction* Action_A = AnInputAction(Data, TestAction, EInputActionValueType::Boolean);
	TestNotNull("Mock Input Action", Action_A);
	UInputMappingContext* IMC_HighPri = AnInputContextIsAppliedToAPlayer(Data, IMC_HighPri_Name, 1);
	TestNotNull("Mock Input Mapping IMC_HighPri", IMC_HighPri);
	AnActionIsMappedToAKey(Data, IMC_HighPri_Name, TestAction, EKeys::X);
	
	// Build IMC_LowPri with a mapping to TestAction2 to the X key with a priority of 1
	static FName IMC_LowPri_Name = TEXT("LowPriContext");
	UInputAction* Action_B = AnInputAction(Data, TestAction2, EInputActionValueType::Boolean);
	TestNotNull("Mock Input Action", Action_B);
	UInputMappingContext* IMC_LowPri = AnInputContextIsAppliedToAPlayer(Data, IMC_LowPri_Name, 1);
	TestNotNull("Mock Input Mapping IMC_LowPri", IMC_LowPri);
	AnActionIsMappedToAKey(Data, IMC_LowPri_Name, TestAction2, EKeys::X);

	int32 TestValue = 0;

	constexpr int32 SuccessfulBinding_A = 2;
	constexpr int32 SuccessfulBinding_B = 7;
	constexpr int32 SuccessfulBinding_Total = SuccessfulBinding_A + SuccessfulBinding_B;
	
	// Bind a listener to the "Triggered" event for an action that should consume 
	InputComponent->BindActionValueLambda(Action_A, ETriggerEvent::Triggered, [&TestValue](const FInputActionValue&)
	{
		TestValue += SuccessfulBinding_A;
	});

	InputComponent->BindActionValueLambda(Action_B, ETriggerEvent::Triggered, [&TestValue](const FInputActionValue&)
	{
		TestValue += SuccessfulBinding_B;
	});
	
	// Trigger the input action
	WHEN(AKeyIsActuated(Data, EKeys::X));
	AND(InputIsTicked(Data));

	// Ensure that all bound input delegates have fired
	// This behavior is confusing and I want to change it, but at least this test will document what the expectation is to our users.
	TestTrue(TEXT("The mapping context which was most recently added is the one called."), TestValue == SuccessfulBinding_B);

	return true;
}