// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputTestFramework.h"
#include "Misc/AutomationTest.h"

constexpr auto BasicIntegrationTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter;	// TODO: Run as Smoke/Client? No world on RunSmokeTests startup...

static UInputTrigger* GActionTrigger = nullptr;	// Basic action level trigger applied by ABasicTriggerTest setup.

template<typename T>
UControllablePlayer& ABasicActionTriggerTest(FAutomationTestBase* Test, EInputActionValueType ForValueType = EInputActionValueType::Axis1D)
{
	UWorld* World =
	GIVEN(AnEmptyWorld());

	// Initialise
	UControllablePlayer& Data =
	GIVEN(AControllablePlayer(World));
	Test->TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());	// TODO: Can we early out on a failed Test?

	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputAction(Data, TestAction, ForValueType));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));
	GActionTrigger =
	AND(ATriggerIsAppliedToAnAction(Data, NewObject<T>(), TestAction));

	// TODO: Are UObjects within Data safe from garbage collection? Don't think so.
	return Data;
}

// ***************************************
// Action level trigger integration tests
// ***************************************

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputIntegrationTriggerPressedTest, "Input.Integration.Triggers.Pressed", BasicIntegrationTestFlags)

bool FInputIntegrationTriggerPressedTest::RunTest(const FString& Parameters)
{
	// Pressing
	GIVEN(UControllablePlayer& Data = ABasicActionTriggerTest<UInputTriggerPressed>(this));
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	// Holding
	WHEN(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	WHEN(InputIsTicked(Data));
	THEN(HoldingKeyDoesNotTrigger(Data, TestAction));

	// Releasing
	WHEN(AKeyIsReleased(Data, TestKey));
	THEN(ReleasingKeyDoesNotTrigger(Data, TestAction));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputIntegrationTriggerDownTest, "Input.Integration.Triggers.Down", BasicIntegrationTestFlags)

bool FInputIntegrationTriggerDownTest::RunTest(const FString& Parameters)
{
	// Pressing
	GIVEN(UControllablePlayer& Data = ABasicActionTriggerTest<UInputTriggerDown>(this));
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	// Holding for multiple ticks
	const int32 HoldTicks = 100;
	for (int32 i = 0; i < HoldTicks; ++i)
	{
		// Holding
		WHEN(InputIsTicked(Data));
		THEN(HoldingKeyTriggersAction(Data, TestAction));
	}

	// Releasing
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	WHEN(InputIsTicked(Data));
	THEN(ReleasingKeyDoesNotTrigger(Data, TestAction));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputIntegrationTriggerReleasedTest, "Input.Integration.Triggers.Released", BasicIntegrationTestFlags)

bool FInputIntegrationTriggerReleasedTest::RunTest(const FString& Parameters)
{
	// Pressing
	GIVEN(UControllablePlayer& Data = ABasicActionTriggerTest<UInputTriggerReleased>(this));
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	// Holding
	WHEN(InputIsTicked(Data));
	THEN(HoldingKeyTriggersOngoing(Data, TestAction));

	// Releasing
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersAction(Data, TestAction));

	WHEN(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	WHEN(InputIsTicked(Data));
	THEN(ReleasingKeyDoesNotTrigger(Data, TestAction));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputIntegrationTriggerHoldTest, "Input.Integration.Triggers.Hold", BasicIntegrationTestFlags)

bool FInputIntegrationTriggerHoldTest::RunTest(const FString& Parameters)
{
	const float FrameTime = 1.f / 60.f;
	const int HoldFrames = 30;	// Half second hold

	GIVEN(UControllablePlayer& Data = ABasicActionTriggerTest<UInputTriggerHold>(this));

	UInputTriggerHold* HoldTrigger = Cast<UInputTriggerHold>(GActionTrigger);
	TestNotNull(TEXT("Succesfully created a Hold trigger"), HoldTrigger);
	HoldTrigger->HoldTimeThreshold = FrameTime * HoldFrames;
	HoldTrigger->bIsOneShot = false;

	// Test 1 - Releasing before threshold frame cancels

	// Pressing
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	// Releasing cancels
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersCanceled(Data, TestAction));

	// But only once!
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyDoesNotTrigger(Data, TestAction));

	// Test 2 - Holding to threshold frame triggers
	InputIsTicked(Data, FrameTime);	// Clear state

	// Pressing
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	// Holding until trigger frame
	for (int HoldFrame = 1; HoldFrame < HoldFrames - 1; ++HoldFrame)
	{
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersOngoing(Data, TestAction));
	}

	// Holding over the trigger frame - hold threshold is inclusive
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(HoldingKeyTriggersAction(Data, TestAction));

	// Holding for a further frame continues to trigger
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(HoldingKeyTriggersAction(Data, TestAction));

	// Releasing
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	WHEN(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyDoesNotTrigger(Data, TestAction));

	return true;
}

