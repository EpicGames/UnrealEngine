// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputTestFramework.h"
#include "Misc/AutomationTest.h"
#include "InputModifiers.h"
#include "EnhancedActionKeyMapping.h"

// Extended modifier tests covering SwizzleAxis, per-axis Negate, Radial DeadZone,
// ResponseCurveExponential, ScaleByDeltaTime, and modifier application order.

constexpr auto BasicModifierExtTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter;

// Helper: set up a world+player+context+action, return a Data reference.
static UControllablePlayer& ABasicModifierExtTest(FAutomationTestBase* Test, EInputActionValueType ForValueType)
{
	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	Test->TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());
	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputAction(Data, TestAction, ForValueType));

	return Data;
}

// Helper: read the float value from action instance data.
static float GetModExtActionValue(UControllablePlayer& Data, FName ActionName)
{
	return FInputTestHelper::GetActionData(Data, ActionName).GetValue().Get<float>();
}

// FVector::X/Y/Z are double in UE5. These helpers avoid ambiguous TestEqual overload resolution.
static float FX(const FVector& V) { return (float)V.X; }
static float FY(const FVector& V) { return (float)V.Y; }
static float FZ(const FVector& V) { return (float)V.Z; }

// ============================================================
// Swizzle Axis
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifierSwizzleAxisTest, "Input.Modifiers.SwizzleAxis", BasicModifierExtTestFlags)

bool FInputModifierSwizzleAxisTest::RunTest(const FString& Parameters)
{
	// Test 1 - YXZ: swap X and Y.  Most common use: drive the Y component of a 2D action from a 1D analog input.
	{
		UControllablePlayer& Data =
		GIVEN(ABasicModifierExtTest(this, EInputActionValueType::Axis2D));
		AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, EKeys::Gamepad_Left2D));

		UInputModifierSwizzleAxis* Swizzle =
		AND(Cast<UInputModifierSwizzleAxis>(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierSwizzleAxis>(), TestContext, TestAction, EKeys::Gamepad_Left2D)));
		check(Swizzle);
		Swizzle->Order = EInputAxisSwizzle::YXZ;

		// Drive LeftX=0.6, LeftY=0.8 -> raw FVector(0.6, 0.8, 0) -> after YXZ swap -> FVector(0.8, 0.6, 0)
		WHEN(AKeyIsActuated(Data, EKeys::Gamepad_LeftX, 0.6f));
		AND(AKeyIsActuated(Data, EKeys::Gamepad_LeftY, 0.8f));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, TestAction));

		FVector Result = FInputTestHelper::GetTriggered<FVector>(Data, TestAction);
		AND(TestEqual(TEXT("YXZ swap: X becomes Y"), FX(Result), 0.8f));
		AND(TestEqual(TEXT("YXZ swap: Y becomes X"), FY(Result), 0.6f));
	}

	// Test 2 - ZYX: swap X and Z.  Useful when a 1D key should drive the Z component of a 3D action.
	// A 1D key produces FVector(val, 0, 0).  After ZYX swap: FVector(0, 0, val).
	{
		UControllablePlayer& Data =
		GIVEN(ABasicModifierExtTest(this, EInputActionValueType::Axis3D));
		AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));

		UInputModifierSwizzleAxis* Swizzle =
		AND(Cast<UInputModifierSwizzleAxis>(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierSwizzleAxis>(), TestContext, TestAction, TestAxis)));
		check(Swizzle);
		Swizzle->Order = EInputAxisSwizzle::ZYX;

		// TestAxis is a 1D analog -> FVector(0.7, 0, 0) -> ZYX swap X and Z -> FVector(0, 0, 0.7)
		WHEN(AKeyIsActuated(Data, TestAxis, 0.7f));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, TestAction));

		FVector Result = FInputTestHelper::GetTriggered<FVector>(Data, TestAction);
		AND(TestEqual(TEXT("ZYX: X moved to Z"), FX(Result), 0.f));
		AND(TestEqual(TEXT("ZYX: Y unchanged (zero)"), FY(Result), 0.f));
		AND(TestEqual(TEXT("ZYX: Z receives X value"), FZ(Result), 0.7f));
	}

	// Test 3 - XZY: swap Y and Z.  A 1D key (X-only) should be unaffected, confirming correct axis targeting.
	// FVector(0.5, 0, 0) -> XZY swap Y and Z -> FVector(0.5, 0, 0) (no change when Y=Z=0)
	{
		UControllablePlayer& Data =
		GIVEN(ABasicModifierExtTest(this, EInputActionValueType::Axis3D));
		AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));

		UInputModifierSwizzleAxis* Swizzle =
		AND(Cast<UInputModifierSwizzleAxis>(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierSwizzleAxis>(), TestContext, TestAction, TestAxis)));
		check(Swizzle);
		Swizzle->Order = EInputAxisSwizzle::XZY;

		WHEN(AKeyIsActuated(Data, TestAxis, 0.5f));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, TestAction));

		FVector Result = FInputTestHelper::GetTriggered<FVector>(Data, TestAction);
		AND(TestEqual(TEXT("XZY: X unchanged"), FX(Result), 0.5f));
		AND(TestEqual(TEXT("XZY: Y and Z both zero (swapped zeros)"), FY(Result), 0.f));
		AND(TestEqual(TEXT("XZY: Z unchanged (swapped from zero Y)"), FZ(Result), 0.f));
	}

	return true;
}

// ============================================================
// Negate per-axis flags
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifierNegatePerAxisTest, "Input.Modifiers.Negate.PerAxis", BasicModifierExtTestFlags)

bool FInputModifierNegatePerAxisTest::RunTest(const FString& Parameters)
{
	// Use Axis2D so we have independent X and Y values to negate selectively.
	// Drive LeftX=0.5, LeftY=0.5 -> raw FVector(0.5, 0.5, 0).

	// Test 1 - negate X only
	{
		UControllablePlayer& Data =
		GIVEN(ABasicModifierExtTest(this, EInputActionValueType::Axis2D));
		AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, EKeys::Gamepad_Left2D));

		UInputModifierNegate* Negate =
		AND(Cast<UInputModifierNegate>(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierNegate>(), TestContext, TestAction, EKeys::Gamepad_Left2D)));
		check(Negate);
		Negate->bX = true;
		Negate->bY = false;
		Negate->bZ = false;

		WHEN(AKeyIsActuated(Data, EKeys::Gamepad_LeftX, 0.5f));
		AND(AKeyIsActuated(Data, EKeys::Gamepad_LeftY, 0.5f));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, TestAction));

		FVector Result = FInputTestHelper::GetTriggered<FVector>(Data, TestAction);
		AND(TestEqual(TEXT("Negate X only: X negated"), FX(Result), -0.5f));
		AND(TestEqual(TEXT("Negate X only: Y unchanged"), FY(Result), 0.5f));
	}

	// Test 2 - negate Y only
	{
		UControllablePlayer& Data =
		GIVEN(ABasicModifierExtTest(this, EInputActionValueType::Axis2D));
		AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, EKeys::Gamepad_Left2D));

		UInputModifierNegate* Negate =
		AND(Cast<UInputModifierNegate>(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierNegate>(), TestContext, TestAction, EKeys::Gamepad_Left2D)));
		check(Negate);
		Negate->bX = false;
		Negate->bY = true;
		Negate->bZ = false;

		WHEN(AKeyIsActuated(Data, EKeys::Gamepad_LeftX, 0.5f));
		AND(AKeyIsActuated(Data, EKeys::Gamepad_LeftY, 0.5f));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, TestAction));

		FVector Result = FInputTestHelper::GetTriggered<FVector>(Data, TestAction);
		AND(TestEqual(TEXT("Negate Y only: X unchanged"), FX(Result), 0.5f));
		AND(TestEqual(TEXT("Negate Y only: Y negated"), FY(Result), -0.5f));
	}

	// Test 3 - negate both X and Y
	{
		UControllablePlayer& Data =
		GIVEN(ABasicModifierExtTest(this, EInputActionValueType::Axis2D));
		AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, EKeys::Gamepad_Left2D));

		UInputModifierNegate* Negate =
		AND(Cast<UInputModifierNegate>(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierNegate>(), TestContext, TestAction, EKeys::Gamepad_Left2D)));
		check(Negate);
		Negate->bX = true;
		Negate->bY = true;
		Negate->bZ = false;

		WHEN(AKeyIsActuated(Data, EKeys::Gamepad_LeftX, 0.5f));
		AND(AKeyIsActuated(Data, EKeys::Gamepad_LeftY, 0.5f));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, TestAction));

		FVector Result = FInputTestHelper::GetTriggered<FVector>(Data, TestAction);
		AND(TestEqual(TEXT("Negate X+Y: X negated"), FX(Result), -0.5f));
		AND(TestEqual(TEXT("Negate X+Y: Y negated"), FY(Result), -0.5f));
	}

	// Test 4 - no axes negated (all false): pass-through
	{
		UControllablePlayer& Data =
		GIVEN(ABasicModifierExtTest(this, EInputActionValueType::Axis2D));
		AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, EKeys::Gamepad_Left2D));

		UInputModifierNegate* Negate =
		AND(Cast<UInputModifierNegate>(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierNegate>(), TestContext, TestAction, EKeys::Gamepad_Left2D)));
		check(Negate);
		Negate->bX = false;
		Negate->bY = false;
		Negate->bZ = false;

		WHEN(AKeyIsActuated(Data, EKeys::Gamepad_LeftX, 0.5f));
		AND(AKeyIsActuated(Data, EKeys::Gamepad_LeftY, 0.5f));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, TestAction));

		FVector Result = FInputTestHelper::GetTriggered<FVector>(Data, TestAction);
		AND(TestEqual(TEXT("No negate: X unchanged"), FX(Result), 0.5f));
		AND(TestEqual(TEXT("No negate: Y unchanged"), FY(Result), 0.5f));
	}

	return true;
}

// ============================================================
// Dead Zone - Radial type
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifierDeadZoneRadialTest, "Input.Modifiers.DeadZone.Radial", BasicModifierExtTestFlags)

bool FInputModifierDeadZoneRadialTest::RunTest(const FString& Parameters)
{
	// Radial dead zone applied at the action level so it sees the full assembled 2D vector.
	UControllablePlayer& Data =
	GIVEN(ABasicModifierExtTest(this, EInputActionValueType::Axis2D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, EKeys::Gamepad_Left2D));

	UInputModifierDeadZone* DeadZone =
	AND(Cast<UInputModifierDeadZone>(AModifierIsAppliedToAnAction(Data, NewObject<UInputModifierDeadZone>(), TestAction)));
	check(DeadZone);
	DeadZone->LowerThreshold = 0.2f;
	DeadZone->UpperThreshold = 1.0f;
	DeadZone->Type = EDeadZoneType::Radial;

	// Test 1 - Input below the radial threshold: magnitude 0.1 < 0.2 -> no trigger
	WHEN(AKeyIsActuated(Data, EKeys::Gamepad_LeftX, 0.1f));
	AND(AKeyIsActuated(Data, EKeys::Gamepad_LeftY, 0.f));
	AND(InputIsTicked(Data));
	THEN(PressingKeyDoesNotTrigger(Data, TestAction));

	// Test 2 - Input above the radial threshold: LeftX=0.3, LeftY=0.4 -> magnitude=0.5 > 0.2
	// Expected: normal=(0.6,0.8), scaled_mag = (0.5-0.2)/(1.0-0.2) = 0.375
	// Result = (0.6*0.375, 0.8*0.375) = (0.225, 0.3)
	const float InputX = 0.3f;
	const float InputY = 0.4f;
	const float Magnitude = FMath::Sqrt(InputX * InputX + InputY * InputY); // 0.5
	const float ScaledMag = FMath::Min(1.f, FMath::Max(0.f, Magnitude - DeadZone->LowerThreshold) / (DeadZone->UpperThreshold - DeadZone->LowerThreshold));
	const float ExpectedX = (InputX / Magnitude) * ScaledMag;
	const float ExpectedY = (InputY / Magnitude) * ScaledMag;

	WHEN(AKeyIsActuated(Data, EKeys::Gamepad_LeftX, InputX));
	AND(AKeyIsActuated(Data, EKeys::Gamepad_LeftY, InputY));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	FVector Result = FInputTestHelper::GetTriggered<FVector>(Data, TestAction);
	AND(TestEqual(TEXT("Radial deadzone: X component"), FX(Result), ExpectedX));
	AND(TestEqual(TEXT("Radial deadzone: Y component"), FY(Result), ExpectedY));

	// Test 3 - Input at exactly the upper threshold: magnitude 1.0, scaled_mag = 1.0, result unchanged
	WHEN(AKeyIsActuated(Data, EKeys::Gamepad_LeftX, 0.6f));
	AND(AKeyIsActuated(Data, EKeys::Gamepad_LeftY, 0.8f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));

	Result = FInputTestHelper::GetTriggered<FVector>(Data, TestAction);
	AND(TestEqual(TEXT("Radial deadzone: at upper threshold X"), FX(Result), 0.6f));
	AND(TestEqual(TEXT("Radial deadzone: at upper threshold Y"), FY(Result), 0.8f));

	// Test 4 - Release should stop triggering
	WHEN(AKeyIsReleased(Data, EKeys::Gamepad_LeftX));
	AND(AKeyIsReleased(Data, EKeys::Gamepad_LeftY));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	return true;
}

// ============================================================
// Response Curve Exponential
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifierResponseCurveExponentialTest, "Input.Modifiers.ResponseCurveExponential", BasicModifierExtTestFlags)

bool FInputModifierResponseCurveExponentialTest::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(ABasicModifierExtTest(this, EInputActionValueType::Axis1D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));

	UInputModifierResponseCurveExponential* Curve =
	AND(Cast<UInputModifierResponseCurveExponential>(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierResponseCurveExponential>(), TestContext, TestAction, TestAxis)));
	check(Curve);

	// Test 1 - Exponent 1.0 (identity): output == input
	Curve->CurveExponent = FVector(1.f, 1.f, 1.f);
	WHEN(AKeyIsActuated(Data, TestAxis, 0.5f));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("Exponent 1.0: identity"), GetModExtActionValue(Data, TestAction), 0.5f));

	// Test 2 - Exponent 2.0: output = sign(x) * pow(|x|, 2) = 0.25 for input 0.5
	Curve->CurveExponent = FVector(2.f, 1.f, 1.f);
	WHEN(AKeyIsActuated(Data, TestAxis, 0.5f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("Exponent 2.0 positive input"), GetModExtActionValue(Data, TestAction), 0.25f));

	// Test 3 - Exponent 2.0 with negative input: sign is preserved (-0.25 for -0.5)
	WHEN(AKeyIsActuated(Data, TestAxis, -0.5f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("Exponent 2.0 negative input"), GetModExtActionValue(Data, TestAction), -0.25f));

	// Test 4 - Exponent 2.0 at input 1.0: pow(1, 2) = 1.0 (boundary unchanged)
	WHEN(AKeyIsActuated(Data, TestAxis, 1.0f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("Exponent 2.0 at max input"), GetModExtActionValue(Data, TestAction), 1.0f));

	// Test 5 - Exponent 0.5 (square root): output = sqrt(0.25) = 0.5 for input 0.25 (expands the curve)
	Curve->CurveExponent = FVector(0.5f, 1.f, 1.f);
	WHEN(AKeyIsActuated(Data, TestAxis, 0.25f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("Exponent 0.5 (sqrt): value expanded"), GetModExtActionValue(Data, TestAction), 0.5f));

	// Release
	WHEN(AKeyIsReleased(Data, TestAxis));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	return true;
}

// ============================================================
// Scale By Delta Time
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifierScaleByDeltaTimeTest, "Input.Modifiers.ScaleByDeltaTime", BasicModifierExtTestFlags)

bool FInputModifierScaleByDeltaTimeTest::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(ABasicModifierExtTest(this, EInputActionValueType::Axis1D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));
	AND(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierScaleByDeltaTime>(), TestContext, TestAction, TestAxis));

	// Test 1 - Input 1.0 with DeltaTime=1/60: output = 1.0 * (1/60)
	const float FrameTime = 1.f / 60.f;
	WHEN(AKeyIsActuated(Data, TestAxis, 1.0f));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("Scale by DeltaTime at 60fps"), GetModExtActionValue(Data, TestAction), 1.0f * FrameTime));

	// Test 2 - Same input with DeltaTime=1/30 (half frame rate): output doubles
	const float HalfRate = 1.f / 30.f;
	WHEN(InputIsTicked(Data, HalfRate));
	THEN(HoldingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("Scale by DeltaTime at 30fps"), GetModExtActionValue(Data, TestAction), 1.0f * HalfRate));

	// Test 3 - Different input magnitude: 0.5 * (1/60)
	WHEN(AKeyIsActuated(Data, TestAxis, 0.5f));
	AND(InputIsTicked(Data, FrameTime));
	THEN(HoldingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("Scale by DeltaTime: half magnitude"), GetModExtActionValue(Data, TestAction), 0.5f * FrameTime));

	// Release
	WHEN(AKeyIsReleased(Data, TestAxis));
	AND(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	return true;
}

// ============================================================
// Modifier application order: mapping-level runs before action-level
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifierChainOrderTest, "Input.Modifiers.ApplicationOrder", BasicModifierExtTestFlags)

bool FInputModifierChainOrderTest::RunTest(const FString& Parameters)
{
	// Mapping-level modifier: Scalar(3x)
	// Action-level modifier: DeadZone(LowerThreshold=0.5)
	// Input: 0.2
	//
	// If mapping runs first:  0.2 * 3 = 0.6 -> deadzone((0.6-0.5)/(1.0-0.5)) = 0.2 -> TRIGGERED with value 0.2
	// If action runs first:   deadzone(0.2) = 0.0 (below threshold) -> 0.0 * 3 = 0.0 -> NOT TRIGGERED
	//
	// The expected UE behavior is mapping modifiers before action modifiers.
	UControllablePlayer& Data =
	GIVEN(ABasicModifierExtTest(this, EInputActionValueType::Axis1D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));

	// Pre-configure the Scalar so the deep-copy during RebuildControlMappings captures the 3x value.
	// Each ControlMappingsAreRebuilt call deep-copies context modifiers, invalidating any prior live pointer.
	// By setting the value on the original BEFORE applying, subsequent rebuilds produce correct duplicates.
	UInputModifierScalar* OrigScalar = NewObject<UInputModifierScalar>();
	OrigScalar->Scalar = FVector(3.f, 3.f, 3.f);
	AND(AModifierIsAppliedToAnActionMapping(Data, OrigScalar, TestContext, TestAction, TestAxis));

	// Pre-configure the DeadZone similarly before its rebuild.
	UInputModifierDeadZone* OrigDeadZone = NewObject<UInputModifierDeadZone>();
	OrigDeadZone->LowerThreshold = 0.5f;
	OrigDeadZone->UpperThreshold = 1.0f;
	OrigDeadZone->Type = EDeadZoneType::Axial;
	AND(AModifierIsAppliedToAnAction(Data, OrigDeadZone, TestAction));

	// After the last rebuild, get a fresh pointer to the live Scalar for use in sub-tests.
	FEnhancedActionKeyMapping* LiveMap = FInputTestHelper::FindLiveActionMapping(Data, TestAction, TestAxis);
	check(LiveMap && !LiveMap->Modifiers.IsEmpty());
	UInputModifierScalar* LiveScalar = Cast<UInputModifierScalar>(LiveMap->Modifiers[0]);
	check(LiveScalar);

	// Input 0.2: must be scaled to 0.6 by the mapping modifier BEFORE hitting the deadzone.
	WHEN(AKeyIsActuated(Data, TestAxis, 0.2f));
	AND(InputIsTicked(Data));

	// Should trigger because 0.2 -> 0.6 (above deadzone lower threshold of 0.5)
	THEN(PressingKeyTriggersAction(Data, TestAction));
	// Value after deadzone: (0.6-0.5)/(1.0-0.5) = 0.2
	AND(TestEqual(TEXT("Mapping modifier ran before action modifier"), GetModExtActionValue(Data, TestAction), 0.2f));

	// Test 2 - Set scalar to identity so raw 0.2 hits deadzone directly.
	// LastState=Triggered (from Test 1), 0.2 now killed by deadzone -> TriggerState=None -> Triggered->None=Completed.
	LiveScalar->Scalar = FVector(1.f, 1.f, 1.f);
	WHEN(AKeyIsActuated(Data, TestAxis, 0.2f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersCompleted(Data, TestAction)); // Triggered->None transition when value drops below deadzone

	// Restore scale: action returns from None -> StartedAndTriggered fires.
	LiveScalar->Scalar = FVector(3.f, 3.f, 3.f);
	WHEN(AKeyIsActuated(Data, TestAxis, 0.2f));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	return true;
}
