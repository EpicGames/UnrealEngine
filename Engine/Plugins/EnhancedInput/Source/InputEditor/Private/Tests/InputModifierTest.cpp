// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputTestFramework.h"
#include "Algo/MaxElement.h"
#include "Misc/AutomationTest.h"
#include "EnhancedInputModule.h"
#include "ProfilingDebugging/ScopedTimers.h"
// Tests focused on individual modifiers


// THEN step wrappers to give human readable test failure output.
bool TestActionIsActuated(UControllablePlayer& Data, FName ActionName)
{
	return FInputTestHelper::GetActionData(Data, ActionName).GetValue().Get<bool>();
}

bool TestActionIsNotActuated(UControllablePlayer& Data, FName ActionName)
{
	return !FInputTestHelper::GetActionData(Data, ActionName).GetValue().Get<bool>();
}

float GetActionValue(UControllablePlayer& Data, FName ActionName)
{
	return FInputTestHelper::GetActionData(Data, ActionName).GetValue().Get<float>();
}

constexpr auto BasicModifierTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter;	// TODO: Run as Smoke/Client? No world on RunSmokeTests startup...

UControllablePlayer& ABasicModifierTest(FAutomationTestBase* Test, EInputActionValueType ForValueType)
{
	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	Test->TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());	// TODO: Can we early out on a failed Test?
	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	UInputAction* Action =
	AND(AnInputAction(Data, TestAction, ForValueType));

	return Data;
}


// ******************************
// Value modification tests
// ******************************

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifieNegateTest, "Input.Modifiers.Negate", BasicModifierTestFlags)

bool FInputModifieNegateTest::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(ABasicModifierTest(this, EInputActionValueType::Axis1D));

	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));
	AND(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierNegate>(), TestContext, TestAction, TestKey));

	// Bool key tests. Negating a false/true bool should give -0/-1, not true/false, allowing driving a negative axis movement.
	// An unactuated bool input should always return false.

	// Test 1 - By default, output is false (0) (unactuated bools always return false)
	WHEN(InputIsTicked(Data));
	THEN(TestActionIsNotActuated(Data, TestAction));

	// Test 2 - Key press/hold/release

	// Actuated output is true (-1) when key is down
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(TestActionIsActuated(Data, TestAction));
	AND(TestEqual(TEXT("Key actuated press value"), GetActionValue(Data, TestAction), -1.f));

	// Actuated output remains true (-1) next tick
	WHEN(InputIsTicked(Data));
	THEN(TestActionIsActuated(Data, TestAction));
	AND(TestEqual(TEXT("Key actuated hold value"), GetActionValue(Data, TestAction), -1.f));

	// Releasing the key reverts to false (0)
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(TestActionIsNotActuated(Data, TestAction));

	InputIsTicked(Data);// Clear state

	// Test 3 - Axis press/hold/release
	WHEN(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));
	AND(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierNegate>(), TestContext, TestAction, TestAxis));
	AND(AKeyIsActuated(Data, TestAxis, 0.5f));
	AND(InputIsTicked(Data));
	THEN(TestActionIsActuated(Data, TestAction));
	AND(TestEqual(TEXT("Axis inverted press value"), GetActionValue(Data, TestAction), -0.5f));

	// Actuated output remains constant next tick
	WHEN(InputIsTicked(Data));
	AND(TestEqual(TEXT("Axis inverted hold value"), GetActionValue(Data, TestAction), -0.5f));

	// Actuation stops when axis is released
	WHEN(AKeyIsReleased(Data, TestAxis));
	AND(InputIsTicked(Data));
	THEN(TestActionIsNotActuated(Data, TestAction));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifierScalarTest, "Input.Modifiers.Scalar", BasicModifierTestFlags)

bool FInputModifierScalarTest::RunTest(const FString& Parameters)
{
	GIVEN(UControllablePlayer& Data = ABasicModifierTest(this, EInputActionValueType::Axis1D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));
	AND(UInputModifierScalar* Scalar = Cast<UInputModifierScalar>(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierScalar>(), TestContext, TestAction, TestAxis)));
	check(Scalar);

	// Test 1 - By default (no input), output is 0
	WHEN(InputIsTicked(Data));
	TestEqual(TEXT("No input value"), GetActionValue(Data, TestAction), 0.f);

	// Test 2 - By default scale is 1. When actuated output == actuated value
	WHEN(AKeyIsActuated(Data, TestAxis, 0.5f));
	AND(InputIsTicked(Data));
	TestEqual(TEXT("Input value"), GetActionValue(Data, TestAction), 0.5f);

	// Multi value tests
	TArray<float> TestValues = { 0.5f, -0.5f, 2.f, -1000.f, 0.f };
	for(float TestValue : TestValues)
	{
		WHEN(AKeyIsActuated(Data, TestAxis, TestValue));
		Scalar->Scalar = FVector::OneVector * 1.f;
		AND(InputIsTicked(Data));
		TestEqual(TEXT("Input value (new)"), GetActionValue(Data, TestAction), TestValue * (float)Scalar->Scalar.X);

		// Test 3 - Modify scalar on the fly.
		Scalar->Scalar = FVector::OneVector * 0.5f;
		WHEN(InputIsTicked(Data));
		TestEqual(TEXT("Input value (modify)"), GetActionValue(Data, TestAction), TestValue * (float)Scalar->Scalar.X);

		// Test 4 - negate
		Scalar->Scalar = FVector::OneVector * -2.f;
		WHEN(InputIsTicked(Data));
		TestEqual(TEXT("Input value (negate)"), GetActionValue(Data, TestAction), TestValue * (float)Scalar->Scalar.X);
	}

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifierDeadzoneTest, "Input.Modifiers.DeadZone", BasicModifierTestFlags)

bool FInputModifierDeadzoneTest::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(ABasicModifierTest(this, EInputActionValueType::Axis1D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));

	UInputModifierDeadZone* DeadZone =
	AND(Cast<UInputModifierDeadZone>(AModifierIsAppliedToAnAction(Data, NewObject<UInputModifierDeadZone>(), TestAction)));
	DeadZone->LowerThreshold = 0.1f;

	// Provide initial valid input
	WHEN(AKeyIsActuated(Data, TestAxis, 1.f));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	// Drop below deadzone
	WHEN(AKeyIsActuated(Data, TestAxis, DeadZone->LowerThreshold * 0.5f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersCompleted(Data, TestAction));

	// Jitter samples below deadzone lower threshold
	TArray<float> Sample = { 0.01f, 0.0f, 0.02f, 0.07f, 0.01f };
	const int NumSamples = 50;
	for (int32 i = 0; i < NumSamples; ++i)
	{
		WHEN(AKeyIsActuated(Data, TestAxis, Sample[i % Sample.Num()]));
		AND(InputIsTicked(Data));
		THEN(HoldingKeyDoesNotTrigger(Data, TestAction));
	}

	// No noise on release
	WHEN(AKeyIsReleased(Data, TestAxis));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyDoesNotTrigger(Data, TestAction));


	// Upper threshold testing
	DeadZone->UpperThreshold = 0.9f;
	WHEN(AKeyIsActuated(Data, TestAxis, 0.5f));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	// At threshold response is 1
	WHEN(AKeyIsActuated(Data, TestAxis, 0.9f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("Upper threshold value at threshold"), GetActionValue(Data, TestAction), 1.f));

	// Past threshold response is clamped to 1
	WHEN(AKeyIsActuated(Data, TestAxis, 0.99f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("Upper threshold value beyond threshold"), GetActionValue(Data, TestAction), 1.f));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifierUnscaledRadialDeadzoneTest, "Input.Modifiers.UnscaledRadialDeadZone", BasicModifierTestFlags)
bool FInputModifierUnscaledRadialDeadzoneTest::RunTest(const FString& Parameters)
{
	// A 1D test axis with a positive value
	{
		UControllablePlayer& Data =
			GIVEN(ABasicModifierTest(this, EInputActionValueType::Axis1D));
		AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis2));

		UInputModifierDeadZone* DeadZone =
			AND(Cast<UInputModifierDeadZone>(AModifierIsAppliedToAnAction(Data, NewObject<UInputModifierDeadZone>(), TestAction)));
		DeadZone->LowerThreshold = 0.1f;
		DeadZone->Type = EDeadZoneType::UnscaledRadial;

		// Provide a negative value which should still be within the deadzone
		WHEN(AKeyIsActuated(Data, TestAxis2, 0.5f));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, TestAction));
		AND(TestEqual(TEXT("Positive Value is correct"), GetActionValue(Data, TestAction), 0.5f));
	}	

	// A 1D test axis with a negative value
	{
		UControllablePlayer& Data =
			GIVEN(ABasicModifierTest(this, EInputActionValueType::Axis1D));
		AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis2));

		UInputModifierDeadZone* DeadZone =
			AND(Cast<UInputModifierDeadZone>(AModifierIsAppliedToAnAction(Data, NewObject<UInputModifierDeadZone>(), TestAction)));
		DeadZone->LowerThreshold = 0.1f;
		DeadZone->Type = EDeadZoneType::UnscaledRadial;

		// Provide a negative value which should still be within the deadzone
		WHEN(AKeyIsActuated(Data, TestAxis2, -0.5f));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, TestAction));
		AND(TestEqual(TEXT("Value is correct"), GetActionValue(Data, TestAction), -0.5f));
	}
	
	return true;
}

/**
 * A simple "stress test" for Enhanced Input modifiers that we can use to measure the performance
 * of applying multiple UInputModifiers to a single key mapping.
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifierPerformanceTest, "Input.Modifiers.Performance", BasicModifierTestFlags)

bool FInputModifierPerformanceTest::RunTest(const FString& Parameters)
{
	GIVEN(UControllablePlayer& Data = ABasicModifierTest(this, EInputActionValueType::Axis3D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));

	// Apply a few different types of input modifiers to this key
	AND(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierScalar>(), TestContext, TestAction, TestAxis));
	AND(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierNegate>(), TestContext, TestAction, TestAxis));
	AND(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierDeadZone>(), TestContext, TestAction, TestAxis));
	AND(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierSwizzleAxis>(), TestContext, TestAction, TestAxis));
	AND(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierFOVScaling>(), TestContext, TestAction, TestAxis));

	// Test applying a a key value of some kind
	WHEN(AKeyIsActuated(Data, TestAxis, 0.84648f));

	auto RunPerfTick = [&Data]() -> double
	{
		double Duration = 0.0;
		{
			FDurationTimer Timer{ Duration };
			Timer.Start();
	
			static constexpr int32 NumTicksToMeasure = 10000;
			for (int32 i = 0; i < NumTicksToMeasure; ++i)
			{
				// Tick the input stack, which will call the "ModifyRaw" function on every modifier we have
				AND(InputIsTicked(Data));		
			}
		
			Timer.Stop();
		}
		return Duration;
	};
	
	// Tick all these modifiers a bunch of times
	static constexpr int32 NumTimesToRun = 30;

	TArray<double, TInlineAllocator<NumTimesToRun>> AverageTimings = {};
	for (int32 i = 0; i < NumTimesToRun; ++i)
	{
		AverageTimings.Emplace(RunPerfTick());		
	}

	AverageTimings.Sort([](const double A, const double B)
	{
		return A < B;
	});

	const double MinRun = *Algo::MinElement(AverageTimings);
	const double MaxRun = *Algo::MaxElement(AverageTimings);
	
	// Note; this requires that the AverageTimings array has an even number of elements in it...
	// Otherwise you could just do AverageTimings[NumTimesToRun / 2] and get the middle element
	check(AverageTimings.Num() % 2 == 0);
	const double MedianTiming = (AverageTimings[NumTimesToRun / 2 - 1] + AverageTimings[NumTimesToRun / 2]) / 2;;

	UE_LOGF(LogEnhancedInput, Log, "Modifiers Perf Test (in seconds)... Median: %lf   Min: %lf   Max: %lf", MedianTiming, MinRun, MaxRun);
	
	TestLessEqual(TEXT("MinRun is less then max"), MinRun, MaxRun);
	TestNotEqual(TEXT("Median Time was calculated"), MedianTiming, 0.0);
	
	return true;
}