// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputMappingContext.h"
#include "InputTestFramework.h"
#include "Misc/AutomationTest.h"
#include "UserSettings/EnhancedInputUserSettings.h"
#include "Algo/Count.h"

// Tests focused Player Remappable Keys

namespace UE::Input
{
	static FKey TestKeyA = EKeys::A;
	static FKey TestKeyB = EKeys::B;
	static FKey TestKeyC = EKeys::C;

	static FKey TestKeyQ = EKeys::Q;
	static FKey TestKeyX = EKeys::X;
	static FKey TestKeyY = EKeys::Y;
	static FKey TestKeyZ = EKeys::Z;
	
	static FName TestContext_2 = TEXT("TestContext_2");
}

#define REMAPPABLEKEY_SUBTEST(DESC) \
	for(FString ScopedSubTestDescription = TEXT(DESC);ScopedSubTestDescription.Len();ScopedSubTestDescription = "")	// Bodge to create a scoped test description. Usage: REMAPPABLEKEY_SUBTEST("My Test Description") { TestCode... TriggerStateIs(ETriggerState::Triggered); }

constexpr auto BasicPlayerMappableKeysTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter;

UControllablePlayer& ABasicPlayerMappableKeysTest(FAutomationTestBase* Test)
{
	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	Test->TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());

	UEnhancedInputUserSettings* Settings = Data.Subsystem->GetUserSettings();
	Test->TestNotNull("Mocked User Settings", Settings);

	return Data;
}

// Provides a test player with a single input action (TestAction) added to it mapped to X
UControllablePlayer& APlayerMappableKeysTest_WithAction(FAutomationTestBase* Test, const FKey& DefaultKeyMapping = UE::Input::TestKeyX)
{
	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	Test->TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());

	UEnhancedInputUserSettings* Settings = Data.Subsystem->GetUserSettings();
	Test->TestNotNull("Mocked User Settings", Settings);

	// Test context
	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	
	TObjectPtr<UInputMappingContext>* IMC = Data.InputContext.Find(TestContext);
	Test->TestNotNull("Mock Mapping Context", IMC);

	// Add a simple mapping from test action to the given default key
	UInputAction* Action1 =
	AND(AnInputAction(Data, TestAction, EInputActionValueType::Axis2D));

	Test->TestNotNull("Mock Input Action", Action1);

	// Test mapping to X
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, DefaultKeyMapping));
	
	const bool bRes = Settings->RegisterInputMappingContext(*IMC);
	Test->TestTrue("Registered Mock IMC", bRes);

	return Data;
}

struct FMockKeyMappingData
{
	FName ActionName;
	FName ContextName;
	FKey DefaultKey;
	EInputActionValueType ValueType;
	EPlayerMappableKeySlot ExpectedSlot;
	FName HardwareDeviceId;
};


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRegisterIMCTest, "Input.PlayerMappableKeys.Registration", BasicPlayerMappableKeysTestFlags)
bool FRegisterIMCTest::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(ABasicPlayerMappableKeysTest(this));

	UEnhancedInputUserSettings* Settings = Data.Subsystem->GetUserSettings();
	TestNotNull("Mock Settings", Settings);

	static const TArray<FMockKeyMappingData> IMC_Mappings =
	{
		// TestContext
		{ TestAction,  TestContext,  UE::Input::TestKeyX, EInputActionValueType::Axis1D, EPlayerMappableKeySlot::First },
		{ TestAction2, TestContext,  UE::Input::TestKeyY, EInputActionValueType::Axis2D, EPlayerMappableKeySlot::First },
		{ TestAction3, TestContext,  UE::Input::TestKeyZ, EInputActionValueType::Axis3D, EPlayerMappableKeySlot::First },
		{ TestAction3, TestContext,  UE::Input::TestKeyQ, EInputActionValueType::Axis3D, EPlayerMappableKeySlot::Second },

		// UE::Input::TestContext_2
		{ TestAction4, UE::Input::TestContext_2,  UE::Input::TestKeyA, EInputActionValueType::Axis1D, EPlayerMappableKeySlot::First },
		{ TestAction5, UE::Input::TestContext_2,  UE::Input::TestKeyB, EInputActionValueType::Axis2D, EPlayerMappableKeySlot::First },
		{ TestAction6, UE::Input::TestContext_2,  UE::Input::TestKeyC, EInputActionValueType::Axis3D, EPlayerMappableKeySlot::First },
	};
	
	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputContextIsAppliedToAPlayer(Data, UE::Input::TestContext_2, 0));

	for (const FMockKeyMappingData& KeyData : IMC_Mappings)
	{
		// TestContext
		UInputAction* Action1 =
		AND(AnInputAction(Data, KeyData.ActionName, KeyData.ValueType));
		AND(AnActionIsMappedToAKey(Data, KeyData.ContextName, KeyData.ActionName, KeyData.DefaultKey));
	}
	
	// Ensure the state of the key is correct by default.
	TObjectPtr<UInputMappingContext>* IMC_1 = Data.InputContext.Find(TestContext);
	TestNotNull("Mock Mapping Context 1", IMC_1);

	TObjectPtr<UInputMappingContext>* IMC_2 = Data.InputContext.Find(UE::Input::TestContext_2);
	TestNotNull("Mock Mapping Context 2", IMC_2);

	// You can register one IMC
	REMAPPABLEKEY_SUBTEST("Register Mapping Context")
	{
		const bool bRes = Settings->RegisterInputMappingContext(*IMC_1);
		TestTrue("Register Mock Input Mapping Context", bRes);

		// We only expect one context to be registered at this time
		const TSet<TObjectPtr<const UInputMappingContext>>& RegisteredContexts = Settings->GetRegisteredInputMappingContexts();
		TestEqual("Num Registered Contexts", RegisteredContexts.Num(), 1);

		// We should have 3 registered key mappings here, one for each action that was mapped in the IMC
		const UEnhancedPlayerMappableKeyProfile* Profile = Settings->GetActiveKeyProfile();
		TestNotNull("Current Key Profile", Profile);

		// We expect 3 mapping rows, one for each mapping in the IMC
		const TMap<FName, FKeyMappingRow>& Rows = Profile->GetPlayerMappingRows();
		TestEqual("Key Mapping Rows", Rows.Num(), 3);
	}

	// You cannot register that IMC over again
	REMAPPABLEKEY_SUBTEST("Cannot re-register a context")
	{
		const bool bRes = Settings->RegisterInputMappingContext(*IMC_1);
		TestFalse("Re-registering the same IMC should be false", bRes);

		// We only expect one context to be registered at this time
		const TSet<TObjectPtr<const UInputMappingContext>>& RegisteredContexts = Settings->GetRegisteredInputMappingContexts();
		TestEqual("Num Registered Contexts", RegisteredContexts.Num(), 1);
	}
	
	REMAPPABLEKEY_SUBTEST("Register Multiple Mapping Contexts")
	{
		const bool bRes = Settings->RegisterInputMappingContext(*IMC_2);
		TestTrue("Register Mock Input Mapping Context 2", bRes);

		const TSet<TObjectPtr<const UInputMappingContext>>& RegisteredContexts = Settings->GetRegisteredInputMappingContexts();
		TestEqual("Num Registered Contexts", RegisteredContexts.Num(), 2);

		UEnhancedPlayerMappableKeyProfile* Profile = Settings->GetActiveKeyProfile();
		TestNotNull("Current Key Profile", Profile);
		
		const TMap<FName, FKeyMappingRow>& Rows = Profile->GetPlayerMappingRows();

		// We should have 6 rows, one for each test action
		TestEqual("Key Mapping Rows", Rows.Num(), 6);
	}
	
	REMAPPABLEKEY_SUBTEST("Key Mappings Have Correct Number or registered mappings")
	{
		UEnhancedPlayerMappableKeyProfile* Profile = Settings->GetActiveKeyProfile();
		TestNotNull("Current Key Profile", Profile);
		
		for (const FMockKeyMappingData& KeyData : IMC_Mappings)
		{
			const FKeyMappingRow* Row = Profile->FindKeyMappingRow(KeyData.ActionName);
			TestNotNull("Mapping Row", Row);

			UInputMappingContext* IMC = FInputTestHelper::FindContext(Data, KeyData.ContextName);
			UInputAction* Action = FInputTestHelper::FindAction(Data, KeyData.ActionName);

			// The row should have the same number of player mappings as there are mappings for each input action
			auto IsCorrectAction = [&Action](const FEnhancedActionKeyMapping& Mapping) { return Mapping.Action == Action; };
			const int32 NumMappingsToAction = Algo::CountIf(IMC->GetMappings(), IsCorrectAction);
			TestEqual("Correct Number of Rows", Row->Mappings.Num(), NumMappingsToAction);
			
			// Test that the query results are as expected
			FPlayerMappableKeyQueryOptions Opts = {};
			Opts.MappingName = KeyData.ActionName;
			Opts.KeyToMatch = KeyData.DefaultKey;
			Opts.SlotToMatch = KeyData.ExpectedSlot;
			Opts.bMatchBasicKeyTypes = true;
			Opts.bMatchKeyAxisType = true;

			TArray<FKey> MappedKeys;
			Profile->QueryPlayerMappedKeys(Opts, MappedKeys);

			TestEqual("Number of mapped keys", MappedKeys.Num(), 1);
			if (!MappedKeys.IsEmpty())
			{
				TestEqual("Mapped to the correct key", MappedKeys[0], KeyData.DefaultKey);
			}
		}
	}
	
	// Unregister the IMC's
	REMAPPABLEKEY_SUBTEST("Unregister IMC_1")
	{
		const bool bRes = Settings->UnregisterInputMappingContext(*IMC_1);
		TestTrue("Register Mock Input Mapping Context 1", bRes);

		const TSet<TObjectPtr<const UInputMappingContext>>& RegisteredContexts = Settings->GetRegisteredInputMappingContexts();
		TestEqual("Num Registered Contexts", RegisteredContexts.Num(), 1);

		UEnhancedPlayerMappableKeyProfile* Profile = Settings->GetActiveKeyProfile();
		TestNotNull("Current Key Profile", Profile);

		// Unregistering a mapping context shouldn't actually remove it's mapping rows because they will be saved
		const TMap<FName, FKeyMappingRow>& Rows = Profile->GetPlayerMappingRows();
		TestEqual("Key Mapping Rows", Rows.Num(), 6);
	}

	REMAPPABLEKEY_SUBTEST("Unregister IMC_2")
	{
		const bool bRes = Settings->UnregisterInputMappingContext(*IMC_2);
		TestTrue("Register Mock Input Mapping Context 2", bRes);

		const TSet<TObjectPtr<const UInputMappingContext>>& RegisteredContexts = Settings->GetRegisteredInputMappingContexts();
		TestEqual("Num Registered Contexts", RegisteredContexts.Num(), 0);

		UEnhancedPlayerMappableKeyProfile* Profile = Settings->GetActiveKeyProfile();
		TestNotNull("Current Key Profile", Profile);
		
		const TMap<FName, FKeyMappingRow>& Rows = Profile->GetPlayerMappingRows();
		TestEqual("Key Mapping Rows", Rows.Num(), 6);
	}
	
	return true;
}

//////////////////////////////////////////////////////////////
// Mapping Keys

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMapPlayerKeyTest, "Input.PlayerMappableKeys.Map", BasicPlayerMappableKeysTestFlags)
bool FMapPlayerKeyTest::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(APlayerMappableKeysTest_WithAction(this));

	UEnhancedInputUserSettings* Settings = Data.Subsystem->GetUserSettings();
	TestNotNull("Mock Settings", Settings);

	UEnhancedPlayerMappableKeyProfile* Profile = Settings->GetActiveKeyProfile();
	TestNotNull("Current Key Profile", Profile);

	FGameplayTagContainer ErrorReason;
	
	// Remap the action in slot 1 from x to z
	FMapPlayerKeyArgs Args = {};
	Args.MappingName = TestAction;
	Args.NewKey = UE::Input::TestKeyZ;
	Args.Slot = EPlayerMappableKeySlot::First;
	
	Settings->MapPlayerKey(Args, OUT ErrorReason);

	TestTrue("No Error Reason", ErrorReason.IsEmpty());

	// Query this key now and make sure it was remapped correctly
	FPlayerMappableKeyQueryOptions Opts = {};
	Opts.MappingName = Args.MappingName;
	Opts.KeyToMatch = Args.NewKey;
	Opts.SlotToMatch = Args.Slot;

	TArray<FKey> MappedKeys;
	Profile->QueryPlayerMappedKeys(Opts, MappedKeys);

	TestEqual("Number of mapped keys", MappedKeys.Num(), 1);
	if (!MappedKeys.IsEmpty())
	{
		TestEqual("re-mapped to the correct key", MappedKeys[0], Args.NewKey);
	}

	// If we call map player key on a slot that doesn't exist then it should create one
	Args.Slot = EPlayerMappableKeySlot::Third;
	Args.NewKey = UE::Input::TestKeyA;
	Args.bCreateMatchingSlotIfNeeded = true;
	
	Settings->MapPlayerKey(Args, OUT ErrorReason);
	TestTrue("No Error Reason", ErrorReason.IsEmpty());

	Opts.MappingName = Args.MappingName;
	Opts.KeyToMatch = Args.NewKey;
	Opts.SlotToMatch = Args.Slot;
	Profile->QueryPlayerMappedKeys(Opts, MappedKeys);

	TestEqual("Number of mapped keys", MappedKeys.Num(), 1);
	if (!MappedKeys.IsEmpty())
	{
		TestEqual("Added a new slot correctly", MappedKeys[0], Args.NewKey);
	}
	
	return true;
}

// Unmapping keys
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUnmapKeyTest, "Input.PlayerMappableKeys.Unmap", BasicPlayerMappableKeysTestFlags)
bool FUnmapKeyTest::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(APlayerMappableKeysTest_WithAction(this));

	UEnhancedInputUserSettings* Settings = Data.Subsystem->GetUserSettings();
	TestNotNull("Mock Settings", Settings);

	UEnhancedPlayerMappableKeyProfile* Profile = Settings->GetActiveKeyProfile();
	TestNotNull("Current Key Profile", Profile);

	FGameplayTagContainer ErrorReason;
	
	// Add a second key mapping to Z
	FMapPlayerKeyArgs Args = {};
	Args.MappingName = TestAction;
	Args.NewKey = UE::Input::TestKeyZ;
	Args.Slot = EPlayerMappableKeySlot::Second;
	
	Settings->MapPlayerKey(Args, OUT ErrorReason);
	TestTrue("Successful Remap", ErrorReason.IsEmpty());


	// Unmap this key now
	Settings->UnMapPlayerKey(Args, ErrorReason);
	TestTrue("No Error Reason during unmap", ErrorReason.IsEmpty());

	// Query this key now and make sure it was remapped correctly
	FPlayerMappableKeyQueryOptions Opts = {};
	Opts.MappingName = Args.MappingName;
	Opts.KeyToMatch = Args.NewKey;
	Opts.SlotToMatch = Args.Slot;

	TArray<FKey> MappedKeys;
	Profile->QueryPlayerMappedKeys(Opts, MappedKeys);

	// The key was unmapped, so it should be set to Invalid because it is not on by default
	TestEqual("Number of mapped keys", MappedKeys.Num(), 1);
	if (!MappedKeys.IsEmpty())
	{
		TestEqual("Added a new slot correctly", MappedKeys[0], EKeys::Invalid);
	}

	return true;
}

// Reset to default
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FResetToDefaultTest, "Input.PlayerMappableKeys.ResetToDefault", BasicPlayerMappableKeysTestFlags)

bool FResetToDefaultTest::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(APlayerMappableKeysTest_WithAction(this));

	UEnhancedInputUserSettings* Settings = Data.Subsystem->GetUserSettings();
	TestNotNull("Mock Settings", Settings);

	UEnhancedPlayerMappableKeyProfile* Profile = Settings->GetActiveKeyProfile();
	TestNotNull("Current Key Profile", Profile);

	FGameplayTagContainer ErrorReason;
	
	// Remap the action in slot 1 from x to z
	FMapPlayerKeyArgs Args = {};
	Args.MappingName = TestAction;
	Args.NewKey = UE::Input::TestKeyZ;
	Args.Slot = EPlayerMappableKeySlot::First;
	
	Settings->MapPlayerKey(Args, OUT ErrorReason);
	TestTrue("Successful Remap", ErrorReason.IsEmpty());

	// Reset the mapping back to default
	Profile->ResetMappingToDefault(TestAction);

	// Confirm this key has been reset!
	FPlayerMappableKeyQueryOptions Opts = {};
	Opts.MappingName = Args.MappingName;
	Opts.SlotToMatch = Args.Slot;

	// Query the key mapping and make sure it is back to the default of X
	TArray<FKey> MappedKeys;
	Profile->QueryPlayerMappedKeys(Opts, MappedKeys);

	TestEqual("Number of mapped keys", MappedKeys.Num(), 1);
	if (!MappedKeys.IsEmpty())
	{
		TestEqual("reset to default correctly!", MappedKeys[0], UE::Input::TestKeyX);
	}
	
	return true;
}

// A test to create multiple key mapping profiles
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMultipleKeyProfilesTest, "Input.PlayerMappableKeys.MultipleProfiles", BasicPlayerMappableKeysTestFlags)
bool FMultipleKeyProfilesTest::RunTest(const FString& Parameters)
{
	// Set up the test
	UControllablePlayer& Data =
		GIVEN(APlayerMappableKeysTest_WithAction(this));
	
	UEnhancedInputUserSettings* Settings = Data.Subsystem->GetUserSettings();
	TestNotNull("Mock Settings", Settings);
	
	UEnhancedPlayerMappableKeyProfile* Profile_A = Settings->GetActiveKeyProfile();
	TestNotNull("Current Key Profile", Profile_A);	
	
	const FString Profile_A_ID = Profile_A->GetProfileIdString();
	TestFalse("Current Key Profile has a valid string ID", Profile_A_ID.IsEmpty());

	// Map the "Z" key to the test action on the default profile
	{
		FGameplayTagContainer ErrorReason;

		// Remap the action in slot 1 from x to z
		FMapPlayerKeyArgs Args = {};
		Args.MappingName = TestAction;
		Args.NewKey = UE::Input::TestKeyZ;
		Args.Slot = EPlayerMappableKeySlot::First;

		Settings->MapPlayerKey(Args, OUT ErrorReason);
		TestTrue("Successful Remap", ErrorReason.IsEmpty());

		TArray<FKey> MappedKeys;

		FPlayerMappableKeyQueryOptions Opts = {};
		Opts.MappingName = TestAction;

		Profile_A->QueryPlayerMappedKeys(Opts, OUT MappedKeys);
		TestTrue("Query Mapped Keys populated the array of keys", !MappedKeys.IsEmpty());

		TestTrue("Default profile correctly mapped key to TestKeyZ", MappedKeys[0] == UE::Input::TestKeyZ);

		// Ensure that the mapping is now mapped to the Z key
		const FKeyMappingRow* Row = Profile_A->FindKeyMappingRow(TestAction);
		TestTrue("Row has mappings", Row->Mappings.Num() == 1);

		const FPlayerKeyMapping& SingleMapping = *Row->Mappings.CreateConstIterator();
		
		TestTrue("Mapping is correctly marked as dirty", SingleMapping.IsDirty());
		TestTrue("Mapping is in the correct slot", SingleMapping.GetSlot() == EPlayerMappableKeySlot::First);
		TestTrue("Mapping has the right custom FKey", SingleMapping.GetCurrentKey() == UE::Input::TestKeyZ);
		TestTrue("Mapping has the right default FKey", SingleMapping.GetDefaultKey() == UE::Input::TestKeyX);
	}

	// Create a new key profile and set it as the currently active one.
	UEnhancedPlayerMappableKeyProfile* Profile_B = nullptr;
	const FString Profile_B_Requested_ID = TEXT("input.test.profile.b");
	{
		FPlayerMappableKeyProfileCreationArgs Args = {};
		Args.ProfileStringIdentifier = Profile_B_Requested_ID;
		Args.bSetAsCurrentProfile = true;
		// use the default profile type in case there is a custom one set for the project these tests are run in.
		Args.ProfileType = UEnhancedPlayerMappableKeyProfile::StaticClass();
		Profile_B = Settings->CreateNewKeyProfile(Args);

		TestNotNull("Profile B is valid", Profile_B);
		TestTrue("Profile B has the correct String Identifier", Profile_B->GetProfileIdString().Equals(Profile_B_Requested_ID));
		
		const UEnhancedPlayerMappableKeyProfile* ActiveProfile = Settings->GetActiveKeyProfile();
		TestTrue("Profile B was set as the active profile", Profile_B == ActiveProfile);	

		// Ensure that the state of the new profiles key mappings are correct
		// Profile B should have the default key mappings set up because the customization was done on the default profile			
		TArray<FKey> MappedKeys;

		FPlayerMappableKeyQueryOptions Opts = {};
		Opts.MappingName = TestAction;

		Profile_B->QueryPlayerMappedKeys(Opts, OUT MappedKeys);
		TestTrue("Profile B: Query Mapped Keys populated the array of keys", !MappedKeys.IsEmpty());
		TestTrue("Profile B:correctly reset its mapping mapped key to TestKeyX", MappedKeys[0] == UE::Input::TestKeyX);

		// Ensure that the mapping is reset back to the default X key
		const FKeyMappingRow* Row = Profile_B->FindKeyMappingRow(TestAction);
		TestTrue("Profile B: Row has mappings", Row->Mappings.Num() == 1);

		const FPlayerKeyMapping& SingleMapping = *Row->Mappings.CreateConstIterator();

		TestFalse("Profile B: Mapping is correctly not marked as dirty", SingleMapping.IsDirty());
		TestTrue("Profile B: Mapping is in the correct slot", SingleMapping.GetSlot() == EPlayerMappableKeySlot::First);
		TestTrue("Profile B: Mapping has the right custom FKey", SingleMapping.GetCurrentKey() == UE::Input::TestKeyX);
		TestTrue("Profile B: Mapping has the right default FKey", SingleMapping.GetDefaultKey() == UE::Input::TestKeyX);
	}

	// Ensure that you can still query the keys from a non-active key profile
	{		
		TArray<FKey> MappedKeys;

		FPlayerMappableKeyQueryOptions Opts = {};
		Opts.MappingName = TestAction;

		Profile_A->QueryPlayerMappedKeys(Opts, OUT MappedKeys);
		TestTrue("Profile A: Query Mapped Keys populated the array of keys", !MappedKeys.IsEmpty());
		TestTrue("Profile A:correctly reset its mapping mapped key to TestKeyX", MappedKeys[0] == UE::Input::TestKeyZ);

		// Ensure that the mapping is reset back to the default X key
		const FKeyMappingRow* Row = Profile_A->FindKeyMappingRow(TestAction);
		TestTrue("Profile A: Row has mappings", Row->Mappings.Num() == 1);

		const FPlayerKeyMapping& SingleMapping = *Row->Mappings.CreateConstIterator();

		TestTrue("Profile A: Mapping is correctly not marked as dirty", SingleMapping.IsDirty());
		TestTrue("Profile A: Mapping is in the correct slot", SingleMapping.GetSlot() == EPlayerMappableKeySlot::First);
		TestTrue("Profile A: Mapping has the right custom FKey", SingleMapping.GetCurrentKey() == UE::Input::TestKeyZ);
		TestTrue("Profile A: Mapping has the right default FKey", SingleMapping.GetDefaultKey() == UE::Input::TestKeyX);
	}
	
	// Ensure that both profiles are registered
	{
		const TMap<FString, TObjectPtr<UEnhancedPlayerMappableKeyProfile>> ProfileMap = Settings->GetAllAvailableKeyProfiles();

		TestTrue("Profile A is in the map", ProfileMap.Contains(Profile_A_ID));
		TestTrue("Profile B is in the map", ProfileMap.Contains(Profile_B_Requested_ID));
	}

	return true;
}