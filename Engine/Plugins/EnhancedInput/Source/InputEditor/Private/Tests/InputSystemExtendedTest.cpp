// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputTestFramework.h"
#include "Misc/AutomationTest.h"
#include "InputAction.h"
#include "InputModifiers.h"
#include "EnhancedActionKeyMapping.h"

// Extended system tests covering AccumulationBehavior, Axis3D value flow,
// and the modifier application order between mapping-level and action-level.

constexpr auto BasicSystemExtTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter;

static UControllablePlayer& ABasicSystemExtTest(FAutomationTestBase* Test, EInputActionValueType ForValueType)
{
	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	Test->TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());

	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputAction(Data, TestAction, ForValueType));

	return Data;
}

// ============================================================
// AccumulationBehavior: TakeHighestAbsoluteValue vs Cumulative
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputAccumulationBehaviorTest, "Input.System.AccumulationBehavior", BasicSystemExtTestFlags)

bool FInputAccumulationBehaviorTest::RunTest(const FString& Parameters)
{
	// Two 1D keys mapped to the same Axis1D action.
	// Pressing both simultaneously lets us test how their values are combined.
	UControllablePlayer& Data =
	GIVEN(ABasicSystemExtTest(this, EInputActionValueType::Axis1D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis2));

	UInputAction* Action = FInputTestHelper::FindAction(Data, TestAction);
	TestNotNull(TEXT("Action found"), Action);

	// -------------------------------------------------------
	// Test 1 - TakeHighestAbsoluteValue (default):
	//   Both keys pressed: 0.3 and 0.8 -> result = 0.8 (highest absolute value wins)
	// -------------------------------------------------------
	Action->AccumulationBehavior = EInputActionAccumulationBehavior::TakeHighestAbsoluteValue;

	WHEN(AKeyIsActuated(Data, TestAxis, 0.3f));
	AND(AKeyIsActuated(Data, TestAxis2, 0.8f));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("TakeHighest: 0.8 beats 0.3"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 0.8f));

	// Test 1b - Higher absolute value wins regardless of evaluation order
	WHEN(AKeyIsActuated(Data, TestAxis, 0.8f));
	AND(AKeyIsActuated(Data, TestAxis2, 0.3f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("TakeHighest: 0.8 beats 0.3 (reversed)"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 0.8f));

	// Test 1c - Negative value with higher absolute magnitude wins
	WHEN(AKeyIsActuated(Data, TestAxis, -0.8f));
	AND(AKeyIsActuated(Data, TestAxis2, 0.3f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("TakeHighest: -0.8 beats 0.3 (sign preserved)"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), -0.8f));

	// -------------------------------------------------------
	// Test 2 - Cumulative:
	//   Both keys pressed: 0.3 and 0.5 -> result = 0.8 (values summed)
	// -------------------------------------------------------
	Action->AccumulationBehavior = EInputActionAccumulationBehavior::Cumulative;

	WHEN(AKeyIsActuated(Data, TestAxis, 0.3f));
	AND(AKeyIsActuated(Data, TestAxis2, 0.5f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("Cumulative: 0.3 + 0.5 = 0.8"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 0.8f));

	// Test 2b - Opposite-sign values cancel out (WASD-style: W=+1, S=-1 -> 0)
	// NOTE: SetStateForNoTriggers is called per-mapping with each mapping's INDIVIDUAL value.
	// Both ±0.5 mappings are individually non-zero, so each contributes Triggered to the tracker.
	// The accumulated sum is 0.0 but the trigger state is still Triggered (actual system behavior).
	WHEN(AKeyIsActuated(Data, TestAxis, 0.5f));
	AND(AKeyIsActuated(Data, TestAxis2, -0.5f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction)); // both mappings individually non-zero -> Triggered
	AND(TestEqual(TEXT("Cumulative: +0.5 + -0.5 = 0"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 0.0f));

	// Test 2c - Single key: Cumulative with one key behaves the same as TakeHighest
	WHEN(AKeyIsReleased(Data, TestAxis2));
	AND(AKeyIsActuated(Data, TestAxis, 0.7f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("Cumulative: single key"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 0.7f));

	return true;
}

// ============================================================
// Axis3D action value flow
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputAxis3DActionTest, "Input.System.Axis3D", BasicSystemExtTestFlags)

bool FInputAxis3DActionTest::RunTest(const FString& Parameters)
{
	// A 1D analog key maps to the X component of an Axis3D action.
	// We verify the value is correctly placed in X and that Y/Z are zero.
	UControllablePlayer& Data =
	GIVEN(ABasicSystemExtTest(this, EInputActionValueType::Axis3D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));

	// Test 1 - Single 1D key: value lands on X, Y and Z are zero
	WHEN(AKeyIsActuated(Data, TestAxis, 0.7f));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	FVector Result = FInputTestHelper::GetTriggered<FVector>(Data, TestAction);
	AND(TestEqual(TEXT("Axis3D: X from 1D key"), (float)Result.X, 0.7f));
	AND(TestEqual(TEXT("Axis3D: Y is zero"), (float)Result.Y, 0.f));
	AND(TestEqual(TEXT("Axis3D: Z is zero"), (float)Result.Z, 0.f));

	// Test 2 - Modify the key value and verify the updated X
	WHEN(AKeyIsActuated(Data, TestAxis, -0.4f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));

	Result = FInputTestHelper::GetTriggered<FVector>(Data, TestAction);
	AND(TestEqual(TEXT("Axis3D: negative X"), (float)Result.X, -0.4f));

	// Test 3 - SwizzleAxis ZYX at the mapping level: X value moves to Z component
	//          Input FVector(0.5, 0, 0) -> after ZYX (swap X,Z) -> FVector(0, 0, 0.5)
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis2)); // map a second key for this sub-test
	// Pre-configure before applying so DeepCopyPtrArray captures ZYX order.
	UInputModifierSwizzleAxis* OrigSwizzle = NewObject<UInputModifierSwizzleAxis>();
	OrigSwizzle->Order = EInputAxisSwizzle::ZYX;
	AND(AModifierIsAppliedToAnActionMapping(Data, OrigSwizzle, TestContext, TestAction, TestAxis2));

	// Release the first key so only TestAxis2 drives the action
	WHEN(AKeyIsReleased(Data, TestAxis));
	AND(AKeyIsActuated(Data, TestAxis2, 0.5f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));

	Result = FInputTestHelper::GetTriggered<FVector>(Data, TestAction);
	AND(TestEqual(TEXT("Axis3D + SwizzleZYX: X is zero"), (float)Result.X, 0.f));
	AND(TestEqual(TEXT("Axis3D + SwizzleZYX: Y is zero"), (float)Result.Y, 0.f));
	AND(TestEqual(TEXT("Axis3D + SwizzleZYX: Z receives value"), (float)Result.Z, 0.5f));

	// Test 4 - Release: Completed
	WHEN(AKeyIsReleased(Data, TestAxis2));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	return true;
}

// ============================================================
// Modifier application order: mapping-level vs action-level
// on independent axes to demonstrate ordering through the pipeline.
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifierApplicationOrderTest, "Input.System.ModifierApplicationOrder", BasicSystemExtTestFlags)

bool FInputModifierApplicationOrderTest::RunTest(const FString& Parameters)
{
	// This test establishes that:
	//   1. Modifiers in the KEY MAPPING chain run before ACTION-LEVEL modifiers.
	//   2. Within a chain (mapping or action level), modifiers run in array order.
	//
	// Strategy: use non-commutative operations so the two orderings produce
	// different results.
	//
	// Mapping-level: Scalar(3)
	// Action-level:  DeadZone(LowerThreshold=0.5, UpperThreshold=1.0)
	// Input: 0.2
	//
	//   Correct order (mapping first):  0.2 * 3 = 0.6 -> deadzone -> (0.6-0.5)/0.5 = 0.2 -> TRIGGERED
	//   Reversed order (action first):  deadzone(0.2) = 0.0 (below 0.5) -> * 3 = 0.0 -> NOT TRIGGERED
	//
	// Sub-test A verifies the correct order fires the trigger.
	// Sub-test B removes the mapping-level scalar to confirm a raw 0.2 input is killed
	// by the 0.5 deadzone, making the contrast explicit.

	UControllablePlayer& Data =
	GIVEN(ABasicSystemExtTest(this, EInputActionValueType::Axis1D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestAxis));

	// Pre-configure modifiers before applying so the deep-copy in RebuildControlMappings
	// captures the correct values. Post-apply pointers are duplicates and will be stale
	// after the next rebuild, so we fetch a fresh live pointer after all Apply calls.
	UInputModifierScalar* OrigScalar = NewObject<UInputModifierScalar>();
	OrigScalar->Scalar = FVector(3.f, 3.f, 3.f);
	AND(AModifierIsAppliedToAnActionMapping(Data, OrigScalar, TestContext, TestAction, TestAxis));

	UInputModifierDeadZone* OrigDeadZone = NewObject<UInputModifierDeadZone>();
	OrigDeadZone->LowerThreshold = 0.5f;
	OrigDeadZone->UpperThreshold = 1.0f;
	OrigDeadZone->Type = EDeadZoneType::Axial;
	AND(AModifierIsAppliedToAnAction(Data, OrigDeadZone, TestAction));

	// Fetch the live Scalar duplicate (created by DeepCopyPtrArray during the last rebuild).
	// This is the pointer we can safely mutate between ticks without triggering another rebuild.
	FEnhancedActionKeyMapping* LiveMap = FInputTestHelper::FindLiveActionMapping(Data, TestAction, TestAxis);
	check(LiveMap && !LiveMap->Modifiers.IsEmpty());
	UInputModifierScalar* LiveScalar = Cast<UInputModifierScalar>(LiveMap->Modifiers[0]);
	check(LiveScalar);

	// Sub-test A: input 0.2 should survive because the mapping scalar runs first (0.2*3=0.6 > 0.5 threshold)
	WHEN(AKeyIsActuated(Data, TestAxis, 0.2f));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));
	// Post-deadzone value: (0.6 - 0.5) / (1.0 - 0.5) = 0.2
	AND(TestEqual(TEXT("Mapping modifier ran first: value is 0.2 post-deadzone"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 0.2f));

	// Sub-test B: set the mapping-level scalar to identity (1.0).
	// Now raw 0.2 goes straight to the deadzone and is killed (0.2 < 0.5 threshold).
	// State was Triggered from sub-test A; value drops to 0 -> Triggered->None=Completed fires.
	LiveScalar->Scalar = FVector(1.f, 1.f, 1.f);

	WHEN(AKeyIsActuated(Data, TestAxis, 0.2f));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersCompleted(Data, TestAction)); // Triggered->None transition

	// Sub-test C: restore scalar and verify it triggers again, confirming the ordering is stable.
	LiveScalar->Scalar = FVector(3.f, 3.f, 3.f);

	WHEN(AKeyIsActuated(Data, TestAxis, 0.2f));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction)); // None->Triggered=StartedAndTriggered

	// -------------------------------------------------------
	// Sub-test D: within a single level, modifiers run in array (insertion) order.
	// Negate then Scalar(-2): input 1.0 -> negate -> -1.0 -> scalar(-2) -> 2.0
	// Scalar(-2) then Negate: input 1.0 -> scalar(-2) -> -2.0 -> negate -> 2.0
	// Same result for linear ops - but verifying two modifiers in a chain both fire.
	// -------------------------------------------------------
	// Use a fresh test setup to isolate this sub-test.
	{
		UControllablePlayer& ChainData =
		GIVEN(ABasicSystemExtTest(this, EInputActionValueType::Axis1D));
		AND(AnActionIsMappedToAKey(ChainData, TestContext, TestAction, TestAxis));

		// Apply Negate first, then Scalar(-2) - both at mapping level.
		// Pre-configure before applying so DeepCopyPtrArray captures the correct value.
		AND(AModifierIsAppliedToAnActionMapping(ChainData, NewObject<UInputModifierNegate>(), TestContext, TestAction, TestAxis));
		UInputModifierScalar* OrigChainScalar = NewObject<UInputModifierScalar>();
		OrigChainScalar->Scalar = FVector(-2.f, -2.f, -2.f);
		AND(AModifierIsAppliedToAnActionMapping(ChainData, OrigChainScalar, TestContext, TestAction, TestAxis));

		// input 1.0 -> Negate -> -1.0 -> Scalar(-2) -> 2.0
		WHEN(AKeyIsActuated(ChainData, TestAxis, 1.0f));
		AND(InputIsTicked(ChainData));
		THEN(PressingKeyTriggersAction(ChainData, TestAction));
		AND(TestEqual(TEXT("Chain: Negate then Scalar(-2) applied in order"), FInputTestHelper::GetActionData(ChainData, TestAction).GetValue().Get<float>(), 2.0f));
	}

	return true;
}

// ============================================================
// Multi-context priority: modifier values on live key mappings
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputMappingContextPriorityModifierTest, "Input.System.MappingContextPriorityModifier", BasicSystemExtTestFlags)

bool FInputMappingContextPriorityModifierTest::RunTest(const FString& Parameters)
{
	// Two IMCs both map the same action to the same key, each with a Scalar modifier at a
	// different value. IMC_A is pushed at priority 0, IMC_B at priority 1 (higher).
	//
	// After pushing only IMC_A, the single live mapping must carry Scalar = 10.
	// After also pushing IMC_B, RebuildControlMappings processes higher-priority contexts
	// first, so IMC_B's mapping appears first in EnhancedActionMappings. FindLiveActionMapping
	// returns that first entry, which must carry Scalar = 50.

	static const FName ContextA = TEXT("IMC_PriorityTest_A");
	static const FName ContextB = TEXT("IMC_PriorityTest_B");

	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());

	AND(AnInputAction(Data, TestAction, EInputActionValueType::Axis1D));

	// -------------------------------------------------------
	// Step 1: push IMC_A (priority 0) with Scalar(10)
	// -------------------------------------------------------
	AND(AnInputContextIsAppliedToAPlayer(Data, ContextA, 0));
	AND(AnActionIsMappedToAKey(Data, ContextA, TestAction, TestAxis));

	// Pre-configure before applying so DeepCopyPtrArray captures the value.
	UInputModifierScalar* OrigScalarA = NewObject<UInputModifierScalar>();
	OrigScalarA->Scalar = FVector(10.f, 10.f, 10.f);
	AND(AModifierIsAppliedToAnActionMapping(Data, OrigScalarA, ContextA, TestAction, TestAxis));

	// With only IMC_A applied, the single live mapping must have Scalar = 10.
	FEnhancedActionKeyMapping* LiveMapA = FInputTestHelper::FindLiveActionMapping(Data, TestAction, TestAxis);
	check(LiveMapA && !LiveMapA->Modifiers.IsEmpty());
	UInputModifierScalar* LiveScalarA = Cast<UInputModifierScalar>(LiveMapA->Modifiers[0]);
	check(LiveScalarA);
	THEN(TestEqual(TEXT("IMC_A only: live modifier scalar is 10"), (float)LiveScalarA->Scalar.X, 10.f));

	// Confirm end-to-end: input 1.0 scaled by 10 -> triggered value 10.
	WHEN(AKeyIsActuated(Data, TestAxis, 1.0f));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("IMC_A only: triggered value is 10"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 10.f));

	// -------------------------------------------------------
	// Step 2: push IMC_B (priority 1, higher) with Scalar(50)
	// -------------------------------------------------------
	AND(AnInputContextIsAppliedToAPlayer(Data, ContextB, 1));
	AND(AnActionIsMappedToAKey(Data, ContextB, TestAction, TestAxis));

	UInputModifierScalar* OrigScalarB = NewObject<UInputModifierScalar>();
	OrigScalarB->Scalar = FVector(50.f, 50.f, 50.f);
	AND(AModifierIsAppliedToAnActionMapping(Data, OrigScalarB, ContextB, TestAction, TestAxis));

	// RebuildControlMappings processes IMC_B (priority 1) before IMC_A (priority 0),
	// so IMC_B's mapping is first in EnhancedActionMappings. FindLiveActionMapping
	// returns that entry -- its modifier must now be Scalar = 50.
	FEnhancedActionKeyMapping* LiveMapB = FInputTestHelper::FindLiveActionMapping(Data, TestAction, TestAxis);
	check(LiveMapB && !LiveMapB->Modifiers.IsEmpty());
	UInputModifierScalar* LiveScalarB = Cast<UInputModifierScalar>(LiveMapB->Modifiers[0]);
	check(LiveScalarB);
	THEN(TestEqual(TEXT("IMC_A+B: highest-priority live mapping has scalar 50"), (float)LiveScalarB->Scalar.X, 50.f));

	// Confirm end-to-end: with TakeHighestAbsoluteValue, 1.0*50=50 beats 1.0*10=10.
	// ActionInstanceData was reset by the last rebuild, so state is None -> StartedAndTriggered.
	WHEN(AKeyIsActuated(Data, TestAxis, 1.0f));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));
	AND(TestEqual(TEXT("IMC_A+B: triggered value is 50 (highest wins)"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 50.f));

	return true;
}
