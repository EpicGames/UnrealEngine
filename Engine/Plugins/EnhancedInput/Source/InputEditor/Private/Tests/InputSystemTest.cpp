// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputMappingContext.h"
#include "Misc/AutomationTest.h"
#include "InputMappingQuery.h"
#include "InputTestFramework.h"

// Tests focused on the underlying enhanced input system



constexpr auto BasicSystemTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter;	// TODO: Run as Smoke/Client? No world on RunSmokeTests startup...

UControllablePlayer& ABasicSystemTest(FAutomationTestBase* Test, EInputActionValueType ForValueType)
{
	// Initialise
	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	Test->TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());	// TODO: Can we early out on a failed Test?

	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputAction(Data, TestAction, ForValueType));

	return Data;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputActionValueMatchesTriggerValue, "Input.System.ActionValueMatchesTriggerValue", BasicSystemTestFlags)

bool FInputActionValueMatchesTriggerValue::RunTest(const FString& Parameters)
{
	// Initialise
	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());	// TODO: Can we early out on a failed Test?
	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));

	// Test 1 - Boolean
	GIVEN(AnInputAction(Data, TestAction, EInputActionValueType::Boolean));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));
	AND(TestTrue(TEXT("Desired bool"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<bool>()));
	AND(TestEqual(TEXT("Matching bool"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<bool>(), FInputTestHelper::GetTriggered<bool>(Data, TestAction)));

	// Test 2 - Axis1D
	float Test1DValue = 1.5f;
	GIVEN(AnInputAction(Data, TestAction2, EInputActionValueType::Axis1D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction2, TestAxis));
	WHEN(AKeyIsActuated(Data, TestAxis, Test1DValue));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction2));
	AND(TestEqual(TEXT("Desired 1D"), FInputTestHelper::GetActionData(Data, TestAction2).GetValue().Get<float>(), Test1DValue));
	AND(TestEqual(TEXT("Matching 1D"), FInputTestHelper::GetActionData(Data, TestAction2).GetValue().Get<float>(), FInputTestHelper::GetTriggered<float>(Data, TestAction2)));

	// Test 3 - Axis2D
	FVector2D Test2DValue(1.5f, -0.25f);
	GIVEN(AnInputAction(Data, TestAction3, EInputActionValueType::Axis2D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction3, EKeys::Gamepad_Left2D));
	WHEN(AKeyIsActuated(Data, EKeys::Gamepad_LeftX, Test2DValue.X));
	AND(AKeyIsActuated(Data, EKeys::Gamepad_LeftY, Test2DValue.Y));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction3));
	// TestEqual can't handle printing out FVector2Ds so promote to FVector here
	AND(TestEqual(TEXT("Desired 2D"), FInputTestHelper::GetActionData(Data, TestAction3).GetValue().Get<FVector>(), FVector(Test2DValue, 0.f)));
	AND(TestEqual(TEXT("Matching 2D"), FInputTestHelper::GetActionData(Data, TestAction3).GetValue().Get<FVector>(), FInputTestHelper::GetTriggered<FVector>(Data, TestAction3)));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputModifierPairedAxisTest, "Input.System.PairedAxis", BasicSystemTestFlags)

bool FInputModifierPairedAxisTest::RunTest(const FString& Parameters)
{
	GIVEN(UControllablePlayer & Data = ABasicSystemTest(this, EInputActionValueType::Axis2D));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, EKeys::Gamepad_Left2D));

	// 2D axis is driven by the 1D component keys, rather than directly.
	WHEN(AKeyIsActuated(Data, EKeys::Gamepad_LeftX, 1.f));
	AND(AKeyIsActuated(Data, EKeys::Gamepad_LeftY, 1.f));
	AND(InputIsTicked(Data));

	FVector2D Result = FInputTestHelper::GetTriggered<FVector2D>(Data, TestAction);
	THEN(Result.X == 1.f && Result.Y == 1.f);

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputMappingQueryTest, "Input.System.MappingQuery", BasicSystemTestFlags)

bool FInputMappingQueryTest::RunTest(const FString& Parameters)
{
	UWorld* World = AnEmptyWorld();
	UControllablePlayer& Data =
	GIVEN(AControllablePlayer(World));
	UInputMappingContext* BaseConfig =
	AND(AnInputContextIsAppliedToAPlayer(Data, TEXT("BaseConfig"), 0));
	UInputMappingContext* CollisionContext =
	AND(AnInputContextIsAppliedToAPlayer(Data, TEXT("CollisionContext"), 10));
	UInputMappingContext* TopConfig =
	AND(AnInputContextIsAppliedToAPlayer(Data, TEXT("TopConfig"), 20));
	UInputAction* Collision =
	AND(AnInputAction(Data, TEXT("Collision"), EInputActionValueType::Boolean));
	UInputAction* TestActionBase =
	AND(AnInputAction(Data, TEXT("TestActionBase"), EInputActionValueType::Boolean));
	UInputAction* TestActionTop =
	AND(AnInputAction(Data, TEXT("TestActionTop"), EInputActionValueType::Boolean));


	// Bind collidable key mapping
	WHEN(AnActionIsMappedToAKey(Data, TEXT("CollisionContext"), TEXT("Collision"), TestKey));

	// Bind alternative keys for these actions
	AND(AnActionIsMappedToAKey(Data, TEXT("BaseConfig"), TEXT("TestActionBase"), TestKey2));
	AND(AnActionIsMappedToAKey(Data, TEXT("TopConfig"), TEXT("TestActionTop"), TestKey3));

	TArray<FMappingQueryIssue> BaseIssues;
	THEN(Data.Subsystem->QueryMapKeyInActiveContextSet(BaseConfig, TestActionBase, TestKey, BaseIssues, EMappingQueryIssue::NoIssue) == EMappingQueryResult::MappingAvailable);
	AND(TestEqual(TEXT("Single issue"), BaseIssues.Num(), 1));
	AND(TestEqual(TEXT("Hidden by existing"), BaseIssues[0].Issue, EMappingQueryIssue::HiddenByExistingMapping));
	ANDALSO(BaseIssues[0].BlockingAction == Collision);
	ANDALSO(BaseIssues[0].BlockingContext == CollisionContext);

	TArray<FMappingQueryIssue> TopIssues;
	THEN(Data.Subsystem->QueryMapKeyInActiveContextSet(TopConfig, TestActionTop, TestKey, TopIssues, EMappingQueryIssue::HidesExistingMapping) == EMappingQueryResult::NotMappable);
	AND(TestEqual(TEXT("Single issue"), TopIssues.Num(), 1));
	AND(TestEqual(TEXT("Hides existing"), TopIssues[0].Issue, EMappingQueryIssue::HidesExistingMapping));
	ANDALSO(TopIssues[0].BlockingAction == Collision);
	ANDALSO(TopIssues[0].BlockingContext == CollisionContext);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputInjectionTest, "Input.System.InputInjection", BasicSystemTestFlags)

bool FInputInjectionTest::RunTest(const FString& Parameters)
{
	UControllablePlayer& Data =
	GIVEN(ABasicSystemTest(this, EInputActionValueType::Axis1D));

	// Test 1 - Pre first input tick injection is okay
	WHEN(AnInputIsInjected(Data, TestAction, FInputActionValue(0.5f)));
	AND(InputIsTicked(Data));
	/*THEN*/(TestEqual(TEXT("Pre first tick"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 0.5f));


	// Test 2 - Player has no mapped actions at all
	WHEN(AnInputIsInjected(Data, TestAction, FInputActionValue(0.5f)));
	AND(InputIsTicked(Data));
	/*THEN*/(TestEqual(TEXT("None mapped"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 0.5f));

	// Test 3 - Injection over a mapped action can override action when magnitude exceeds action.
	GIVEN(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));
	AND(AKeyIsActuated(Data, TestKey));

	WHEN(AnInputIsInjected(Data, TestAction, FInputActionValue(0.5f)));
	AND(InputIsTicked(Data));
	/*THEN*/(TestEqual(TEXT("Mapped action overrides"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 1.0f));

	WHEN(AnInputIsInjected(Data, TestAction, FInputActionValue(1.5f)));
	AND(InputIsTicked(Data));
	/*THEN*/(TestEqual(TEXT("Mapped action overridden"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 1.5f));

	// Release test key for further tests
	AKeyIsReleased(Data, TestKey);

	// Test 4 - Injection into an unmapped action, but player has a (different) mapped action
	GIVEN(AnInputAction(Data, TestAction2, EInputActionValueType::Axis1D));
	WHEN(AnInputIsInjected(Data, TestAction2, FInputActionValue(0.5f)));
	AND(InputIsTicked(Data));
	/*THEN*/(TestEqual(TEXT("Unmapped with mapped action"), FInputTestHelper::GetActionData(Data, TestAction2).GetValue().Get<float>(), 0.5f));

	// Test 5 - Post tick injection will not affect the action data results!
	WHEN(InputIsTicked(Data));
	AND(AnInputIsInjected(Data, TestAction, FInputActionValue(0.5f)));
	/*THEN*/(TestEqual(TEXT("Post tick"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 0.0f));

	// ...until the next tick
	WHEN(InputIsTicked(Data));
	/*THEN*/(TestEqual(TEXT("Post second tick"), FInputTestHelper::GetActionData(Data, TestAction).GetValue().Get<float>(), 0.5f));


	// TODO: Held vs pressed vs released.
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputContextRegistrationTrackingTest, "Input.System.ContextRegistrationTracking", BasicSystemTestFlags)
bool FInputContextRegistrationTrackingTest::RunTest(const FString& Parameters)
{
	UWorld* World = AnEmptyWorld();
	UControllablePlayer& Data =
	GIVEN(AControllablePlayer(World));
	AND(const FName UntrackedMappingName = TEXT("UntrackedMapping"));
	AND(const FName CountedMappingName = TEXT("CountedMapping"));

	// Test 1 - An untracked mapping that is added twice then removed once is no longer applied
	WHEN(AnInputContextWithTrackingModeIsAppliedToAPlayer(Data, UntrackedMappingName, 0, EMappingContextRegistrationTrackingMode::Untracked));
	AND(AnInputContextIsReappliedToAPlayer(Data, UntrackedMappingName, 0));
	AND(AnInputContextIsRemovedFromAPlayer(Data, UntrackedMappingName));
	THEN(InputContextIsNotApplied(Data, UntrackedMappingName));

	// Test 2a - A registration counted mapping is added twice and then removed once is still applied
	WHEN(AnInputContextWithTrackingModeIsAppliedToAPlayer(Data, CountedMappingName, 0, EMappingContextRegistrationTrackingMode::CountRegistrations));
	AND(AnInputContextIsReappliedToAPlayer(Data, CountedMappingName, 0));
	AND(AnInputContextIsRemovedFromAPlayer(Data, CountedMappingName));
	THEN(InputContextIsApplied(Data, CountedMappingName));

	// Test 2b - The registration counted mapping is removed once more and then no longer applied
	WHEN(AnInputContextIsRemovedFromAPlayer(Data, CountedMappingName));
	THEN(InputContextIsNotApplied(Data, CountedMappingName));

	return true;
}

// Event transition tests

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputEventTransitionTest, "Input.System.Events.Transitions", BasicSystemTestFlags)	// TODO: Split into individual transition tests?

bool FInputEventTransitionTest::RunTest(const FString& Parameters)
{
	// Event transition tests
	UControllablePlayer& Data =
	GIVEN(ABasicSystemTest(this, EInputActionValueType::Boolean));

	// TODO: We should create a mocked trigger object similar to Hold for this but we don't want UHT doing reflection on it so just assume Hold's behavior. Create a blueprinted trigger for this?
	const float FrameTime = 1.f / 60.f;
	const int TriggerFrames = 3;

	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));
	AND(UInputTriggerHold * TestTrigger = Cast<UInputTriggerHold>(ATriggerIsAppliedToAnAction(Data, NewObject<UInputTriggerHold>(), TestAction))); // Reassign test trigger to the trigger instance so we can change it later
	check(TestTrigger);	// Need to apply trigger after setting up the mapping or we won't get a valid trigger instance back

	TestTrigger->HoldTimeThreshold = FrameTime * TriggerFrames;
	TestTrigger->bIsOneShot = true;

	// Test 1 - Started -> Canceled

	// Pressing triggers "Started" event
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	// Releasing cancels
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersCanceled(Data, TestAction));

	// But only once!
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyDoesNotTrigger(Data, TestAction));

	// Test 2 - Started -> Ongoing -> Canceled
	InputIsTicked(Data, FrameTime);	// Clear state

	// Pressing
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	// Holding until trigger frame
	for (int HoldFrame = 1; HoldFrame < TriggerFrames - 1; ++HoldFrame)
	{
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersOngoing(Data, TestAction));
	}

	// Holding over the trigger frame - hold threshold is inclusive
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersCanceled(Data, TestAction));

	// Test 3 - Started -> Ongoing -> Triggered -> Completed
	InputIsTicked(Data, FrameTime);	// Clear state

	// Pressing
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	// Holding until trigger frame
	for (int HoldFrame = 1; HoldFrame < TriggerFrames - 1; ++HoldFrame)
	{
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersOngoing(Data, TestAction));
	}

	// Holding for a further frame triggers
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(HoldingKeyTriggersAction(Data, TestAction));

	// And then completes
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));


	// Test 4 Started + Triggered on same frame -> Completed
	TestTrigger->HoldTimeThreshold = 0.f;
	InputIsTicked(Data, FrameTime);	// Clear state

	// Pressing triggers both Started and Triggered
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));
	ANDALSO(PressingKeyTriggersAction(Data, TestAction));

	// And complete
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(HoldingKeyTriggersCompleted(Data, TestAction));


	// TODO: Triggered -> Ongoing via multiple keys with differing triggers (should be Triggered -> Ongoing + Completed!)

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputEventCompletedTest, "Input.System.Events.Completed", BasicSystemTestFlags)

bool FInputEventCompletedTest::RunTest(const FString& Parameters)
{
	// Holding two mapped keys triggers an action
	GIVEN(UControllablePlayer & Data = ABasicSystemTest(this, EInputActionValueType::Boolean));
	WHEN(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey2));
	AND(AKeyIsActuated(Data, TestKey));
	AND(AKeyIsActuated(Data, TestKey2));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	// Releasing one leaves the action triggered
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersAction(Data, TestAction));

	// Releasing both transitions to Completed
	WHEN(AKeyIsReleased(Data, TestKey2));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	// Completed transitions to None the next tick
	WHEN(InputIsTicked(Data));
	THEN(FInputTestHelper::TestNoTrigger(Data, TestAction));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputSystemContextChangeIgnoreHeldKeysTest, "Input.System.ContextChange.IgnoreHeldKeys", BasicSystemTestFlags)

bool FInputSystemContextChangeIgnoreHeldKeysTest::RunTest(const FString& Parameters)
{
	// Test that a Pressed trigger does not re-fire when a different input context is added while
	// the key is still held. The UE-270589 fix replaces triggers with fresh instances during
	// rebuild, but runtime state (LastValue) is transferred from old to new triggers so that
	// Pressed correctly sees "already actuated" rather than a false edge. (UE-372464)

	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));

	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputAction(Data, TestAction, EInputActionValueType::Boolean));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));
	AND(ATriggerIsAppliedToAnAction(Data, NewObject<UInputTriggerPressed>(), TestAction));

	// 1 - Press key, Pressed trigger fires once
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	// 2 - Hold tick, trigger transitions to Completed
	WHEN(InputIsTicked(Data));
	THEN(HoldingKeyTriggersCompleted(Data, TestAction));

	// 3 - Hold tick, action is quiet (Pressed fires only once)
	WHEN(InputIsTicked(Data));
	THEN(HoldingKeyDoesNotTrigger(Data, TestAction));

	// 4 - Add a second context while key is held, forcing a mapping rebuild.
	// Using bForceImmediately to rebuild synchronously, like a runtime context change.
	// NOT using ControlMappingsAreRebuilt() — that clears ActionInstanceData (trigger state),
	// which is the data we are testing here.
	// bIgnoreAllPressedKeysUntilRelease defaults to true in FModifyContextOptions.
	{
		UInputMappingContext* SecondContext = NewObject<UInputMappingContext>();
		FModifyContextOptions Options;
		Options.bForceImmediately = true;
		Data.Subsystem->AddMappingContext(SecondContext, 10, Options);
	}

	// 5 - Tick after context change, Pressed must NOT re-fire (MAIN TEST)
	// Without the fix: fresh triggers have LastValue=0, so Pressed sees
	// IsActuated(current)=true AND !IsActuated(LastValue=0)=true -> re-fires.
	// With the fix: LastValue is transferred from old triggers, so Pressed correctly
	// sees !IsActuated(LastValue)=false and does not re-fire.
	WHEN(InputIsTicked(Data));
	THEN(HoldingKeyDoesNotTrigger(Data, TestAction));

	// 6 - Release key
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data));

	// 7 - Re-press key, Pressed trigger fires again normally.
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputSystemContextChangeNoHeldKeysTest, "Input.System.ContextChange.NoHeldKeys", BasicSystemTestFlags)

bool FInputSystemContextChangeNoHeldKeysTest::RunTest(const FString& Parameters)
{
	// Regression test: adding a context while no keys are held must not suppress subsequent
	// trigger firing. bShouldBeIgnored should only be set when a key is actually down.

	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));

	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputAction(Data, TestAction, EInputActionValueType::Boolean));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));
	AND(ATriggerIsAppliedToAnAction(Data, NewObject<UInputTriggerPressed>(), TestAction));

	// 1 - Add a second context with no keys held
	{
		UInputMappingContext* SecondContext = NewObject<UInputMappingContext>();
		FModifyContextOptions Options;
		Options.bForceImmediately = true;
		Data.Subsystem->AddMappingContext(SecondContext, 10, Options);
	}

	// 2 - Press key, Pressed trigger fires normally (not incorrectly suppressed)
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputSystemContextChangeDownTriggerTest, "Input.System.ContextChange.DownTriggerContinuesAfterRebuild", BasicSystemTestFlags)

bool FInputSystemContextChangeDownTriggerTest::RunTest(const FString& Parameters)
{
	// Test that a Down trigger continues to fire every tick after a mapping rebuild caused by
	// adding a second IMC while the key is held. The previous bShouldBeIgnored-based fix for
	// UE-372464 would skip the entire mapping, causing the Down action to fire Completed and
	// go dead. The correct fix transfers trigger runtime state so no mapping skip is needed.

	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));

	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputAction(Data, TestAction, EInputActionValueType::Boolean));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));
	AND(ATriggerIsAppliedToAnAction(Data, NewObject<UInputTriggerDown>(), TestAction));

	// 1 - Press key, Down trigger fires Triggered
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));

	// 2 - Hold tick, Down continues to fire Triggered
	WHEN(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));

	// 3 - Add a second context while key is held, forcing a mapping rebuild
	{
		UInputMappingContext* SecondContext = NewObject<UInputMappingContext>();
		FModifyContextOptions Options;
		Options.bForceImmediately = true;
		Data.Subsystem->AddMappingContext(SecondContext, 10, Options);
	}

	// 4 - Tick after context change, Down must STILL fire Triggered (MAIN TEST)
	// Without the fix: bShouldBeIgnored skips the mapping -> Triggered->None -> Completed -> dead
	// With the fix: trigger state is transferred, no mapping skip, Down continues normally
	WHEN(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));

	// 5 - Another hold tick, still firing
	WHEN(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));

	// 6 - Release key, Down transitions to Completed
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	// 7 - Next tick quiet
	WHEN(InputIsTicked(Data));
	THEN(FInputTestHelper::TestNoTrigger(Data, TestAction));

	return true;
}
