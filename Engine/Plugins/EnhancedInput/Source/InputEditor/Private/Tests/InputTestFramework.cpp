// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputTestFramework.h"

#include "Engine/Level.h"
#include "Engine/World.h"
#include "EnhancedInputComponent.h"
#include "GameFramework/PlayerController.h"
#include "InputMappingContext.h"
#include "Tests/AutomationEditorCommon.h"
#include "UserSettings/EnhancedInputUserSettings.h"
#include "NativeGameplayTags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputTestFramework)

void UMockedEnhancedInputSubsystem::Init(const UControllablePlayer& PlayerData)
{
	PlayerInput = PlayerData.PlayerInput;
	UserSettings = PlayerData.UserSettings;
	InitalizeUserSettings();
}

UEnhancedInputUserSettings* UMockedEnhancedInputSubsystem::GetUserSettings() const
{
	return UserSettings.Get();
}

void UMockedEnhancedInputSubsystem::InitalizeUserSettings()
{
	ULocalPlayer* LP = PlayerInput->GetOwningLocalPlayer();
	
	UserSettings->Initialize(LP);
	UserSettings->ApplySettings();
}

void UMockInputUserSettings::Initialize(ULocalPlayer* LP)
{
	// Create a default key mapping profile in the case where one doesn't exist
	if (!GetActiveKeyProfile())
	{
		const FString Profile_Default_Requested_ID = TEXT("input.test.profile.default");

		FPlayerMappableKeyProfileCreationArgs Args = {};
		Args.bSetAsCurrentProfile = true;
		Args.ProfileStringIdentifier = Profile_Default_Requested_ID;
	
		CreateNewKeyProfile(Args);
	}
}

UWorld* AnEmptyWorld()
{
#if WITH_AUTOMATION_TESTS
	return FAutomationEditorCommonUtils::CreateNewMap();
#else
	return nullptr;
#endif
}

UControllablePlayer& AControllablePlayer(UWorld* World)
{
	EKeys::Initialize();

	UControllablePlayer* PlayerData = NewObject<UControllablePlayer>(World);

	PlayerData->Player = NewObject<APlayerController>(World->GetCurrentLevel());
	PlayerData->Player->InputComponent = NewObject<UEnhancedInputComponent>(PlayerData->Player);
	PlayerData->Player->PlayerInput = NewObject<UEnhancedPlayerInput>(PlayerData->Player);
	PlayerData->PlayerInput = Cast<UEnhancedPlayerInput>(PlayerData->Player->PlayerInput);
	PlayerData->InputComponent = Cast<UEnhancedInputComponent>(PlayerData->Player->InputComponent);
	PlayerData->Player->InitInputSystem();
	PlayerData->UserSettings = NewObject<UMockInputUserSettings>(GetTransientPackage(), UMockInputUserSettings::StaticClass());

	PlayerData->Subsystem = NewObject<UMockedEnhancedInputSubsystem>(PlayerData->Player);
	PlayerData->Subsystem->Init(*PlayerData);

	check(PlayerData->IsValid());

	return *PlayerData;
}

UInputMappingContext* AnInputContextIsAppliedToAPlayer(UControllablePlayer& PlayerData, FName ContextName, int32 WithPriority)
{
	UInputMappingContext* Context = PlayerData.InputContext.Emplace(ContextName, NewObject<UInputMappingContext>(PlayerData.Player, ContextName));
	PlayerData.Subsystem->AddMappingContext(Context, WithPriority);
	return Context;
}

UInputMappingContext* AnInputContextWithTrackingModeIsAppliedToAPlayer(UControllablePlayer& PlayerData, FName ContextName, int32 WithPriority, EMappingContextRegistrationTrackingMode TrackingMode)
{
	// IMC Extension that allows simulating a user setting a tracking mode in the editor
	class UInputMappingContextWithTrackingSetter : public UInputMappingContext
	{
	public:
		void SetTrackingMode(EMappingContextRegistrationTrackingMode InTrackingMode)
		{
			RegistrationTrackingMode = InTrackingMode;
		}
	};
	
	UInputMappingContextWithTrackingSetter* Context = NewObject<UInputMappingContextWithTrackingSetter>(PlayerData.Player, ContextName);
	Context->SetTrackingMode(TrackingMode);
	PlayerData.InputContext.Emplace(ContextName, Context);
	PlayerData.Subsystem->AddMappingContext(Context, WithPriority);
	return Context;
}

UInputMappingContext* AnInputContextIsReappliedToAPlayer(UControllablePlayer& PlayerData, FName ContextName, int32 WithPriority)
{
	UInputMappingContext* Context = PlayerData.InputContext.FindRef(ContextName);
	if (ensureMsgf(Context, TEXT("For an IMC to be reapplied, it must already exist")))
	{
		PlayerData.Subsystem->AddMappingContext(Context, WithPriority);
	}
	return Context;
}

UInputMappingContext* AnInputContextIsRemovedFromAPlayer(UControllablePlayer& PlayerData, FName ContextName)
{
	UInputMappingContext* Context = PlayerData.InputContext.FindRef(ContextName);
	if (ensureMsgf(Context, TEXT("For an IMC to be removed, it must already exist")))
	{
		PlayerData.Subsystem->RemoveMappingContext(Context);
	}
	return Context;
}

UInputAction* AnInputAction(UControllablePlayer& PlayerData, FName ActionName, EInputActionValueType ValueType)
{
	UTestMappableKeysAction* Action = NewObject<UTestMappableKeysAction>(PlayerData.Player, ActionName);
	Action->ValueType = ValueType;
	Action->SetPlayerMappableKeyOptions(ActionName);
	
	return PlayerData.InputAction.Emplace(ActionName, Action);
}

void ControlMappingsAreRebuilt(UControllablePlayer& PlayerData)
{
	FInputTestHelper::ResetActionInstanceData(PlayerData);
	FModifyContextOptions Options;
	Options.bForceImmediately = true;
	PlayerData.Subsystem->RequestRebuildControlMappings(Options);
}

FEnhancedActionKeyMapping& AnActionIsMappedToAKey(UControllablePlayer& PlayerData, FName ContextName, FName ActionName, FKey Key)
{
	UInputMappingContext* Context = FInputTestHelper::FindContext(PlayerData, ContextName);
	UInputAction* Action = FInputTestHelper::FindAction(PlayerData, ActionName);
	if(Context && Action)
	{
		// Bind action to the binding targets so we can check if they were called correctly, but only the first time we bind the action!
		bool bAlreadyBound = false;
		PlayerData.MappedActionListeners.Add(Action, &bAlreadyBound);
		if (!bAlreadyBound)
		{
			FBindingTargets& BindingTargets = PlayerData.BindingTargets.Emplace(ActionName, FBindingTargets(PlayerData.Player));

			PlayerData.InputComponent->BindAction(Action, ETriggerEvent::Started, BindingTargets.Started.Get(), &UInputBindingTarget::MappingListener);
			PlayerData.InputComponent->BindAction(Action, ETriggerEvent::Ongoing, BindingTargets.Ongoing.Get(), &UInputBindingTarget::MappingListener);
			PlayerData.InputComponent->BindAction(Action, ETriggerEvent::Canceled, BindingTargets.Canceled.Get(), &UInputBindingTarget::MappingListener);
			PlayerData.InputComponent->BindAction(Action, ETriggerEvent::Completed, BindingTargets.Completed.Get(), &UInputBindingTarget::MappingListener);
			PlayerData.InputComponent->BindAction(Action, ETriggerEvent::Triggered, BindingTargets.Triggered.Get(), &UInputBindingTarget::MappingListener);
		}

		// Initialise mapping in context
		Context->MapKey(Action, Key);

		// Generate a live mapping on the player
		ControlMappingsAreRebuilt(PlayerData);
		if (FEnhancedActionKeyMapping* Mapping = FInputTestHelper::FindLiveActionMapping(PlayerData, ActionName, Key))
		{
			return *Mapping;
		}
	}

	// TODO: Error here.
	static FEnhancedActionKeyMapping NullMapping;
	return NullMapping;
}

UInputModifier* AModifierIsAppliedToAnAction(UControllablePlayer& PlayerData, class UInputModifier* Modifier, FName ActionName)
{
	if (UInputAction* Action = FInputTestHelper::FindAction(PlayerData, ActionName))
	{
		Action->Modifiers.Add(Modifier);

		// Control mapping rebuild required to recalculate modifier default values
		// TODO: This will be an issue for run time modification of modifiers
		ControlMappingsAreRebuilt(PlayerData);
		if(FInputTestHelper::HasActionData(PlayerData, ActionName))
		{
			const TArray<UInputModifier*>& TestModifiers = FInputTestHelper::GetActionData(PlayerData, ActionName).GetModifiers();
			// If the action hasn't been mapped to yet we can't get a valid instance. TODO: assert?
			if(ensure(!TestModifiers.IsEmpty()))
			{
				return TestModifiers.Last();
			}
		}
	}
	return nullptr;
}

UInputModifier* AModifierIsAppliedToAnActionMapping(UControllablePlayer& PlayerData, class UInputModifier* Modifier, FName ContextName, FName ActionName, FKey Key)
{
	if (UInputMappingContext* Context = FInputTestHelper::FindContext(PlayerData, ContextName))
	{
		TArray<FEnhancedActionKeyMapping>::SizeType MappingIdx = Context->GetMappings().IndexOfByPredicate(
			[&ActionName, &Key](const FEnhancedActionKeyMapping& Mapping) {
				return Mapping.Action->GetFName() == ActionName && Mapping.Key == Key;
			});
		if (MappingIdx != INDEX_NONE)
		{
			Context->GetMapping(MappingIdx).Modifiers.Add(Modifier);

			// Control mapping rebuild required to recalculate modifier default values
			ControlMappingsAreRebuilt(PlayerData);	// Generate the live mapping instance for this key
			if (FEnhancedActionKeyMapping* LiveMapping = FInputTestHelper::FindLiveActionMapping(PlayerData, ActionName, Key))
			{
				return !LiveMapping->Modifiers.IsEmpty() ? LiveMapping->Modifiers.Last() : nullptr;
			}
		}
	}
	return nullptr;
}

UInputTrigger* ATriggerIsAppliedToAnAction(UControllablePlayer& PlayerData, class UInputTrigger* Trigger, FName ActionName)
{
	if (UInputAction* Action = FInputTestHelper::FindAction(PlayerData, ActionName))
	{
		Action->Triggers.Add(Trigger);
		ControlMappingsAreRebuilt(PlayerData);
		if(FInputTestHelper::HasActionData(PlayerData, ActionName))
		{
			const FInputActionInstance& ActionInstance = FInputTestHelper::GetActionData(PlayerData, ActionName);
			const TArray<UInputTrigger*>& InstanceTriggers = ActionInstance.GetTriggers();
			// If the action hasn't been mapped to yet we can't get a valid instance. TODO: assert?
			if(ensure(!InstanceTriggers.IsEmpty()))
			{
				return InstanceTriggers.Last();
			}
		}
	}
	return nullptr;
}

UInputTrigger* ATriggerIsAppliedToAnActionMapping(UControllablePlayer& PlayerData, class UInputTrigger* Trigger, FName ContextName, FName ActionName, FKey Key)
{
	if (UInputMappingContext* Context = FInputTestHelper::FindContext(PlayerData, ContextName))
	{
		TArray<FEnhancedActionKeyMapping>::SizeType MappingIdx = Context->GetMappings().IndexOfByPredicate(
			[&ActionName, &Key](const FEnhancedActionKeyMapping& Mapping) {
				return Mapping.Action->GetFName() == ActionName && Mapping.Key == Key;
			});
		if (MappingIdx != INDEX_NONE)
		{
			Context->GetMapping(MappingIdx).Triggers.Add(Trigger);
			ControlMappingsAreRebuilt(PlayerData);	// Generate the live mapping instance for this key
			if (FEnhancedActionKeyMapping* LiveMapping = FInputTestHelper::FindLiveActionMapping(PlayerData, ActionName, Key))
			{
				return !LiveMapping->Triggers.IsEmpty() ? LiveMapping->Triggers.Last() : nullptr;
			}
		}
	}
	return nullptr;
}

void AKeyIsActuated(UControllablePlayer& PlayerData, FKey Key, float Delta)
{
	if (Key.IsAnalog())
	{
		PlayerData.Player->InputKey(FInputKeyEventArgs::CreateSimulated(Key, IE_Axis, Delta));
	}
	else
	{
		PlayerData.Player->InputKey(FInputKeyEventArgs::CreateSimulated(Key, IE_Pressed, 1.0f));
	}
}

void AKeyIsReleased(UControllablePlayer& PlayerData, FKey Key)
{
	if (Key.IsAnalog())
	{
		PlayerData.Player->InputKey(FInputKeyEventArgs::CreateSimulated(Key, IE_Axis, 0.0f));
	}
	else
	{
		PlayerData.Player->InputKey(FInputKeyEventArgs::CreateSimulated(Key, IE_Released, 0.0f));
	}
}

void AnInputIsInjected(UControllablePlayer& PlayerData, FName ActionName, FInputActionValue Value)
{
	if (UInputAction* Action = FInputTestHelper::FindAction(PlayerData, ActionName))
	{
		PlayerData.PlayerInput->InjectInputForAction(Action, Value);
	}
}

void InputIsTicked(UControllablePlayer& PlayerData, float Delta)
{
	// Reset any binding triggered state before the tick
	for (TPair<FName, FBindingTargets>& BindingPair : PlayerData.BindingTargets)
	{
		FBindingTargets& BindingTarget = BindingPair.Value;
		BindingTarget.Started->bTriggered = false;
		BindingTarget.Ongoing->bTriggered = false;
		BindingTarget.Canceled->bTriggered = false;
		BindingTarget.Completed->bTriggered = false;
		BindingTarget.Triggered->bTriggered = false;
	}

	PlayerData.Player->PlayerTick(Delta);
}

UInputMappingContext* FInputTestHelper::FindContext(UControllablePlayer& Data, FName ContextName)
{
	TObjectPtr<UInputMappingContext>* Context = Data.InputContext.Find(ContextName);
	return Context ? *Context : nullptr;
}

UInputAction* FInputTestHelper::FindAction(UControllablePlayer& Data, FName ActionName)
{
	TObjectPtr<UInputAction>* Action = Data.InputAction.Find(ActionName);
	return Action ? *Action : nullptr;
}

FEnhancedActionKeyMapping* FInputTestHelper::FindLiveActionMapping(UControllablePlayer& Data, FName ActionName, FKey Key)
{
	// TODO: Potential failure when two identical action/key mappings are applied (but with different modifiers/triggers)
	return Data.PlayerInput->EnhancedActionMappings.FindByPredicate([&ActionName, &Key](FEnhancedActionKeyMapping& Test) {
			return Test.Action->GetFName() == ActionName && Test.Key == Key;
		});
}

bool FInputTestHelper::HasActionData(UControllablePlayer& Data, FName ActionName)
{
	UInputAction* Action = FInputTestHelper::FindAction(Data, ActionName);
	return Action && Data.PlayerInput->ActionInstanceData.Find(Action) != nullptr;
}

void FInputTestHelper::ResetActionInstanceData(UControllablePlayer& Data)
{
	Data.PlayerInput->ActionInstanceData.Empty();
}

const FInputActionInstance& FInputTestHelper::GetActionData(UControllablePlayer& Data, FName ActionName)
{
	if(UInputAction* Action = FInputTestHelper::FindAction(Data, ActionName))
	{
		if (FInputActionInstance* EventData = Data.PlayerInput->ActionInstanceData.Find(Action))
		{
			return *EventData;
		}
	}

	static FInputActionInstance NoActionData;
	return NoActionData;
}

bool FInputTestHelper::HasAppliedInputMappingContext(UControllablePlayer& Data, FName ContextName)
{
	return Data.PlayerInput->GetAppliedInputContextData().Contains(FInputTestHelper::FindContext(Data, ContextName));
}
