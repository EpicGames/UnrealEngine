// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "EnhancedInputSubsystemInterface.h"
#include "PlayerMappableKeySettings.h"
#include "UserSettings/EnhancedInputUserSettings.h"
#include "InputTestFramework.generated.h"

enum class EMappingContextRegistrationTrackingMode : uint8;
struct FEnhancedActionKeyMapping;
class UEnhancedPlayerInput;

UCLASS()
class UInputBindingTarget : public UObject
{
	GENERATED_BODY()

public:
	bool bTriggered;
	FInputActionValue Result;
	float ElapsedTime;
	float TriggeredTime;

	// Records delegate triggering result, allowing tests to validate that they fired correctly
	void MappingListener(const FInputActionInstance& Instance)
	{
		bTriggered = true;
		Result = Instance.GetValue();
		ElapsedTime = Instance.GetElapsedTime();
		TriggeredTime = Instance.GetTriggeredTime();
	}
};

USTRUCT()
struct FBindingTargets
{
	GENERATED_BODY()

	FBindingTargets() = default;

	FBindingTargets(APlayerController* Owner)
		: Started(NewObject<UInputBindingTarget>((UObject*)Owner))
		, Ongoing(NewObject<UInputBindingTarget>((UObject*)Owner))
		, Canceled(NewObject<UInputBindingTarget>((UObject*)Owner))
		, Completed(NewObject<UInputBindingTarget>((UObject*)Owner))
		, Triggered(NewObject<UInputBindingTarget>((UObject*)Owner))
	{
	}

	UPROPERTY()
	TObjectPtr<UInputBindingTarget> Started = nullptr;

	UPROPERTY()
	TObjectPtr<UInputBindingTarget> Ongoing = nullptr;

	UPROPERTY()
	TObjectPtr<UInputBindingTarget> Canceled = nullptr;

	UPROPERTY()
	TObjectPtr<UInputBindingTarget> Completed = nullptr;

	UPROPERTY()
	TObjectPtr<UInputBindingTarget> Triggered = nullptr;
};

// Mock input subsystems to avoid having to create an actual subsystem + local player + game instance.
UCLASS()
class UMockedEnhancedInputSubsystem : public UObject, public IEnhancedInputSubsystemInterface
{
	GENERATED_BODY()
	
	TWeakObjectPtr<UEnhancedPlayerInput> PlayerInput;
	
	TWeakObjectPtr<UEnhancedInputUserSettings> UserSettings;
	
public:
	void Init(const class UControllablePlayer& PlayerData); 
	virtual UEnhancedPlayerInput* GetPlayerInput() const override { return PlayerInput.Get(); }
	virtual UEnhancedInputUserSettings* GetUserSettings() const override;
	virtual void InitalizeUserSettings() override;

protected:
	virtual TMap<TObjectPtr<const UInputAction>, FInjectedInput>& GetContinuouslyInjectedInputs() override { return ContinuouslyInjectedInputs; }
	
	// Map of inputs that should be injected every frame. These inputs will be injected when ForcedInput is ticked.
	UPROPERTY(Transient)
	TMap<TObjectPtr<const UInputAction>, FInjectedInput> ContinuouslyInjectedInputs;
};

UCLASS(hidedropdown, NotBlueprintable, NotBlueprintType)
class UMockInputUserSettings : public UEnhancedInputUserSettings
{
	GENERATED_BODY()
public:
	// There won't be an owning local player here so we will override it to avoid unnecessary error logging.
	virtual void Initialize(ULocalPlayer* LP) override;
};

UCLASS()
class UControllablePlayer : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TObjectPtr<class APlayerController> Player;

	// These exist on (and have their lifetime managed by) Player, but are downcast here for rapid access.
	TWeakObjectPtr<UEnhancedPlayerInput> PlayerInput;
	TWeakObjectPtr<class UEnhancedInputComponent> InputComponent;

	UPROPERTY()
	TMap<FName, FBindingTargets> BindingTargets;

	// Storage for input mapping contexts applied to the player
	UPROPERTY()
	TMap<FName, TObjectPtr<UInputMappingContext>> InputContext;

	// Storage for input actions applied to the player
	UPROPERTY()
	TMap<FName, TObjectPtr<UInputAction>> InputAction;

	// Ensure we don't try to double bind listeners when applying multiple key mappings
	TSet<const UInputAction*> MappedActionListeners;

	// The user settings object that we can use to test. Normally this is owned by the EI Local Player subsystem 
	UPROPERTY()
	TObjectPtr<UEnhancedInputUserSettings> UserSettings;

	bool IsValid() const { return Player && Subsystem.IsValid() && PlayerInput.IsValid() && InputComponent.IsValid() && UserSettings; }

	// Mocked subsystem implementing IEnhancedInputSubsystemInterface
	TWeakObjectPtr<UMockedEnhancedInputSubsystem> Subsystem;
};

// A subclass of UInputAction that will have it's player mappable key settings object set automatically so 
// we can test re-mappable keys.
UCLASS(hidedropdown, NotBlueprintable, NotBlueprintType)
class UTestMappableKeysAction : public UInputAction
{
	GENERATED_BODY()
public:

	void SetPlayerMappableKeyOptions(const FName MappingName)
	{
		PlayerMappableKeySettings = NewObject<UPlayerMappableKeySettings>(this, TEXT("TestMappingSettings"));
		PlayerMappableKeySettings->Name = MappingName;
	}
};

// Default test values
static FName TestContext = TEXT("TestContext");
static FName TestAction = TEXT("TestAction");
static FName TestAction2 = TEXT("TestAction2");
static FName TestAction3 = TEXT("TestAction3");
static FName TestAction4 = TEXT("TestAction4");
static FName TestAction5 = TEXT("TestAction5");
static FName TestAction6 = TEXT("TestAction6");
// TODO: Create custom keys for the lifetime of the test. Using public keys can cause interference (e.g. AxisConfigs)
static FKey TestKey = EKeys::A;
static FKey TestKey2 = EKeys::S;
static FKey TestKey3 = EKeys::D;
static FKey TestKey4 = EKeys::F;
static FKey TestAxis = EKeys::Gamepad_LeftTriggerAxis;
static FKey TestAxis2 = EKeys::Gamepad_Special_Left_X;
static FKey TestAxis3 = EKeys::Gamepad_RightTriggerAxis;
static FKey TestAxis4 = EKeys::Gamepad_Special_Left_Y;

// FInputTestHelper has friend access to UEnhancedPlayerInput
struct FInputTestHelper
{
	// Test each event type to see if it fired

	static bool TestNoTrigger(UControllablePlayer& Data, FName ActionName)
	{
		FBindingTargets& BindingTargets = Data.BindingTargets[ActionName];
		return !(BindingTargets.Started->bTriggered | BindingTargets.Ongoing->bTriggered | BindingTargets.Canceled->bTriggered  | BindingTargets.Completed->bTriggered | BindingTargets.Triggered->bTriggered);
	}

	static bool TestStarted(UControllablePlayer& Data, FName ActionName)
	{
		return Data.BindingTargets[ActionName].Started->bTriggered;
	}


	static bool TestOngoing(UControllablePlayer& Data, FName ActionName)
	{
		return Data.BindingTargets[ActionName].Ongoing->bTriggered;
	}

	static bool TestCanceled(UControllablePlayer& Data, FName ActionName)
	{
		return Data.BindingTargets[ActionName].Canceled->bTriggered;
	}

	static bool TestCompleted(UControllablePlayer& Data, FName ActionName)
	{
		return Data.BindingTargets[ActionName].Completed->bTriggered;
	}

	static bool TestTriggered(UControllablePlayer& Data, FName ActionName)
	{
		return Data.BindingTargets[ActionName].Triggered->bTriggered;
	}


	// Read results from each event type

	template<typename T>
	static T GetStarted(UControllablePlayer& Data, FName ActionName)
	{
		return Data.BindingTargets[ActionName].Started->Result.Get<T>();
	}

	template<typename T>
	static T GetOngoing(UControllablePlayer& Data, FName ActionName)
	{
		return Data.BindingTargets[ActionName].Ongoing->Result.Get<T>();
	}

	template<typename T>
	static T GetCanceled(UControllablePlayer& Data, FName ActionName)
	{
		return Data.BindingTargets[ActionName].Canceled->Result.Get<T>();
	}

	template<typename T>
	static T GetCompleted(UControllablePlayer& Data, FName ActionName)
	{
		return Data.BindingTargets[ActionName].Completed->Result.Get<T>();
	}

	template<typename T>
	static T GetTriggered(UControllablePlayer& Data, FName ActionName)
	{
		return Data.BindingTargets[ActionName].Triggered->Result.Get<T>();
	}

	static UInputMappingContext* FindContext(UControllablePlayer& Data, FName ContextName);
	static UInputAction* FindAction(UControllablePlayer& Data, FName ActionName);
	static FEnhancedActionKeyMapping* FindLiveActionMapping(UControllablePlayer& Data, FName ActionName, FKey Key);	// NOTE: This is an action to key mapping mapped to a player, not one from an input context

	static bool HasActionData(UControllablePlayer& Data, FName ActionName);

	static void ResetActionInstanceData(UControllablePlayer& Data);

	static const struct FInputActionInstance& GetActionData(UControllablePlayer& Data, FName ActionName);

	static bool HasAppliedInputMappingContext(UControllablePlayer& Data, FName ContextName);
};

// TODO: Switch to something like Catch2?
#define GIVEN(X) X
#define WHEN(X) X
#define AND(X) X
#define THEN(X) TestTrueExpr(X)
// TODO: Would be nice to be able to do THEN(X) AND(Y) and have TestTrueExpr called for the AND.
#define ANDALSO(X) TestTrueExpr(X)

// Reusable step definitions

UWorld* AnEmptyWorld();
UControllablePlayer& AControllablePlayer(UWorld* World);
UInputMappingContext* AnInputContextIsAppliedToAPlayer(UControllablePlayer& PlayerData, FName ContextName, int32 WithPriority);
UInputMappingContext* AnInputContextWithTrackingModeIsAppliedToAPlayer(UControllablePlayer& PlayerData, FName ContextName, int32 WithPriority, EMappingContextRegistrationTrackingMode TrackingMode);
UInputMappingContext* AnInputContextIsReappliedToAPlayer(UControllablePlayer& PlayerData, FName ContextName, int32 WithPriority);
UInputMappingContext* AnInputContextIsRemovedFromAPlayer(UControllablePlayer& PlayerData, FName ContextName);
UInputAction* AnInputAction(UControllablePlayer& PlayerData, FName ActionName, EInputActionValueType ValueType);
FEnhancedActionKeyMapping& AnActionIsMappedToAKey(UControllablePlayer& PlayerData, FName ContextName, FName ActionName, FKey Key); // NOTE: Adds a template mapping to the input context but returns the resulting live action mapping on the player

UInputModifier* AModifierIsAppliedToAnAction(UControllablePlayer& PlayerData, class UInputModifier* Modifier, FName ActionName);
UInputModifier* AModifierIsAppliedToAnActionMapping(UControllablePlayer& PlayerData, class UInputModifier* Modifier, FName ContextName, FName ActionName, FKey Key);		// NOTE: This will not be the modifier passed in, but an instanced copy.
UInputTrigger* ATriggerIsAppliedToAnAction(UControllablePlayer& PlayerData, class UInputTrigger* Trigger, FName ActionName);
UInputTrigger* ATriggerIsAppliedToAnActionMapping(UControllablePlayer& PlayerData, class UInputTrigger* Trigger, FName ContextName, FName ActionName, FKey Key);			// NOTE: This will not be the trigger passed in, but an instanced copy.
void AKeyIsActuated(UControllablePlayer& PlayerData, FKey Key, float Delta = 1.f);
void AKeyIsReleased(UControllablePlayer& PlayerData, FKey Key);
void AnInputIsInjected(UControllablePlayer& PlayerData, FName ActionName, FInputActionValue Value);
void InputIsTicked(UControllablePlayer& PlayerData, float Delta = 1.f / 60.f);

// THEN step wrappers to give human readable test failure output.

inline bool PressingKeyDoesNotTrigger(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestNoTrigger(Data, ActionName); }
inline bool PressingKeyTriggersStarted(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestStarted(Data, ActionName); }
inline bool PressingKeyTriggersOngoing(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestOngoing(Data, ActionName); }
inline bool PressingKeyTriggersCanceled(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestCanceled(Data, ActionName); }
inline bool PressingKeyTriggersCompleted(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestCompleted(Data, ActionName); }
inline bool PressingKeyTriggersAction(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestTriggered(Data, ActionName); }

inline bool HoldingKeyDoesNotTrigger(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestNoTrigger(Data, ActionName); }
inline bool HoldingKeyTriggersStarted(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestStarted(Data, ActionName); }
inline bool HoldingKeyTriggersOngoing(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestOngoing(Data, ActionName); }
inline bool HoldingKeyTriggersCanceled(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestCanceled(Data, ActionName); }
inline bool HoldingKeyTriggersCompleted(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestCompleted(Data, ActionName); }
inline bool HoldingKeyTriggersAction(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestTriggered(Data, ActionName); }

inline bool ReleasingKeyDoesNotTrigger(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestNoTrigger(Data, ActionName); }
inline bool ReleasingKeyTriggersStarted(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestStarted(Data, ActionName); }
inline bool ReleasingKeyTriggersOngoing(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestOngoing(Data, ActionName); }
inline bool ReleasingKeyTriggersCanceled(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestCanceled(Data, ActionName); }
inline bool ReleasingKeyTriggersCompleted(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestCompleted(Data, ActionName); }
inline bool ReleasingKeyTriggersAction(UControllablePlayer& Data, FName ActionName) { return FInputTestHelper::TestTriggered(Data, ActionName); }

inline bool InputContextIsApplied(UControllablePlayer& Data, FName ContextName) { return FInputTestHelper::HasAppliedInputMappingContext(Data, ContextName); }
inline bool InputContextIsNotApplied(UControllablePlayer& Data, FName ContextName) { return !FInputTestHelper::HasAppliedInputMappingContext(Data, ContextName); }
