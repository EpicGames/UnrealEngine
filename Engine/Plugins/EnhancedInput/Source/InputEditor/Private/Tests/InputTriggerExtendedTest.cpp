// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputTestFramework.h"
#include "Misc/AutomationTest.h"
#include "InputTriggers.h"

// Extended trigger integration tests: Pulse, Hold (OneShot), HoldAndRelease, Tap, and ChordBlocker.
// These are integration-style tests using a full UControllablePlayer+world, which lets time-based
// triggers receive proper DeltaTime through the input stack.

constexpr auto BasicTriggerExtTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter;

// Local helper: creates world+player+context+action+key mapping and applies trigger T at the action level.
// Returns the applied trigger instance cast to T*.
template<typename T>
static T* ABasicTriggerExtTest(FAutomationTestBase* Test, UControllablePlayer*& OutData)
{
	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	Test->TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());

	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputAction(Data, TestAction, EInputActionValueType::Boolean));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));

	T* TriggerInstance = Cast<T>(ATriggerIsAppliedToAnAction(Data, NewObject<T>(), TestAction));
	Test->TestNotNull(TEXT("Trigger instance created"), TriggerInstance);

	OutData = &Data;
	return TriggerInstance;
}

// ============================================================
// Pulse trigger
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputIntegrationTriggerPulseTest, "Input.Integration.Triggers.Pulse", BasicTriggerExtTestFlags)

bool FInputIntegrationTriggerPulseTest::RunTest(const FString& Parameters)
{
	const float FrameTime = 1.f / 60.f;
	// Use a 30-frame interval. Setting Interval to 29.5 frames ensures the fire tick (frame 30)
	// is strictly past the threshold with no floating-point ambiguity (avoids HeldDuration == Interval).
	const int32 IntervalFrames = 30;
	const float Interval = FrameTime * (IntervalFrames - 0.5f);

	// -------------------------------------------------------
	// Test 1 - bTriggerOnStart = true:
	//   First tick while held -> immediately fires Triggered (HeldDuration > Interval*0 = 0).
	//   Subsequent ticks before the next interval -> Ongoing (not triggered).
	//   At the next interval threshold -> fires again.
	// -------------------------------------------------------
	{
		UControllablePlayer* DataPtr = nullptr;
		UInputTriggerPulse* Pulse = ABasicTriggerExtTest<UInputTriggerPulse>(this, DataPtr);
		check(Pulse && DataPtr);
		UControllablePlayer& Data = *DataPtr;

		Pulse->bTriggerOnStart = true;
		Pulse->Interval = Interval;
		Pulse->TriggerLimit = 0; // unlimited

		// First tick: HeldDuration = FrameTime > 0 -> triggers immediately
		WHEN(AKeyIsActuated(Data, TestKey));
		AND(InputIsTicked(Data, FrameTime));
		THEN(PressingKeyTriggersAction(Data, TestAction));

		// Mid-interval ticks: HeldDuration < Interval*1 -> Ongoing only
		for (int32 Frame = 1; Frame < IntervalFrames - 1; ++Frame)
		{
			WHEN(InputIsTicked(Data, FrameTime));
			THEN(HoldingKeyTriggersOngoing(Data, TestAction));
		}

		// At/past the interval threshold (frame IntervalFrames): triggers again
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersAction(Data, TestAction));

		// Releasing transitions to Completed
		WHEN(AKeyIsReleased(Data, TestKey));
		AND(InputIsTicked(Data, FrameTime));
		THEN(ReleasingKeyTriggersCompleted(Data, TestAction));
	}

	// -------------------------------------------------------
	// Test 2 - bTriggerOnStart = false:
	//   First tick -> Ongoing (not triggered yet).
	//   At the first interval threshold -> fires Triggered.
	//   Second interval -> fires again.
	// -------------------------------------------------------
	{
		UControllablePlayer* DataPtr = nullptr;
		UInputTriggerPulse* Pulse = ABasicTriggerExtTest<UInputTriggerPulse>(this, DataPtr);
		check(Pulse && DataPtr);
		UControllablePlayer& Data = *DataPtr;

		Pulse->bTriggerOnStart = false;
		Pulse->Interval = Interval;
		Pulse->TriggerLimit = 0;

		// First tick: HeldDuration = FrameTime < Interval -> Ongoing.
		// State transition: None->Ongoing fires Started (not Ongoing).
		WHEN(AKeyIsActuated(Data, TestKey));
		AND(InputIsTicked(Data, FrameTime));
		THEN(PressingKeyTriggersStarted(Data, TestAction));

		// Ticks up to but not including the interval frame
		for (int32 Frame = 1; Frame < IntervalFrames - 1; ++Frame)
		{
			WHEN(InputIsTicked(Data, FrameTime));
			THEN(HoldingKeyTriggersOngoing(Data, TestAction));
		}

		// At the interval threshold: fires first Triggered
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersAction(Data, TestAction));

		// Mid-second-interval: Ongoing again
		for (int32 Frame = 0; Frame < IntervalFrames - 1; ++Frame)
		{
			WHEN(InputIsTicked(Data, FrameTime));
			THEN(HoldingKeyTriggersOngoing(Data, TestAction));
		}

		// At the second interval: fires again
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersAction(Data, TestAction));
	}

	// -------------------------------------------------------
	// Test 3 - TriggerLimit = 2: fires at most twice, then goes silent (None)
	// -------------------------------------------------------
	{
		UControllablePlayer* DataPtr = nullptr;
		UInputTriggerPulse* Pulse = ABasicTriggerExtTest<UInputTriggerPulse>(this, DataPtr);
		check(Pulse && DataPtr);
		UControllablePlayer& Data = *DataPtr;

		Pulse->bTriggerOnStart = true;
		Pulse->Interval = Interval;
		Pulse->TriggerLimit = 2;

		// First fire: frame 1 (bTriggerOnStart)
		WHEN(AKeyIsActuated(Data, TestKey));
		AND(InputIsTicked(Data, FrameTime));
		THEN(PressingKeyTriggersAction(Data, TestAction));

		// Ticks through to second fire
		for (int32 Frame = 1; Frame < IntervalFrames - 1; ++Frame)
		{
			WHEN(InputIsTicked(Data, FrameTime));
			THEN(HoldingKeyTriggersOngoing(Data, TestAction));
		}

		// Second fire at interval threshold (TriggerCount will reach limit here)
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersAction(Data, TestAction));

		// After TriggerLimit reached, Pulse returns None. LastState was Triggered -> Triggered->None=Completed.
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersCompleted(Data, TestAction));

		// Further ticks: None->None -> no event.
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyDoesNotTrigger(Data, TestAction));
	}

	return true;
}

// ============================================================
// Hold trigger - bIsOneShot = true
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputIntegrationTriggerHoldOneShotTest, "Input.Integration.Triggers.Hold.OneShot", BasicTriggerExtTestFlags)

bool FInputIntegrationTriggerHoldOneShotTest::RunTest(const FString& Parameters)
{
	const float FrameTime = 1.f / 60.f;
	const int32 HoldFrames = 30;

	UControllablePlayer* DataPtr = nullptr;
	UInputTriggerHold* HoldTrigger = ABasicTriggerExtTest<UInputTriggerHold>(this, DataPtr);
	check(HoldTrigger && DataPtr);
	UControllablePlayer& Data = *DataPtr;

	HoldTrigger->HoldTimeThreshold = FrameTime * HoldFrames;
	HoldTrigger->bIsOneShot = true;

	// -------------------------------------------------------
	// Test 1 - OneShot: fires exactly once at threshold, then stays silent while held.
	// -------------------------------------------------------

	// Pressing
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	// Holding until threshold
	for (int32 Frame = 1; Frame < HoldFrames - 1; ++Frame)
	{
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersOngoing(Data, TestAction));
	}

	// Reaching threshold: fires once
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(HoldingKeyTriggersAction(Data, TestAction));

	// Tick after OneShot: Hold (bIsOneShot=true, already triggered) returns None.
	// LastState=Triggered -> Triggered->None=Completed fires.
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(HoldingKeyTriggersCompleted(Data, TestAction));

	// Further ticks while held: None->None -> no event.
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(HoldingKeyDoesNotTrigger(Data, TestAction));

	// Releasing: state was already None after Completed, key release produces None->None -> no event.
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyDoesNotTrigger(Data, TestAction));

	// -------------------------------------------------------
	// Test 2 - OneShot reset: pressing again after release should fire once more.
	// -------------------------------------------------------
	InputIsTicked(Data, FrameTime); // clear state

	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	for (int32 Frame = 1; Frame < HoldFrames - 1; ++Frame)
	{
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersOngoing(Data, TestAction));
	}

	WHEN(InputIsTicked(Data, FrameTime));
	THEN(HoldingKeyTriggersAction(Data, TestAction)); // fires again on re-press

	// bIsOneShot: returns None immediately after firing -> Triggered->None=Completed.
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(HoldingKeyTriggersCompleted(Data, TestAction));

	return true;
}

// ============================================================
// HoldAndRelease trigger (integration - replaces the disabled unit test)
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputIntegrationTriggerHoldAndReleaseTest, "Input.Integration.Triggers.HoldAndRelease", BasicTriggerExtTestFlags)

bool FInputIntegrationTriggerHoldAndReleaseTest::RunTest(const FString& Parameters)
{
	const float FrameTime = 1.f / 60.f;
	const int32 HoldFrames = 30;

	UControllablePlayer* DataPtr = nullptr;
	UInputTriggerHoldAndRelease* HARTrigger = ABasicTriggerExtTest<UInputTriggerHoldAndRelease>(this, DataPtr);
	check(HARTrigger && DataPtr);
	UControllablePlayer& Data = *DataPtr;

	HARTrigger->HoldTimeThreshold = FrameTime * HoldFrames;

	// -------------------------------------------------------
	// Test 1 - Release before threshold -> Canceled (never triggers)
	// -------------------------------------------------------

	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	// Release before threshold
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersCanceled(Data, TestAction));

	// No further events
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyDoesNotTrigger(Data, TestAction));

	// -------------------------------------------------------
	// Test 2 - Hold to threshold then release -> Triggered
	// -------------------------------------------------------
	InputIsTicked(Data, FrameTime); // clear state

	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	for (int32 Frame = 1; Frame < HoldFrames - 1; ++Frame)
	{
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersOngoing(Data, TestAction));
	}

	// Hold one more frame (reaches threshold)
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(HoldingKeyTriggersOngoing(Data, TestAction)); // still ongoing (not released yet)

	// Release on the threshold frame triggers
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersAction(Data, TestAction));

	// Completed on next tick
	WHEN(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	WHEN(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyDoesNotTrigger(Data, TestAction));

	// -------------------------------------------------------
	// Test 3 - Hold past threshold then release -> also Triggered
	// -------------------------------------------------------
	InputIsTicked(Data, FrameTime); // clear state

	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	// Hold well past threshold
	for (int32 Frame = 1; Frame < HoldFrames + 10; ++Frame)
	{
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersOngoing(Data, TestAction));
	}

	// Release past threshold -> still triggers
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersAction(Data, TestAction));

	return true;
}

// ============================================================
// Tap trigger (integration - replaces the disabled unit test)
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputIntegrationTriggerTapTest, "Input.Integration.Triggers.Tap", BasicTriggerExtTestFlags)

bool FInputIntegrationTriggerTapTest::RunTest(const FString& Parameters)
{
	const float FrameTime = 1.f / 60.f;
	// Use a 10-frame tap window (consistent with the disabled unit test)
	const int32 TapWindowFrames = 10;
	const float TapThreshold = FrameTime * TapWindowFrames;

	UControllablePlayer* DataPtr = nullptr;
	UInputTriggerTap* TapTrigger = ABasicTriggerExtTest<UInputTriggerTap>(this, DataPtr);
	check(TapTrigger && DataPtr);
	UControllablePlayer& Data = *DataPtr;

	TapTrigger->TapReleaseTimeThreshold = TapThreshold;

	// -------------------------------------------------------
	// Test 1 - Quick release (frame 1, well within window) -> Triggered
	// -------------------------------------------------------
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersAction(Data, TestAction));

	WHEN(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	InputIsTicked(Data, FrameTime); // clear state

	// -------------------------------------------------------
	// Test 2 - Release on the last valid frame (frame TapWindowFrames-1, just before threshold)
	//          HeldDuration < TapThreshold on release -> Triggered
	// -------------------------------------------------------
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	for (int32 Frame = 1; Frame < TapWindowFrames - 1; ++Frame)
	{
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersOngoing(Data, TestAction));
	}

	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(ReleasingKeyTriggersAction(Data, TestAction));

	InputIsTicked(Data, FrameTime); // clear state

	// -------------------------------------------------------
	// Test 3 - Hold past the tap window -> tap Triggered event never fires.
	//          While held: HeldDuration < TapThreshold -> Ongoing
	//          Once HeldDuration >= TapThreshold: trigger returns None, action fires Canceled (window missed)
	//          Releasing after the window: no Triggered event (tap was not registered)
	// -------------------------------------------------------
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	THEN(PressingKeyTriggersStarted(Data, TestAction));

	// Hold through the tap window: Ongoing on each frame up to (not including) the threshold tick.
	// At tick TapWindowFrames, HeldDuration >= TapThreshold fires None (window expired).
	// Stop one tick early so the final loop tick is still Ongoing.
	for (int32 Frame = 1; Frame < TapWindowFrames - 1; ++Frame)
	{
		WHEN(InputIsTicked(Data, FrameTime));
		THEN(HoldingKeyTriggersOngoing(Data, TestAction));
		AND(TestFalse(TEXT("Tap: Triggered should not fire while holding within window"), FInputTestHelper::TestTriggered(Data, TestAction)));
	}

	// One more tick: HeldDuration now >= TapThreshold, trigger returns None -> Canceled fires (window expired)
	WHEN(InputIsTicked(Data, FrameTime));
	TestFalse(TEXT("Tap: Triggered must not fire after window expires"), FInputTestHelper::TestTriggered(Data, TestAction));

	// Releasing after the tap window: Triggered still does not fire
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data, FrameTime));
	TestFalse(TEXT("Tap: Triggered must not fire on release after window"), FInputTestHelper::TestTriggered(Data, TestAction));

	return true;
}

// ============================================================
// ChordBlocker trigger
// ============================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerChordBlockerTest, "Input.Triggers.Chords.Blocker", BasicTriggerExtTestFlags)

bool FInputTriggerChordBlockerTest::RunTest(const FString& Parameters)
{
	// Setup:
	//   MainAction  mapped to TestKey  with a ChordBlocker referencing ChordingAction.
	//   ChordingAction mapped to TestKey2 (no trigger - fires while held).
	//
	// Expected behavior:
	//   TestKey alone           -> MainAction fires (ChordingAction inactive, blocker not blocking)
	//   TestKey2 alone          -> ChordingAction fires, MainAction unaffected (not pressed)
	//   TestKey + TestKey2      -> ChordingAction fires, MainAction is BLOCKED
	//   TestKey2 released       -> MainAction fires again (blocker no longer blocking)

	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));
	TestTrue(TEXT("Controllable Player is valid"), Data.IsValid());

	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputAction(Data, TestAction, EInputActionValueType::Boolean));   // MainAction
	AND(AnInputAction(Data, TestAction2, EInputActionValueType::Boolean));  // ChordingAction

	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction2, TestKey2));

	// Add a Down (Explicit) trigger so the action is gated on key actuation.
	// Without an Explicit trigger, a Blocker-only action fires Triggered unconditionally
	// (bFoundExplicit=false, bAllImplicitsTriggered=true -> bTriggered=true always when not blocked).
	AND(ATriggerIsAppliedToAnAction(Data, NewObject<UInputTriggerDown>(), TestAction));

	// Apply ChordBlocker(ChordingAction) to MainAction (added after Down so its pointer stays valid)
	UInputTriggerChordBlocker* Blocker =
	AND(Cast<UInputTriggerChordBlocker>(ATriggerIsAppliedToAnAction(Data, NewObject<UInputTriggerChordBlocker>(), TestAction)));
	TestNotNull(TEXT("ChordBlocker created"), Blocker);
	Blocker->ChordAction = FInputTestHelper::FindAction(Data, TestAction2);
	TestNotNull(TEXT("ChordAction assigned to blocker"), Blocker->ChordAction.Get());

	// Test 1 - TestKey alone: MainAction fires, ChordingAction not active so blocker does not block
	WHEN(AKeyIsActuated(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(PressingKeyTriggersAction(Data, TestAction));
	AND(FInputTestHelper::TestNoTrigger(Data, TestAction2)); // ChordingAction not pressed

	// Test 2 - Press TestKey2 while TestKey is still held:
	//          ChordingAction activates -> blocker becomes active -> MainAction blocked
	WHEN(AKeyIsActuated(Data, TestKey2));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction2));       // ChordingAction fires
	AND(FInputTestHelper::TestNoTrigger(Data, TestAction));  // MainAction is blocked

	// Test 3 - Hold both keys: MainAction continues to be blocked
	WHEN(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction2));
	AND(FInputTestHelper::TestNoTrigger(Data, TestAction));

	// Test 4 - Release TestKey2: ChordingAction stops -> blocker deactivates -> MainAction fires again
	WHEN(AKeyIsReleased(Data, TestKey2));
	AND(InputIsTicked(Data));
	THEN(HoldingKeyTriggersAction(Data, TestAction));        // MainAction unblocked

	// Test 5 - Release TestKey: MainAction completes
	WHEN(AKeyIsReleased(Data, TestKey));
	AND(InputIsTicked(Data));
	THEN(ReleasingKeyTriggersCompleted(Data, TestAction));

	return true;
}
