// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputTestFramework.h"
#include "Misc/AutomationTest.h"

// Tests focused on individual triggers



constexpr auto BasicTriggerTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter;	// TODO: Run as Smoke/Client? No world on RunSmokeTests startup...
constexpr auto DisabledBasicTriggerTestFlags = BasicTriggerTestFlags | EAutomationTestFlags::Disabled;

// Dumping ground for local trigger tests
static UInputTrigger* TestTrigger = nullptr;
static ETriggerState LastTestTriggerState = ETriggerState::None;

// This will be cleared out by GC as soon as it ticks
template<typename T>
T* ATrigger()
{
	return Cast<T>(TestTrigger = NewObject<T>());
}

void TriggerGetsValue(FInputActionValue Value, float DeltaTime = 0.f)
{
	LastTestTriggerState = ETriggerState::None;

	if (TestTrigger)
	{
		// TODO: Provide an EnhancedPlayerInput
		LastTestTriggerState = TestTrigger->UpdateState(nullptr, Value, DeltaTime);
		TestTrigger->LastValue = Value;
	}
}

// Must declare one of these around a subtest to use TriggerStateIs
#define TRIGGER_SUBTEST(DESC) \
	for(FString ScopedSubTestDescription = TEXT(DESC);ScopedSubTestDescription.Len();ScopedSubTestDescription = "")	// Bodge to create a scoped test description. Usage: TRIGGER_SUBTEST("My Test Description") { TestCode... TriggerStateIs(ETriggerState::Triggered); }

// Forced to true to stop multiple errors from the THEN() TestTrueExpr wrapper
#define TriggerStateIs(STATE) \
	(TestEqual(ScopedSubTestDescription, *UEnum::GetValueAsString(TEXT("EnhancedInput.ETriggerState"), LastTestTriggerState), *UEnum::GetValueAsString(TEXT("EnhancedInput.ETriggerState"), STATE)) || true)


// ******************************
// Delegate firing (notification) tests for device (FKey) based triggers
// ******************************

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerPressedTest, "Input.Triggers.Pressed", BasicTriggerTestFlags)

bool FInputTriggerPressedTest::RunTest(const FString& Parameters)
{
	TRIGGER_SUBTEST("1 - Instant trigger on press")
	{
		GIVEN(ATrigger<UInputTriggerPressed>());
		WHEN(TriggerGetsValue(true));
		THEN(TriggerStateIs(ETriggerState::Triggered));
	}

	TRIGGER_SUBTEST("2 - Trigger stops on release")
	{
		GIVEN(ATrigger<UInputTriggerPressed>());
		WHEN(TriggerGetsValue(true));
		AND(TriggerGetsValue(false));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	TRIGGER_SUBTEST("3 - Trigger stops on hold")
	{
		GIVEN(ATrigger<UInputTriggerPressed>());
		WHEN(TriggerGetsValue(true));
		AND(TriggerGetsValue(true));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerDownTest, "Input.Triggers.Down", BasicTriggerTestFlags)

bool FInputTriggerDownTest::RunTest(const FString& Parameters)
{
	TRIGGER_SUBTEST("Instant trigger on press")
	{
		GIVEN(ATrigger<UInputTriggerDown>());
		WHEN(TriggerGetsValue(true));
		THEN(TriggerStateIs(ETriggerState::Triggered));
	}

	TRIGGER_SUBTEST("Trigger stops on release")
	{
		GIVEN(ATrigger<UInputTriggerDown>());
		WHEN(TriggerGetsValue(true));
		AND(TriggerGetsValue(false));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	TRIGGER_SUBTEST("Trigger retained on hold")
	{
		GIVEN(ATrigger<UInputTriggerDown>());
		WHEN(TriggerGetsValue(true));
		AND(TriggerGetsValue(true));
		THEN(TriggerStateIs(ETriggerState::Triggered));

		// Then lost on release 
		WHEN(TriggerGetsValue(false));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerReleasedTest, "Input.Triggers.Released", BasicTriggerTestFlags)

bool FInputTriggerReleasedTest::RunTest(const FString& Parameters)
{
	TRIGGER_SUBTEST("No trigger on press")
	{
		GIVEN(ATrigger<UInputTriggerReleased>());
		WHEN(TriggerGetsValue(true));
		THEN(TriggerStateIs(ETriggerState::Ongoing));
	}

	TRIGGER_SUBTEST("No trigger on hold")
	{
		WHEN(TriggerGetsValue(true));
		THEN(TriggerStateIs(ETriggerState::Ongoing));
	}

	TRIGGER_SUBTEST("Trigger on release")
	{
		WHEN(TriggerGetsValue(false));
		THEN(TriggerStateIs(ETriggerState::Triggered));
		// But only once
		WHEN(TriggerGetsValue(false));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	TRIGGER_SUBTEST("No trigger for no input")
	{
		GIVEN(ATrigger<UInputTriggerReleased>());
		WHEN(TriggerGetsValue(false));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	return true;
}

// TODO: Provide a player input pointer to run the Timed Tests
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerHoldTest, "Input.Triggers.Hold", DisabledBasicTriggerTestFlags)

bool FInputTriggerHoldTest::RunTest(const FString& Parameters)
{
	const float FrameTime = 1.f / 60.f;
	const int HoldFrames = 30;	// Half second hold

	TRIGGER_SUBTEST("Release before threshold frame cancels")
	{
		GIVEN(ATrigger<UInputTriggerHold>())->HoldTimeThreshold = FrameTime * HoldFrames;
		WHEN(TriggerGetsValue(true, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Ongoing));
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));
	}


	TRIGGER_SUBTEST("Holding to threshold fires trigger")
	{
		GIVEN(ATrigger<UInputTriggerHold>())->HoldTimeThreshold = FrameTime * HoldFrames;
		WHEN(TriggerGetsValue(true, FrameTime));
		for (int HoldFrame = 1; HoldFrame < HoldFrames - 1; ++HoldFrame)
		{
			AND(TriggerGetsValue(true, FrameTime));
			THEN(TriggerStateIs(ETriggerState::Ongoing));
		}
		WHEN(TriggerGetsValue(true, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Triggered));

		// Continues to fire
		WHEN(TriggerGetsValue(true, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Triggered));

		// Release stops fire
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	TRIGGER_SUBTEST("One shot trigger")
	{
		UInputTriggerHold* Trigger =
		GIVEN(ATrigger<UInputTriggerHold>());
		Trigger->HoldTimeThreshold = FrameTime * HoldFrames;
		Trigger->bIsOneShot = true;
		for (int HoldFrame = 0; HoldFrame < HoldFrames - 1; ++HoldFrame)
		{
			AND(TriggerGetsValue(true, FrameTime));
		}
		WHEN(TriggerGetsValue(true, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Triggered));

		// Stops firing
		WHEN(TriggerGetsValue(true, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	return true;
}

// TODO: Provide a player input pointer to run the Timed Tests
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerHoldAndReleaseTest, "Input.Triggers.HoldAndRelease", DisabledBasicTriggerTestFlags)

bool FInputTriggerHoldAndReleaseTest::RunTest(const FString& Parameters)
{
	const float FrameTime = 1.f / 60.f;
	const int HoldFrames = 30;	// Half second hold

	TRIGGER_SUBTEST("Release before threshold frame does not trigger")
	{
		GIVEN(ATrigger<UInputTriggerHoldAndRelease>())->HoldTimeThreshold = FrameTime * HoldFrames;
		WHEN(TriggerGetsValue(true, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Ongoing));
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	TRIGGER_SUBTEST("Holding to threshold frame triggers")
	{
		// Hold to frame 29, release frame 30

		GIVEN(ATrigger<UInputTriggerHoldAndRelease>())->HoldTimeThreshold = FrameTime * HoldFrames;
		for (int HoldFrame = 0; HoldFrame < HoldFrames - 1; ++HoldFrame)
		{
			WHEN(TriggerGetsValue(true, FrameTime));
			THEN(TriggerStateIs(ETriggerState::Ongoing));
		}
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Triggered));

		// Ticking a further frame resets the trigger
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));
	}


	TRIGGER_SUBTEST("Holding beyond threshold frame triggers")
	{
		// Hold to frame 30, release frame 31.

		GIVEN(ATrigger<UInputTriggerHoldAndRelease>())->HoldTimeThreshold = FrameTime * HoldFrames;
		for (int HoldFrame = 0; HoldFrame < HoldFrames; ++HoldFrame)
		{
			WHEN(TriggerGetsValue(true, FrameTime));
			THEN(TriggerStateIs(ETriggerState::Ongoing));
		}
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Triggered));

		// Ticking a further frame resets the trigger
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	return true;
}

// TODO: Provide a player input pointer to run the Timed Tests
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerTapTest, "Input.Triggers.Tap", DisabledBasicTriggerTestFlags)

bool FInputTriggerTapTest::RunTest(const FString& Parameters)
{
	const float FrameTime = 1.f / 60.f;
	const int MaxTapFrames = 10;

	TRIGGER_SUBTEST("Releasing on first frame fires trigger")
	{
		GIVEN(ATrigger<UInputTriggerTap>()->TapReleaseTimeThreshold = FrameTime * MaxTapFrames);

		// Pressing
		WHEN(TriggerGetsValue(true, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Ongoing));

		// Releasing immediately
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Triggered));

		// Ticking a further frame resets the trigger
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	TRIGGER_SUBTEST("Releasing on final frame fires trigger")
	{
		// Hold to frame 9, release on frame 10 = trigger.

		GIVEN(ATrigger<UInputTriggerTap>()->TapReleaseTimeThreshold = FrameTime * MaxTapFrames);
		// Holding until last trigger frame
		for (int HoldFrame = 0; HoldFrame < MaxTapFrames - 1; ++HoldFrame)
		{
			WHEN(TriggerGetsValue(true, FrameTime));
			THEN(TriggerStateIs(ETriggerState::Ongoing));
		}
		// Releasing
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Triggered));

		// Ticking a further frame resets the trigger
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	TRIGGER_SUBTEST("Holding beyond final frame cancels trigger")
	{
		//Hold to frame 9, canceled on frame 10 as still actuated.

		GIVEN(ATrigger<UInputTriggerTap>()->TapReleaseTimeThreshold = FrameTime * MaxTapFrames);
		// Holding until last trigger frame
		for (int HoldFrame = 0; HoldFrame < MaxTapFrames - 1; ++HoldFrame)
		{
			WHEN(TriggerGetsValue(true, FrameTime));
			THEN(TriggerStateIs(ETriggerState::Ongoing));
		}
		// Holding past threshold
		WHEN(TriggerGetsValue(true, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));

		// Doesn't transition back to Ongoing
		WHEN(TriggerGetsValue(true, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));

		// Releasing doesn't trigger
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	TRIGGER_SUBTEST("Releasing immediately after final frame doesn't tick")
	{
		//Hold to frame 10, release frame 11.

		GIVEN(ATrigger<UInputTriggerTap>()->TapReleaseTimeThreshold = FrameTime * MaxTapFrames);
		// Holding until last trigger frame
		for (int HoldFrame = 0; HoldFrame < MaxTapFrames - 1 ; ++HoldFrame)
		{
			WHEN(TriggerGetsValue(true, FrameTime));
			THEN(TriggerStateIs(ETriggerState::Ongoing));
		}

		// Holding past threshold
		WHEN(TriggerGetsValue(true, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));

		// Releasing doesn't trigger
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::None));
	}

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerChordsMultiContextTest, "Input.Triggers.Chords.MultiContext", BasicTriggerTestFlags)

bool FInputTriggerChordsMultiContextTest::RunTest(const FString& Parameters)
{
	// Test chords work when the chord action is in a higher priority context
	FKey ChordKey = TestKey2;
	FName BaseAction = TEXT("BaseAction");				// Base action
	FName ChordedAction = TEXT("ChordedAction");			// Chord triggered action
	FName ChordingAction = TEXT("ChordingAction");		// Chording action driving special case e.g. ShiftDown/AcrobaticModifier

	UWorld* World =
	GIVEN(AnEmptyWorld());

	// Initialise
	UControllablePlayer& Data =
	AND(AControllablePlayer(World));

	FName BaseContext = TEXT("BaseContext"), ChordContext = TEXT("ChordContext");
	AND(AnInputContextIsAppliedToAPlayer(Data, BaseContext, 1));
	AND(AnInputContextIsAppliedToAPlayer(Data, ChordContext, 100));

	// Set up action
	AND(AnInputAction(Data, BaseAction, EInputActionValueType::Axis1D));
	AND(AnInputAction(Data, ChordedAction, EInputActionValueType::Axis1D));

	// Set up the chording action (modifier key action)
	AND(UInputAction * ChordingActionPtr = AnInputAction(Data, ChordingAction, EInputActionValueType::Boolean));

	// Bind the chording modifier in the high priority context
	AND(AnActionIsMappedToAKey(Data, ChordContext, ChordingAction, ChordKey));

	// Bind the action to the same key in both contexts
	AND(AnActionIsMappedToAKey(Data, BaseContext, BaseAction, TestAxis));
	AND(AnActionIsMappedToAKey(Data, ChordContext, ChordedAction, TestAxis));

	// But the chorded version inverts the result
	AND(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierNegate>(), ChordContext, ChordedAction, TestAxis));

	// Apply a chord action trigger to the chorded mapping
	UInputTriggerChordAction* Trigger = NewObject<UInputTriggerChordAction>();
	Trigger->ChordAction = ChordingActionPtr;
	AND(ATriggerIsAppliedToAnActionMapping(Data, Trigger, ChordContext, ChordedAction, TestAxis));


	TRIGGER_SUBTEST("With chord key pressed neither main action triggers, but chording action does")
	{
		WHEN(AKeyIsActuated(Data, ChordKey));
		AND(InputIsTicked(Data));
		THEN(PressingKeyDoesNotTrigger(Data, BaseAction));
		ANDALSO(PressingKeyDoesNotTrigger(Data, ChordedAction));
		ANDALSO(PressingKeyTriggersAction(Data, ChordingAction));
	}

	const float AxisValue = 0.5f;

	TRIGGER_SUBTEST("Switching to test key the action supplies the unmodified value")
	{
		WHEN(AKeyIsReleased(Data, ChordKey));
		AND(AKeyIsActuated(Data, TestAxis, AxisValue));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, BaseAction));
		ANDALSO(ReleasingKeyTriggersCompleted(Data, ChordingAction));
		AND(TestEqual(TEXT("BaseAction"), FInputTestHelper::GetTriggered<float>(Data, BaseAction), AxisValue));
	}

	TRIGGER_SUBTEST("Depressing chord key triggers chorded action modified value")
	{
		WHEN(AKeyIsActuated(Data, ChordKey));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, ChordedAction));
		ANDALSO(ReleasingKeyTriggersCompleted(Data, BaseAction));
		AND(TestEqual(TEXT("BaseAction"), FInputTestHelper::GetTriggered<float>(Data, ChordedAction), -AxisValue));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerChordsSingleContextTest, "Input.Triggers.Chords.SingleContext", BasicTriggerTestFlags)

bool FInputTriggerChordsSingleContextTest::RunTest(const FString& Parameters)
{
	// Test chords work when all base, chorded, and chording actions are in the same context (they are processed in the correct order)
	FKey ChordKey = TestKey2;
	FName BaseAction = TEXT("BaseAction");				// Base action
	FName ChordedAction = TEXT("ChordedAction");			// Chord triggered action
	FName ChordingAction = TEXT("ChordingAction");		// Chording action driving special case e.g. ShiftDown/AcrobaticModifier

	UWorld* World =
		GIVEN(AnEmptyWorld());

	// Initialise
	UControllablePlayer& Data =
		AND(AControllablePlayer(World));

	FName SingleContext = TEXT("Context");
	AND(AnInputContextIsAppliedToAPlayer(Data, SingleContext, 1));

	// Set up action
	AND(AnInputAction(Data, BaseAction, EInputActionValueType::Axis1D));
	AND(AnInputAction(Data, ChordedAction, EInputActionValueType::Axis1D));

	// Set up the chording action (modifier key action)
	AND(UInputAction * ChordingActionPtr = AnInputAction(Data, ChordingAction, EInputActionValueType::Boolean));

	// Bind the chording modifier
	AND(AnActionIsMappedToAKey(Data, SingleContext, ChordingAction, ChordKey));

	// Bind the actions to the same key
	AND(AnActionIsMappedToAKey(Data, SingleContext, BaseAction, TestAxis));
	AND(AnActionIsMappedToAKey(Data, SingleContext, ChordedAction, TestAxis));

	// But the chorded version inverts the result
	AND(AModifierIsAppliedToAnActionMapping(Data, NewObject<UInputModifierNegate>(), SingleContext, ChordedAction, TestAxis));

	// Apply a chord action trigger to the chorded mapping
	UInputTriggerChordAction* Trigger = NewObject<UInputTriggerChordAction>();
	Trigger->ChordAction = ChordingActionPtr;
	AND(ATriggerIsAppliedToAnActionMapping(Data, Trigger, SingleContext, ChordedAction, TestAxis));


	TRIGGER_SUBTEST("With chord key pressed neither main action triggers, but chording action does")
	{
		WHEN(AKeyIsActuated(Data, ChordKey));
		AND(InputIsTicked(Data));
		THEN(PressingKeyDoesNotTrigger(Data, BaseAction));
		ANDALSO(PressingKeyDoesNotTrigger(Data, ChordedAction));
		ANDALSO(PressingKeyTriggersAction(Data, ChordingAction));
	}

	const float AxisValue = 0.5f;

	TRIGGER_SUBTEST("Switching to test key the action supplies the unmodified value")
	{
		WHEN(AKeyIsReleased(Data, ChordKey));
		AND(AKeyIsActuated(Data, TestAxis, AxisValue));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, BaseAction));
		ANDALSO(ReleasingKeyTriggersCompleted(Data, ChordingAction));
		AND(TestEqual(TEXT("BaseAction"), FInputTestHelper::GetTriggered<float>(Data, BaseAction), AxisValue));
	}

	TRIGGER_SUBTEST("Depressing chord key triggers chorded action modified value")
	{
		WHEN(AKeyIsActuated(Data, ChordKey));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, ChordedAction));
		ANDALSO(ReleasingKeyTriggersCompleted(Data, BaseAction));
		AND(TestEqual(TEXT("BaseAction"), FInputTestHelper::GetTriggered<float>(Data, ChordedAction), -AxisValue));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerChordsPressedTriggersTest, "Input.Triggers.Chords.WithPressedTriggers", BasicTriggerTestFlags)

bool FInputTriggerChordsPressedTriggersTest::RunTest(const FString& Parameters)
{
	// Test chord behavior with pressed triggers
	// Expected: Main key trigger state should be retained by both base and chorded action, across any chord key state transitions.
	// Pressing or releasing the chord key shouldn't cause any action to trigger by itself (Note: triggering would continue to occur for a down trigger).

	FKey ChordKey = TestKey2;
	FName BaseAction = TEXT("BaseAction");				// Base action
	FName ChordedAction = TEXT("ChordedAction");			// Chord triggered action
	FName ChordingAction = TEXT("ChordingAction");		// Chording action driving special case e.g. ShiftDown/AcrobaticModifier

	UWorld* World =
		GIVEN(AnEmptyWorld());

	// Initialise
	UControllablePlayer& Data =
		AND(AControllablePlayer(World));

	FName BaseContext = TEXT("BaseContext"), ChordContext = TEXT("ChordContext");
	AND(AnInputContextIsAppliedToAPlayer(Data, BaseContext, 1));
	AND(AnInputContextIsAppliedToAPlayer(Data, ChordContext, 100));

	// Set up action
	AND(AnInputAction(Data, BaseAction, EInputActionValueType::Boolean));
	AND(AnInputAction(Data, ChordedAction, EInputActionValueType::Boolean));

	// Set up the chording action (modifier key action)
	AND(UInputAction * ChordingActionPtr = AnInputAction(Data, ChordingAction, EInputActionValueType::Boolean));

	// Bind the chording modifier in the high priority context
	AND(AnActionIsMappedToAKey(Data, ChordContext, ChordingAction, ChordKey));

	// Bind the actions to the same key in both contexts
	AND(AnActionIsMappedToAKey(Data, BaseContext, BaseAction, TestKey));
	AND(AnActionIsMappedToAKey(Data, ChordContext, ChordedAction, TestKey));

	// Apply pressed triggers to both actions
	AND(ATriggerIsAppliedToAnAction(Data, NewObject<UInputTriggerPressed>(), BaseAction));
	AND(ATriggerIsAppliedToAnAction(Data, NewObject<UInputTriggerPressed>(), ChordedAction));

	// Apply a chord action trigger to the chorded mapping
	UInputTriggerChordAction* Trigger = NewObject<UInputTriggerChordAction>();
	Trigger->ChordAction = ChordingActionPtr;
	AND(ATriggerIsAppliedToAnAction(Data, Trigger, ChordedAction));

	TRIGGER_SUBTEST("Pressing key triggers base action")
	{
		WHEN(AKeyIsActuated(Data, TestKey));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, BaseAction));
		ANDALSO(PressingKeyDoesNotTrigger(Data, ChordedAction));
	}

	TRIGGER_SUBTEST("Pressing chord key does not trigger chorded action, but stops base")
	{
		WHEN(AKeyIsActuated(Data, ChordKey));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersStarted(Data, ChordedAction));	// Begins tracking trigger...	// TODO: Started -> Permanent Ongoing. The implicit chord action is true, but explict Pressed blocks. Make chord action a 4th type? ImplicitBlocker?
		THEN(!PressingKeyTriggersAction(Data, ChordedAction));	// but does not fire
		ANDALSO(PressingKeyTriggersCompleted(Data, BaseAction));
	}

	// Release main key
	AKeyIsReleased(Data, TestKey);
	InputIsTicked(Data);

	TRIGGER_SUBTEST("Pressing key again triggers chorded action only")
	{
		WHEN(AKeyIsActuated(Data, TestKey));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, ChordedAction));
		ANDALSO(PressingKeyDoesNotTrigger(Data, BaseAction));
	}

	TRIGGER_SUBTEST("Releasing chord key stops chorded action but does not trigger base action")
	{
		WHEN(AKeyIsReleased(Data, ChordKey));
		AND(InputIsTicked(Data));
		THEN(ReleasingKeyTriggersCompleted(Data, ChordedAction));
		ANDALSO(ReleasingKeyDoesNotTrigger(Data, BaseAction));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerChordsMultiMappingTest, "Input.Triggers.Chords.MultiMapping", BasicTriggerTestFlags)

bool FInputTriggerChordsMultiMappingTest::RunTest(const FString& Parameters)
{
	// Regression test for UE-305754: when an Input Action has multiple mappings in the
	// same IMC, a sibling mapping without triggers must not bleed its state into a
	// mapping that carries an IMC-level chord trigger. Before the fix, the sibling's
	// empty TriggerStateTracker could win the per-action FMath::Max merge (state tied
	// at None, but bEvaluatedTriggers not compared), and its bMappingTriggerApplied=false
	// was unconditionally written to the action-level tracker after the merge. That
	// dropped the chord block and let the action-level Pressed trigger fire on the
	// chorded key alone.

	FName BaseAction = TEXT("BaseAction");
	FName ChordingAction = TEXT("ChordingAction");

	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));

	FName Context = TEXT("Context");
	AND(AnInputContextIsAppliedToAPlayer(Data, Context, 1));

	AND(AnInputAction(Data, BaseAction, EInputActionValueType::Boolean));
	AND(UInputAction* ChordingActionPtr = AnInputAction(Data, ChordingAction, EInputActionValueType::Boolean));

	// Chord key
	AND(AnActionIsMappedToAKey(Data, Context, ChordingAction, TestKey2));

	// Two mappings for BaseAction: TestKey is chord-gated, TestKey3 is the "bleed" sibling with no IMC triggers.
	// Order matters for the repro — the unchorded sibling must be processed after the chorded mapping.
	AND(AnActionIsMappedToAKey(Data, Context, BaseAction, TestKey));
	AND(AnActionIsMappedToAKey(Data, Context, BaseAction, TestKey3));

	// Action-level Pressed trigger on BaseAction (the thing that incorrectly fires with the bug)
	AND(ATriggerIsAppliedToAnAction(Data, NewObject<UInputTriggerPressed>(), BaseAction));

	// IMC-level chord trigger, applied only to the TestKey mapping
	UInputTriggerChordAction* ChordTrigger = NewObject<UInputTriggerChordAction>();
	ChordTrigger->ChordAction = ChordingActionPtr;
	AND(ATriggerIsAppliedToAnActionMapping(Data, ChordTrigger, Context, BaseAction, TestKey));

	TRIGGER_SUBTEST("Pressing chorded key alone does not fire the action (sibling mapping must not drop the chord block)")
	{
		WHEN(AKeyIsActuated(Data, TestKey));
		AND(InputIsTicked(Data));
		THEN(!PressingKeyTriggersAction(Data, BaseAction));
	}

	AKeyIsReleased(Data, TestKey);
	InputIsTicked(Data);

	TRIGGER_SUBTEST("Chord key + chorded key fires the action")
	{
		WHEN(AKeyIsActuated(Data, TestKey2));
		AND(AKeyIsActuated(Data, TestKey));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, BaseAction));
	}

	AKeyIsReleased(Data, TestKey);
	AKeyIsReleased(Data, TestKey2);
	InputIsTicked(Data);

	TRIGGER_SUBTEST("Pressing the unchorded sibling key still triggers the action")
	{
		WHEN(AKeyIsActuated(Data, TestKey3));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersAction(Data, BaseAction));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerTouchBooleanReleasedTest, "Input.Triggers.Touch.BooleanReleased", BasicTriggerTestFlags)

bool FInputTriggerTouchBooleanReleasedTest::RunTest(const FString& Parameters)
{
	// Test that Boolean actions on touch keys correctly fire Released triggers, even when RawValue
	// contains screen coordinates (which have magnitude >> actuation threshold).
	// This verifies the fix in EnhancedPlayerInput::PrepareInputDelegatesForEvaluation that
	// substitutes bDown for screen coordinates on Boolean touch actions.

	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));

	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputAction(Data, TestAction, EInputActionValueType::Boolean));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, EKeys::TouchKeys[0]));
	AND(ATriggerIsAppliedToAnAction(Data, NewObject<UInputTriggerReleased>(), TestAction));

	TRIGGER_SUBTEST("1 - Press touch, should start Released trigger (Ongoing)")
	{
		WHEN(AKeyIsActuated(Data, EKeys::TouchKeys[0]));
		// Simulate touch screen coordinates (critical for reproducing the bug)
		FKeyState* TouchKeyState = Data.PlayerInput->GetKeyState(EKeys::TouchKeys[0]);
		if (TouchKeyState)
		{
			TouchKeyState->RawValueAccumulator = FVector(500.0f, 300.0f, 0.0f);
		}
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersStarted(Data, TestAction));
		ANDALSO(!PressingKeyTriggersAction(Data, TestAction));
	}

	TRIGGER_SUBTEST("2 - Hold touch, should remain Ongoing")
	{
		WHEN(AKeyIsActuated(Data, EKeys::TouchKeys[0]));
		// Simulate finger movement (different screen coordinates)
		FKeyState* TouchKeyState = Data.PlayerInput->GetKeyState(EKeys::TouchKeys[0]);
		if (TouchKeyState)
		{
			TouchKeyState->RawValueAccumulator = FVector(502.0f, 301.0f, 0.0f);
		}
		AND(InputIsTicked(Data));
		THEN(HoldingKeyTriggersOngoing(Data, TestAction));
	}

	TRIGGER_SUBTEST("3 - Release touch, Released trigger should fire (MAIN TEST)")
	{
		WHEN(AKeyIsReleased(Data, EKeys::TouchKeys[0]));
		// Simulate touch screen coordinates persisting on release (like InputTouch does)
		// This is the critical reproduction of the real-world bug scenario
		FKeyState* TouchKeyState = Data.PlayerInput->GetKeyState(EKeys::TouchKeys[0]);
		if (TouchKeyState)
		{
			TouchKeyState->RawValueAccumulator = FVector(502.0f, 301.0f, 0.0f);
		}
		AND(InputIsTicked(Data));
		// Without the fix, IsActuated(502, 301, 0) = true (magnitude 502 >> 0.5 threshold)
		// so Released trigger never fires. With the fix, bDown=0, IsActuated(0)=false, fires correctly.
		THEN(ReleasingKeyTriggersAction(Data, TestAction));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerKeyboardBooleanReleasedTest, "Input.Triggers.Keyboard.BooleanReleased", BasicTriggerTestFlags)

bool FInputTriggerKeyboardBooleanReleasedTest::RunTest(const FString& Parameters)
{
	// Regression test: verify that keyboard keys with Boolean + Released trigger still work
	// correctly. The touch coordinate fix (IsTouch() guard) must not interfere with normal keys.

	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));

	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputAction(Data, TestAction, EInputActionValueType::Boolean));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, EKeys::SpaceBar));
	AND(ATriggerIsAppliedToAnAction(Data, NewObject<UInputTriggerReleased>(), TestAction));

	TRIGGER_SUBTEST("1 - Press space, should start Released trigger (Ongoing)")
	{
		WHEN(AKeyIsActuated(Data, EKeys::SpaceBar));
		AND(InputIsTicked(Data));
		THEN(PressingKeyTriggersStarted(Data, TestAction));
	}

	TRIGGER_SUBTEST("2 - Hold space, should remain Ongoing")
	{
		WHEN(InputIsTicked(Data));
		THEN(HoldingKeyTriggersOngoing(Data, TestAction));
	}

	TRIGGER_SUBTEST("3 - Release space, Released trigger should fire")
	{
		WHEN(AKeyIsReleased(Data, EKeys::SpaceBar));
		AND(InputIsTicked(Data));
		THEN(ReleasingKeyTriggersAction(Data, TestAction));
	}

	return true;
}

// Regression tests for UE-226494: UInputTriggerRepeatedTap must remain in the Ongoing
// state between taps instead of collapsing to None and firing a spurious Canceled.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputTriggerRepeatedTapTest, "Input.Triggers.RepeatedTap", BasicTriggerTestFlags)

bool FInputTriggerRepeatedTapTest::RunTest(const FString& Parameters)
{
	const float FrameTime = 1.f / 60.f;
	const int32 TapWindowFrames = 10;

	TRIGGER_SUBTEST("bShouldAlwaysTick must be set so the pipeline keeps evaluating between taps")
	{
		UInputTriggerRepeatedTap* T = GIVEN(ATrigger<UInputTriggerRepeatedTap>());
		THEN(TestTrue(TEXT("RepeatedTap must opt into always-tick"), T->bShouldAlwaysTick));
	}

	TRIGGER_SUBTEST("Double tap: state stays Ongoing between taps and fires Triggered on the second press")
	{
		UInputTriggerRepeatedTap* T = GIVEN(ATrigger<UInputTriggerRepeatedTap>());
		T->TapReleaseTimeThreshold = FrameTime * TapWindowFrames;
		T->NumberOfTapsWhichTriggerRepeat = 2;
		// RepeatDelay default (0.5s) is larger than this test's wall-clock span, so the
		// "within valid repeat time range" check stays true throughout.

		// First press: Ongoing (held, HeldDuration < threshold).
		WHEN(TriggerGetsValue(true, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Ongoing));

		// Release within the tap window: bHasSingleTap fires, counter increments to 1,
		// State is forced to Ongoing by the NumberOfTapsSinceLastTrigger > 0 clause.
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Ongoing));

		// Key still up: value=0. Without the always-tick fix, the pipeline skips
		// evaluation here and the action-level state collapses to None (firing Canceled).
		// At the trigger level the NumberOfTapsSinceLastTrigger > 0 guard keeps us Ongoing.
		WHEN(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Ongoing));

		// Second press within the repeat window: NumberOfTapsSinceLastTrigger hits the
		// repeat threshold and state flips to Triggered.
		WHEN(TriggerGetsValue(true, FrameTime));
		AND(TriggerGetsValue(false, FrameTime));
		THEN(TriggerStateIs(ETriggerState::Triggered));
	}

	return true;
}

// Integration regression for UE-226494: exercises the full UEnhancedPlayerInput pipeline,
// which is where the bug actually manifests. Without bShouldAlwaysTick on RepeatedTap,
// ProcessActionMappingEvent skips evaluation on idle frames, ActionsWithEventsThisTick
// loses the action, the post-tick loop defaults TriggerState to None, and the action
// fires Canceled between taps.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInputIntegrationTriggerRepeatedTapTest, "Input.Integration.Triggers.RepeatedTap", BasicTriggerTestFlags)

bool FInputIntegrationTriggerRepeatedTapTest::RunTest(const FString& Parameters)
{
	const float FrameTime = 1.f / 60.f;

	UWorld* World =
	GIVEN(AnEmptyWorld());

	UControllablePlayer& Data =
	AND(AControllablePlayer(World));

	AND(AnInputContextIsAppliedToAPlayer(Data, TestContext, 0));
	AND(AnInputAction(Data, TestAction, EInputActionValueType::Boolean));
	AND(AnActionIsMappedToAKey(Data, TestContext, TestAction, TestKey));

	UInputTriggerRepeatedTap* Trigger = Cast<UInputTriggerRepeatedTap>(
		ATriggerIsAppliedToAnActionMapping(Data, NewObject<UInputTriggerRepeatedTap>(), TestContext, TestAction, TestKey));
	TestNotNull(TEXT("RepeatedTap trigger applied"), Trigger);
	Trigger->TapReleaseTimeThreshold = FrameTime * 10;
	Trigger->NumberOfTapsWhichTriggerRepeat = 2;

	TRIGGER_SUBTEST("First press fires Started, not Canceled, not Triggered")
	{
		WHEN(AKeyIsActuated(Data, TestKey));
		AND(InputIsTicked(Data, FrameTime));
		THEN(PressingKeyTriggersStarted(Data, TestAction));
		ANDALSO(!PressingKeyTriggersCanceled(Data, TestAction));
		ANDALSO(!PressingKeyTriggersAction(Data, TestAction));
	}

	TRIGGER_SUBTEST("Release keeps the action Ongoing (not Canceled)")
	{
		WHEN(AKeyIsReleased(Data, TestKey));
		AND(InputIsTicked(Data, FrameTime));
		THEN(ReleasingKeyTriggersOngoing(Data, TestAction));
		ANDALSO(!ReleasingKeyTriggersCanceled(Data, TestAction));
	}

	TRIGGER_SUBTEST("Idle frames between taps keep Ongoing with no Canceled — UE-226494 regression")
	{
		for (int32 Frame = 0; Frame < 3; ++Frame)
		{
			WHEN(InputIsTicked(Data, FrameTime));
			THEN(HoldingKeyTriggersOngoing(Data, TestAction));
			ANDALSO(!HoldingKeyTriggersCanceled(Data, TestAction));
			ANDALSO(!HoldingKeyTriggersAction(Data, TestAction));
		}
	}

	TRIGGER_SUBTEST("Second press fires Triggered")
	{
		WHEN(AKeyIsActuated(Data, TestKey));
		AND(InputIsTicked(Data, FrameTime));
		AND(AKeyIsReleased(Data, TestKey));
		AND(InputIsTicked(Data, FrameTime));
		THEN(ReleasingKeyTriggersAction(Data, TestAction));
	}

	return true;
}

// TODO: Action level triggers (simple repeat of device level tests)
// TODO: Variable frame delta tests
// TODO: ActionEventData tests (timing, summed values, etc)
