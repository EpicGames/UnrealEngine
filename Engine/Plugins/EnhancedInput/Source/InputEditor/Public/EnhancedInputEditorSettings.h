// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Engine/DeveloperSettingsBackedByCVars.h"

#include "EnhancedInputEditorSettings.generated.h"

#define UE_API INPUTEDITOR_API

struct FDefaultContextSetting;

class UEnhancedPlayerInput;
class UEnhancedInputEditorSubsystem;

/** Settings for the Enhanced Input Editor Subsystem that are persistent between a project's users */
UCLASS(config = Input, defaultconfig, meta = (DisplayName = "Enhanced Input (Editor Only)"))
class UEnhancedInputEditorProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public: 
	UEnhancedInputEditorProjectSettings(const FObjectInitializer& Initializer);
	
	/** The default player input class that the Enhanced Input Editor subsystem will use. */
	UPROPERTY(config, EditAnywhere, NoClear, Category = Default)
	TSoftClassPtr<UEnhancedPlayerInput> DefaultEditorInputClass;

	/** Array of any input mapping contexts that you want to always be applied to the Enhanced Input Editor Subsystem. */
	UPROPERTY(config, EditAnywhere, Category = Default)
	TArray<FDefaultContextSetting> DefaultMappingContexts;
};

/**
 * A collection of useful individual user settings when using the EnhancedInputEditorSubsystem.
 */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, meta=(DisplayName="Enhanced Input Editor Settings"))
class UEnhancedInputEditorSettings : public UDeveloperSettingsBackedByCVars
{
	GENERATED_BODY()
public:

	UE_API UEnhancedInputEditorSettings();

	/**
	 * If true then the Enhanced Input Editor subsystem will log all input that is being processed by it (keypresses, analog values, etc)
	 * Note: This can produce A LOT of logs, so only use this if you are debugging something.
	 */
	UPROPERTY(config, EditAnywhere, Category = Logging)
	uint8 bLogAllInput : 1;
	
	/** If true, then the UEnhancedInputEditorSubsystem will be started when it is initalized */
	UPROPERTY(config, EditAnywhere, Category = Editor)
	uint8 bAutomaticallyStartConsumingInput : 1;

	/** A bitmask of what event pins are visible when you place an Input Action event node in blueprints.  */
	UPROPERTY(config, EditAnywhere, Category = Blueprints, meta = (Bitmask, BitmaskEnum = "/Script/EnhancedInput.ETriggerEvent"))
	uint8 VisibleEventPinsByDefault;
};

#undef UE_API
