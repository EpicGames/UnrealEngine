// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EnhancedInputSubsystemInterface.h"
#include "EditorSubsystem.h"
#include "Tickable.h"
#include "EnhancedInputEditorSubsystem.generated.h"

#define UE_API INPUTEDITOR_API

struct FInputKeyParams;

DECLARE_LOG_CATEGORY_EXTERN(LogEditorInput, Log, All);

class UInputComponent;
class UEnhancedPlayerInput;
class FEnhancedInputEditorProcessor;

/**
 * The Enhanced Input Editor Subsystem can be used to process input outside of PIE within the editor.
 * Calling StartConsumingInput will allow the input preprocessor to drive Input Action delegates
 * to be fired in the editor.
 *
 * This allows you to hook up Input Action delegates in Editor Utilities to make editor tools driven by
 * input.
 */
UCLASS(MinimalAPI)
class UEnhancedInputEditorSubsystem : public UEditorSubsystem, public IEnhancedInputSubsystemInterface, public FTickableGameObject
{
	GENERATED_BODY()

public:

	//~ Begin USubsystem interface
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	//~ End USubsystem interface
	
	//~ Begin FTickableGameObject interface
	UE_API virtual UWorld* GetTickableGameObjectWorld() const override;
	virtual bool IsTickableInEditor() const { return true; }
	UE_API virtual ETickableTickType GetTickableTickType() const override;
	UE_API virtual bool IsTickable() const override;
	UE_API virtual void Tick(float DeltaTime) override;
	TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(UEnhancedInputEditorSubsystem, STATGROUP_Tickables); }
	//~ End FTickableGameObject interface

	//~ UObject interface
	UE_API virtual UWorld* GetWorld() const override;
	//~ End UObject interface
	
	//~ Begin IEnhancedInputSubsystemInterface
	UE_API virtual UEnhancedPlayerInput* GetPlayerInput() const override;
protected:
	virtual TMap<TObjectPtr<const UInputAction>, FInjectedInput>& GetContinuouslyInjectedInputs() override { return ContinuouslyInjectedInputs; }
	//~ End IEnhancedInputSubsystemInterface
	
public:
	/** Pushes this input component onto the stack to be processed by this subsystem's tick function */
	UFUNCTION(BlueprintCallable, Category = "Input|Editor")
	UE_API void PushInputComponent(UInputComponent* InInputComponent);

	/** Removes this input component onto the stack to be processed by this subsystem's tick function */
	UFUNCTION(BlueprintCallable, Category = "Input|Editor")
	UE_API bool PopInputComponent(UInputComponent* InInputComponent);

	/** Start the consumption of input messages in this subsystem. This is required to have any Input Action delegates be fired. */
	UFUNCTION(BlueprintCallable, Category = "Input|Editor")
	UE_API void StartConsumingInput();

	/** Tells this subsystem to stop ticking and consuming any input. This will stop any Input Action Delegates from being called. */
	UFUNCTION(BlueprintCallable, Category = "Input|Editor")
	UE_API void StopConsumingInput();

	/** Returns true if this subsystem is currently consuming input */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Input|Editor")
	bool IsConsumingInput() const { return bIsCurrentlyConsumingInput; }

	/** Inputs a key on this subsystem's player input which can then be processed as normal during Tick. */
	UE_API bool InputKey(const FInputKeyEventArgs& Params);
	
	/** Adds all the default mapping contexts from the UEnhancedInputEditorSettings */
	UE_API void AddDefaultMappingContexts();

	/** Removes all the default mapping contexts from the UEnhancedInputEditorSettings */
	UE_API void RemoveDefaultMappingContexts();
	
private:

	/** The player input that is processing the input within this subsystem */
	UPROPERTY()
	TObjectPtr<UEnhancedPlayerInput> PlayerInput = nullptr;

	/**
	 * Input processor that is created on Initalize. This will take input from the editor and pass it through
	 * to this subsystem via InputKey.
	 */
	TSharedPtr<FEnhancedInputEditorProcessor> InputPreprocessor = nullptr;
	
	/** If true, then this subsystem will Tick and process input delegates. */
	bool bIsCurrentlyConsumingInput = false;
	
	/** Internal. This is the current stack of InputComponents that is being processed by the PlayerInput. */
	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<UInputComponent>> CurrentInputStack;

protected:
	// Map of inputs that should be injected every frame. These inputs will be injected when ForcedInput is ticked. 
	UPROPERTY(Transient) 
	TMap<TObjectPtr<const UInputAction>, FInjectedInput> ContinuouslyInjectedInputs;
};

#undef UE_API
