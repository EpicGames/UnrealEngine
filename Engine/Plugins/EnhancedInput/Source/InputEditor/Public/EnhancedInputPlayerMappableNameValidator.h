// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet2/Kismet2NameValidators.h"

class FEnhancedInputPlayerMappableNameValidator : public FStringSetNameValidator
{
public:
	FEnhancedInputPlayerMappableNameValidator(FName InExistingName);

	// Begin FNameValidatorInterface
	virtual EValidatorResult IsValid(const FString& Name, bool bOriginal) override;
	// End FNameValidatorInterface

	/**
	 * If the error code is EValidatorResult::AlreadyInUse, this
	 * returns special error text that says what asset is using the name.
	 * 
	 * Otherwise it will return INameValidatorInterface::GetErrorText.
	 */
	static FText GetErrorText(const FString& Name, EValidatorResult ErrorCode);
};