// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "Modules/ModuleInterface.h"
#include "AssetTypeCategories.h"

#include "InputEditorModule.generated.h"

#define UE_API INPUTEDITOR_API

DECLARE_LOG_CATEGORY_EXTERN(LogEnhancedInputEditor, Log, All);

////////////////////////////////////////////////////////////////////
// FInputEditorModule

class UInputAction;
class SWindow;
class UPlayerMappableKeySettings;
class IAssetTypeActions;
class IAssetTools;
class FSlateStyleSet;

class FInputEditorModule : public IModuleInterface
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface interface

	static EAssetTypeCategories::Type GetInputAssetsCategory();
	
	/** Returns a pointer to the player mappable settings object that has this mapping name */
	INPUTEDITOR_API static const UPlayerMappableKeySettings* FindMappingByName(const FName InName);

	/** Returns true if the given name is in use by a player mappable key setting */
	INPUTEDITOR_API static bool IsMappingNameInUse(const FName InName);
		
private:
	void OnMainFrameCreationFinished(TSharedPtr<SWindow> InRootWindow, bool bIsRunningStartupDialog);
	
	/** Automatically upgrade the current project to use Enhanced Input if it is currently set to the legacy input classes. */
	void AutoUpgradeDefaultInputClasses();

	static EAssetTypeCategories::Type InputAssetsCategory;
	
	TSharedPtr<FSlateStyleSet> StyleSet;
};

////////////////////////////////////////////////////////////////////
// Asset factories

UCLASS(MinimalAPI)
class UInputMappingContext_Factory : public UFactory
{
	GENERATED_UCLASS_BODY()
public:
	UPROPERTY(EditAnywhere, Category=InputMappingContext)
	TSubclassOf<class UInputMappingContext> InputMappingContextClass;

	UE_API virtual bool ConfigureProperties() override;
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;

	/** Set the array of initial actions that the resulting IMC should be populated with */
	UE_API void SetInitialActions(TArray<TWeakObjectPtr<UInputAction>> InInitialActions);
	
protected:

	/** An array of Input Actions that the mapping context should be populated with upon creation */
	TArray<TWeakObjectPtr<UInputAction>> InitialActions;
	
};

UCLASS(MinimalAPI)
class UInputAction_Factory : public UFactory
{
	GENERATED_UCLASS_BODY()
public:
	UPROPERTY(EditAnywhere, Category=InputAction)
	TSubclassOf<UInputAction> InputActionClass;

	UE_API virtual bool ConfigureProperties() override;
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};

#undef UE_API
